{
	"FileVersion": 3,
	"Version": 1,
	"VersionName": "1.0",
	"FriendlyName": "Cloners and Effectors",
	"Description": "Niagara based cloner system with various layouts and effector affecting each clone instances",
	"Category": "Virtual Production",
	"CreatedBy": "Epic Games, Inc.",
	"CreatedByURL": "https://epicgames.com",
	"DocsURL": "",
	"MarketplaceURL": "",
	"CanContainContent": true,
	"Installed": false,
	"Modules": [
		{
			"Name": "ClonerEffector",
			"Type": "Runtime",
			"LoadingPhase": "Default"
		},
		{
			"Name": "ClonerEffectorEditor",
			"Type": "Editor",
			"LoadingPhase": "Default"
		},
		{
			"Name": "ClonerEffectorMeshBuilder",
			"Type": "Runtime",
			"LoadingPhase": "Default"
		}
	],
	"Plugins": [
		{
			"Name": "GeometryMask",
			"Enabled": true
		},
		{
			"Name": "GeometryScripting",
			"Enabled": true
		},
		{
			"Name": "MeshModelingToolsetExp",
			"Enabled": true
		},
		{
			"Name": "ProceduralMeshComponent",
			"Enabled": true
		},
		{
			"Name": "Niagara",
			"Enabled": true
		},
		{
			"Name": "NiagaraSimCaching",
			"Enabled": true
		}
	]
}