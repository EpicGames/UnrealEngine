[CoreRedirects]
+ClassRedirects=(OldName="/Script/Avalanche.AvaClonerActor",NewName="/Script/ClonerEffector.CEClonerActor")
+ClassRedirects=(OldName="/Script/Avalanche.AvaClonerComponent",NewName="/Script/ClonerEffector.CEClonerComponent")
+ClassRedirects=(OldName="/Script/Avalanche.AvaEffectorActor",NewName="/Script/ClonerEffector.CEEffectorActor")
+ClassRedirects=(OldName="/Script/Avalanche.AvaEffectorComponent",NewName="/Script/ClonerEffector.CEEffectorComponent")
+ClassRedirects=(OldName="/Script/Avalanche.AvaSphereComponent",NewName="/Script/Engine.SphereComponent")
+ClassRedirects=(OldName="/Script/AvalancheEffectors.AvaSphereComponent",NewName="/Script/Engine.SphereComponent")
+ClassRedirects=(OldName="/Script/AvalancheEffectors.AvaClonerSphereLayout",NewName="/Script/ClonerEffector.CEClonerSphereUniformLayout")
+ClassRedirects=(OldName="/Script/AvalancheEffectors.AvaClonerCircleLayout",NewName="/Script/ClonerEffector.CEClonerCircleLayout")
+ClassRedirects=(OldName="/Script/AvalancheEffectors.AvaClonerCylinderLayout",NewName="/Script/ClonerEffector.CEClonerCylinderLayout")
+ClassRedirects=(OldName="/Script/AvalancheEffectors.AvaClonerGridLayout",NewName="/Script/ClonerEffector.CEClonerGridLayout")
+ClassRedirects=(OldName="/Script/AvalancheEffectors.AvaClonerHoneycombLayout",NewName="/Script/ClonerEffector.CEClonerHoneycombLayout")
+ClassRedirects=(OldName="/Script/AvalancheEffectors.AvaClonerLayoutBase",NewName="/Script/ClonerEffector.CEClonerLayoutBase")
+ClassRedirects=(OldName="/Script/AvalancheEffectors.AvaClonerLineLayout",NewName="/Script/ClonerEffector.CEClonerLineLayout")
+ClassRedirects=(OldName="/Script/AvalancheEffectors.AvaClonerMeshLayout",NewName="/Script/ClonerEffector.CEClonerMeshLayout")
+ClassRedirects=(OldName="/Script/AvalancheEffectors.AvaClonerSphereRandomLayout",NewName="/Script/ClonerEffector.CEClonerSphereRandomLayout")
+ClassRedirects=(OldName="/Script/AvalancheEffectors.AvaClonerSphereUniformLayout",NewName="/Script/ClonerEffector.CEClonerSphereUniformLayout")
+ClassRedirects=(OldName="/Script/AvalancheEffectors.AvaClonerSplineLayout",NewName="/Script/ClonerEffector.CEClonerSplineLayout")
+ClassRedirects=(OldName="/Script/AvalancheEffectors.AvaClonerActor",NewName="/Script/ClonerEffector.CEClonerActor")
+ClassRedirects=(OldName="/Script/AvalancheEffectors.AvaClonerComponent",NewName="/Script/ClonerEffector.CEClonerComponent")
+ClassRedirects=(OldName="/Script/AvalancheEffectors.AvaEffectorActor",NewName="/Script/ClonerEffector.CEEffectorActor")
+ClassRedirects=(OldName="/Script/AvalancheEffectors.AvaEffectorComponent",NewName="/Script/ClonerEffector.CEEffectorComponent")
+ClassRedirects=(OldName="/Script/AvalancheEffectors.AvaClonerSubsystem",NewName="/Script/ClonerEffector.CEClonerSubsystem")
+ClassRedirects=(OldName="/Script/AvalancheEffectors.AvaEffectorSubsystem",NewName="/Script/ClonerEffector.CEEffectorSubsystem")
+ClassRedirects=(OldName="/Script/ClonerEffector.CEEffectorDelayExtension",NewName="/Script/ClonerEffector.CEEffectorDelayEffect")
+ClassRedirects=(OldName="/Script/ClonerEffector.CEEffectorForceExtension",NewName="/Script/ClonerEffector.CEEffectorForceEffect")
+ClassRedirects=(OldName="/Script/ClonerEffector.CEEffectorNoiseMode",NewName="/Script/ClonerEffector.CEEffectorProceduralMode")

+StructRedirects=(OldName="/Script/Avalanche.AvaClonerAttachmentItem",NewName="/Script/ClonerEffector.CEClonerAttachmentItem")
+StructRedirects=(OldName="/Script/Avalanche.AvaClonerAttachmentTree",NewName="/Script/ClonerEffector.CEClonerAttachmentTree")
+StructRedirects=(OldName="/Script/Avalanche.AvaClonerEffectorDataInterfaces",NewName="/Script/ClonerEffector.CEClonerEffectorDataInterfaces")
+StructRedirects=(OldName="/Script/Avalanche.AvaClonerSampleMeshOptions",NewName="/Script/ClonerEffector.CEClonerSampleMeshOptions")
+StructRedirects=(OldName="/Script/Avalanche.AvaClonerSampleSplineOptions",NewName="/Script/ClonerEffector.CEClonerSampleSplineOptions")
+StructRedirects=(OldName="/Script/Avalanche.AvaClonerGridConstraintSphere",NewName="/Script/ClonerEffector.CEClonerGridConstraintSphere")
+StructRedirects=(OldName="/Script/Avalanche.AvaClonerGridConstraintCylinder",NewName="/Script/ClonerEffector.CEClonerGridConstraintCylinder")
+StructRedirects=(OldName="/Script/Avalanche.AvaClonerGridConstraintTexture",NewName="/Script/ClonerEffector.CEClonerGridConstraintTexture")
+StructRedirects=(OldName="/Script/Avalanche.AvaClonerGridLayoutOptions",NewName="/Script/ClonerEffector.CEClonerGridLayoutOptions")
+StructRedirects=(OldName="/Script/Avalanche.AvaClonerLineLayoutOptions",NewName="/Script/ClonerEffector.CEClonerLineLayoutOptions")
+StructRedirects=(OldName="/Script/Avalanche.AvaClonerCircleLayoutOptions",NewName="/Script/ClonerEffector.CEClonerCircleLayoutOptions")
+StructRedirects=(OldName="/Script/Avalanche.AvaClonerCylinderLayoutOptions",NewName="/Script/ClonerEffector.CEClonerCylinderLayoutOptions")
+StructRedirects=(OldName="/Script/Avalanche.AvaClonerSphereLayoutOptions",NewName="/Script/ClonerEffector.CEClonerSphereLayoutOptions")
+StructRedirects=(OldName="/Script/Avalanche.AvaClonerHoneycombLayoutOptions",NewName="/Script/ClonerEffector.CEClonerHoneycombLayoutOptions")
+StructRedirects=(OldName="/Script/AvalancheEffectors.AvaClonerAttachmentItem",NewName="/Script/ClonerEffector.CEClonerAttachmentItem")
+StructRedirects=(OldName="/Script/AvalancheEffectors.AvaClonerAttachmentTree",NewName="/Script/ClonerEffector.CEClonerAttachmentTree")
+StructRedirects=(OldName="/Script/AvalancheEffectors.AvaClonerEffectorDataInterfaces",NewName="/Script/ClonerEffector.CEClonerEffectorDataInterfaces")
+StructRedirects=(OldName="/Script/AvalancheEffectors.AvaClonerSampleMeshOptions",NewName="/Script/ClonerEffector.CEClonerSampleMeshOptions")
+StructRedirects=(OldName="/Script/AvalancheEffectors.AvaClonerSampleSplineOptions",NewName="/Script/ClonerEffector.CEClonerSampleSplineOptions")
+StructRedirects=(OldName="/Script/AvalancheEffectors.AvaClonerGridConstraintSphere",NewName="/Script/ClonerEffector.CEClonerGridConstraintSphere")
+StructRedirects=(OldName="/Script/AvalancheEffectors.AvaClonerGridConstraintCylinder",NewName="/Script/ClonerEffector.CEClonerGridConstraintCylinder")
+StructRedirects=(OldName="/Script/AvalancheEffectors.AvaClonerGridConstraintTexture",NewName="/Script/ClonerEffector.CEClonerGridConstraintTexture")
+StructRedirects=(OldName="/Script/AvalancheEffectors.AvaClonerGridLayoutOptions",NewName="/Script/ClonerEffector.CEClonerGridLayoutOptions")
+StructRedirects=(OldName="/Script/AvalancheEffectors.AvaClonerLineLayoutOptions",NewName="/Script/ClonerEffector.CEClonerLineLayoutOptions")
+StructRedirects=(OldName="/Script/AvalancheEffectors.AvaClonerCircleLayoutOptions",NewName="/Script/ClonerEffector.CEClonerCircleLayoutOptions")
+StructRedirects=(OldName="/Script/AvalancheEffectors.AvaClonerCylinderLayoutOptions",NewName="/Script/ClonerEffector.CEClonerCylinderLayoutOptions")
+StructRedirects=(OldName="/Script/AvalancheEffectors.AvaClonerSphereLayoutOptions",NewName="/Script/ClonerEffector.CEClonerSphereLayoutOptions")
+StructRedirects=(OldName="/Script/AvalancheEffectors.AvaClonerHoneycombLayoutOptions",NewName="/Script/ClonerEffector.CEClonerHoneycombLayoutOptions")

+PropertyRedirects=(OldName="/Script/AvalancheEffectors.AvaClonerActor.ForceOrientationRate",NewName="/Script/ClonerEffector.CEClonerActor.OrientationForceRate")
+PropertyRedirects=(OldName="/Script/AvalancheEffectors.AvaClonerActor.ForceOrientationMin",NewName="/Script/ClonerEffector.CEClonerActor.OrientationForceMin")
+PropertyRedirects=(OldName="/Script/AvalancheEffectors.AvaClonerActor.ForceOrientationMax",NewName="/Script/ClonerEffector.CEClonerActor.OrientationForceMax")
+PropertyRedirects=(OldName="/Script/AvalancheEffectors.AvaClonerActor.ForceVortexAmount",NewName="/Script/ClonerEffector.CEClonerActor.VortexForceAmount")
+PropertyRedirects=(OldName="/Script/AvalancheEffectors.AvaClonerActor.ForceVortexAxis",NewName="/Script/ClonerEffector.CEClonerActor.VortexForceAxis")
+PropertyRedirects=(OldName="/Script/AvalancheEffectors.AvaClonerActor.ForceCurlNoiseStrength",NewName="/Script/ClonerEffector.CEClonerActor.CurlNoiseForceStrength")
+PropertyRedirects=(OldName="/Script/AvalancheEffectors.AvaClonerActor.ForceCurlNoiseFrequency",NewName="/Script/ClonerEffector.CEClonerActor.CurlNoiseForceFrequency")
+PropertyRedirects=(OldName="/Script/AvalancheEffectors.AvaClonerActor.StepRotationDelta",NewName="/Script/ClonerEffector.CEClonerActor.DeltaStepRotation")
+PropertyRedirects=(OldName="/Script/AvalancheEffectors.AvaClonerActor.StepScaleDelta",NewName="/Script/ClonerEffector.CEClonerActor.DeltaStepScale")
+PropertyRedirects=(OldName="/Script/AvalancheEffectors.AvaEffectorActor.Strength",NewName="/Script/ClonerEffector.CEEffectorActor.LocationStrength")

+EnumRedirects=(OldName="/Script/AvalancheEffectors.EAvaClonerLayout",NewName="/Script/ClonerEffector.ECEClonerLayout")
+EnumRedirects=(OldName="/Script/AvalancheEffectors.EAvaClonerAxis",NewName="/Script/ClonerEffector.ECEClonerAxis")
+EnumRedirects=(OldName="/Script/AvalancheEffectors.EAvaClonerPlane",NewName="/Script/ClonerEffector.ECEClonerPlane")
+EnumRedirects=(OldName="/Script/AvalancheEffectors.EAvaClonerMeshRenderMode",NewName="/Script/ClonerEffector.ECEClonerMeshRenderMode")
+EnumRedirects=(OldName="/Script/AvalancheEffectors.EAvaClonerGridConstraint",NewName="/Script/ClonerEffector.ECEClonerGridConstraint")
+EnumRedirects=(OldName="/Script/AvalancheEffectors.EAvaClonerEasing",NewName="/Script/ClonerEffector.ECEClonerEasing")
+EnumRedirects=(OldName="/Script/AvalancheEffectors.EAvaClonerMeshAsset",NewName="/Script/ClonerEffector.ECEClonerMeshAsset")
+EnumRedirects=(OldName="/Script/AvalancheEffectors.EAvaClonerMeshSampleData",NewName="/Script/ClonerEffector.ECEClonerMeshSampleData")
+EnumRedirects=(OldName="/Script/AvalancheEffectors.EAvaClonerEffectorType",NewName="/Script/ClonerEffector.ECEClonerEffectorType")
+EnumRedirects=(OldName="/Script/AvalancheEffectors.EAvaClonerEffectorMode",NewName="/Script/ClonerEffector.ECEClonerEffectorMode")
+EnumRedirects=(OldName="/Script/AvalancheEffectors.EAvaClonerAttachmentStatus",NewName="/Script/ClonerEffector.ECEClonerAttachmentStatus")
+EnumRedirects=(OldName="/Script/AvalancheEffectors.EAvaClonerTextureSampleChannel",NewName="/Script/ClonerEffector.ECEClonerTextureSampleChannel")
+EnumRedirects=(OldName="/Script/AvalancheEffectors.EAvaClonerCompareMode",NewName="/Script/ClonerEffector.ECEClonerCompareMode")
+EnumRedirects=(OldName="/Script/AvalancheEffectors.EAvaClonerSpawnLoopMode",NewName="/Script/ClonerEffector.ECEClonerSpawnLoopMode")
+EnumRedirects=(OldName="/Script/AvalancheEffectors.EAvaClonerSpawnBehaviorMode",NewName="/Script/ClonerEffector.ECEClonerSpawnBehaviorMode")
+EnumRedirects=(OldName="/Script/Avalanche.EAvaClonerLayout",NewName="/Script/ClonerEffector.ECEClonerLayout")
+EnumRedirects=(OldName="/Script/Avalanche.EAvaClonerAxis",NewName="/Script/ClonerEffector.ECEClonerAxis")
+EnumRedirects=(OldName="/Script/Avalanche.EAvaClonerPlane",NewName="/Script/ClonerEffector.ECEClonerPlane")
+EnumRedirects=(OldName="/Script/Avalanche.EAvaClonerMeshRenderMode",NewName="/Script/ClonerEffector.ECEClonerMeshRenderMode")
+EnumRedirects=(OldName="/Script/Avalanche.EAvaClonerGridConstraint",NewName="/Script/ClonerEffector.ECEClonerGridConstraint")
+EnumRedirects=(OldName="/Script/Avalanche.EAvaClonerEasing",NewName="/Script/ClonerEffector.ECEClonerEasing")
+EnumRedirects=(OldName="/Script/Avalanche.EAvaClonerMeshAsset",NewName="/Script/ClonerEffector.ECEClonerMeshAsset")
+EnumRedirects=(OldName="/Script/Avalanche.EAvaClonerMeshSampleData",NewName="/Script/ClonerEffector.ECEClonerMeshSampleData")
+EnumRedirects=(OldName="/Script/Avalanche.EAvaClonerEffectorType",NewName="/Script/ClonerEffector.ECEClonerEffectorType")
+EnumRedirects=(OldName="/Script/Avalanche.EAvaClonerEffectorMode",NewName="/Script/ClonerEffector.ECEClonerEffectorMode")
+EnumRedirects=(OldName="/Script/Avalanche.EAvaClonerAttachmentStatus",NewName="/Script/ClonerEffector.ECEClonerAttachmentStatus")
+EnumRedirects=(OldName="/Script/Avalanche.EAvaClonerTextureSampleChannel",NewName="/Script/ClonerEffector.ECEClonerTextureSampleChannel")
+EnumRedirects=(OldName="/Script/Avalanche.EAvaClonerCompareMode",NewName="/Script/ClonerEffector.ECEClonerCompareMode")
+EnumRedirects=(OldName="/Script/ClonerEffector.ECEClonerEffectorMode",ValueChanges=(("NoiseField","Procedural")))

+FunctionRedirects=(OldName="/Script/ClonerEffector.CEEffectorComponent.GetActiveExtensions",NewName="/Script/ClonerEffector.CEEffectorComponent.GetActiveEffects")
+FunctionRedirects=(OldName="/Script/ClonerEffector.CEEffectorLibrary.GetEffectorExtensionClasses",NewName="/Script/ClonerEffector.CEEffectorLibrary.GetEffectorEffectClasses")
