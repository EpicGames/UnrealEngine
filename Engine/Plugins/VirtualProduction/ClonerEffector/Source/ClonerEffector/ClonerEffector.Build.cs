// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class ClonerEffector : ModuleRules
{
	public ClonerEffector(ReadOnlyTargetRules Target) : base(Target)
	{
		PCHUsage = ModuleRules.PCHUsageMode.UseExplicitOrSharedPCHs;
		
		PublicDependencyModuleNames.AddRange(
			new string[]
			{
				"ClonerEffectorMeshBuilder",
				"Core",
				"CoreUObject",
				"Engine",
				"Niagara"
			}
		);

		PrivateDependencyModuleNames.AddRange(
			new string[]
			{
				"DeveloperSettings",
				"GeometryCore",
				"GeometryFramework",
				"GeometryScriptingCore",
				"NiagaraCore"
			}
		);

		if (Target.Type == TargetRules.TargetType.Editor)
		{
			PrivateDependencyModuleNames.AddRange(new string[]
			{
				"UnrealEd",
				"Slate",
				"SlateCore"
			});
		}

		ShortName = "CE";
	}
}
