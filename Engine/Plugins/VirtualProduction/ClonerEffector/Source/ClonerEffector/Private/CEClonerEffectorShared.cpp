// Copyright Epic Games, Inc. All Rights Reserved.

#include "CEClonerEffectorShared.h"

#include "Cloner/CEClonerComponent.h"
#include "Components/InstancedStaticMeshComponent.h"
#include "DataInterface/NiagaraDataInterfaceArrayMesh.h"
#include "DynamicMeshActor.h"
#include "Engine/StaticMeshActor.h"
#include "Engine/Texture2D.h"
#include "Engine/World.h"
#include "NiagaraDataChannelAccessor.h"
#include "NiagaraFunctionLibrary.h"
#include "PrimitiveDrawingUtils.h"
#include "PrimitiveSceneProxy.h"
#include "UObject/UObjectGlobals.h"

#if WITH_EDITOR
#include "AssetToolsModule.h"
#include "AssetRegistry/AssetRegistryModule.h"
#include "Dialogs/DlgPickPath.h"
#include "Editor/EditorEngine.h"
#include "UObject/Package.h"
#endif

#define LOCTEXT_NAMESPACE "CEClonerEffectorShared"

void FCEClonerEffectorChannelData::Write(UNiagaraDataChannelWriter* InWriter) const
{
	if (!InWriter)
	{
		return;
	}

	/** General */
	InWriter->WriteFloat(MagnitudeName, Identifier, Magnitude);
	InWriter->WritePosition(LocationName, Identifier, Location);
	InWriter->WriteQuat(RotationName, Identifier, Rotation);
	InWriter->WriteVector(ScaleName, Identifier, Scale);
	InWriter->WriteLinearColor(ColorName, Identifier, Color);

	/** Type/Shape */
	InWriter->WriteInt(TypeName, Identifier, static_cast<int32>(Type));
	InWriter->WriteInt(EasingName, Identifier, static_cast<int32>(Easing));
	InWriter->WriteVector(InnerExtentName, Identifier, InnerExtent);
	InWriter->WriteVector(OuterExtentName, Identifier, OuterExtent);
	
	/** Mode */
	InWriter->WriteInt(ModeName, Identifier, static_cast<int32>(Mode));
	InWriter->WriteVector(LocationDeltaName, Identifier, LocationDelta);
	InWriter->WriteVector(RotationDeltaName, Identifier, RotationDelta);
	InWriter->WriteVector(ScaleDeltaName, Identifier, ScaleDelta);
	InWriter->WriteFloat(FrequencyName, Identifier, Frequency);
	InWriter->WriteVector(PanName, Identifier, Pan);
	InWriter->WriteInt(PatternName, Identifier, static_cast<int32>(Pattern));
	
	/** Delay */
	InWriter->WriteFloat(DelayInDurationName, Identifier, DelayInDuration);
	InWriter->WriteFloat(DelayOutDurationName, Identifier, DelayOutDuration);
	InWriter->WriteFloat(DelaySpringFrequencyName, Identifier, DelaySpringFrequency);
	InWriter->WriteFloat(DelaySpringFalloffName, Identifier, DelaySpringFalloff);

	/** Forces */
	InWriter->WriteFloat(OrientationForceRateName, Identifier, OrientationForceRate);
	InWriter->WriteVector(OrientationForceMinName, Identifier, OrientationForceMin);
	InWriter->WriteVector(OrientationForceMaxName, Identifier, OrientationForceMax);
	InWriter->WriteFloat(VortexForceAmountName, Identifier, VortexForceAmount);
	InWriter->WriteVector(VortexForceAxisName, Identifier, VortexForceAxis);
	InWriter->WriteFloat(CurlNoiseForceStrengthName, Identifier, CurlNoiseForceStrength);
	InWriter->WriteFloat(CurlNoiseForceFrequencyName, Identifier, CurlNoiseForceFrequency);
	InWriter->WriteFloat(AttractionForceStrengthName, Identifier, AttractionForceStrength);
	InWriter->WriteFloat(AttractionForceFalloffName, Identifier, AttractionForceFalloff);
	InWriter->WriteVector(GravityForceAccelerationName, Identifier, GravityForceAcceleration);
	InWriter->WriteFloat(DragForceLinearName, Identifier, DragForceLinear);
	InWriter->WriteFloat(DragForceRotationalName, Identifier, DragForceRotational);
	InWriter->WriteFloat(VectorNoiseForceAmountName, Identifier, VectorNoiseForceAmount);
}

void FCEClonerEffectorChannelData::Read(const UNiagaraDataChannelReader* InReader)
{
	if (!InReader)
	{
		return;
	}
	
	bool bIsValid;

	/** General */
	Magnitude = InReader->ReadFloat(MagnitudeName, Identifier, bIsValid);
	Location = InReader->ReadPosition(LocationName, Identifier, bIsValid);
	Rotation = InReader->ReadQuat(RotationName, Identifier, bIsValid);
	Scale = InReader->ReadVector(ScaleName, Identifier, bIsValid);
	Color = InReader->ReadLinearColor(ColorName, Identifier, bIsValid);

	/** Type/Shape */
	Type = static_cast<ECEClonerEffectorType>(InReader->ReadInt(TypeName, Identifier, bIsValid));
	Easing = static_cast<ECEClonerEasing>(InReader->ReadInt(EasingName, Identifier, bIsValid));
	InnerExtent = InReader->ReadVector(InnerExtentName, Identifier, bIsValid);
	OuterExtent = InReader->ReadVector(OuterExtentName, Identifier, bIsValid);
	
	/** Mode */
	Mode = static_cast<ECEClonerEffectorMode>(InReader->ReadInt(ModeName, Identifier, bIsValid));
	LocationDelta = InReader->ReadVector(LocationDeltaName, Identifier, bIsValid);
	RotationDelta = InReader->ReadVector(RotationDeltaName, Identifier, bIsValid);
	ScaleDelta = InReader->ReadVector(ScaleDeltaName, Identifier, bIsValid);
	Frequency = InReader->ReadFloat(FrequencyName, Identifier, bIsValid);
	Pan = InReader->ReadVector(PanName, Identifier, bIsValid);
	Pattern = static_cast<ECEClonerEffectorProceduralPattern>(InReader->ReadInt(PatternName, Identifier, bIsValid));
	
	/** Delay */
	DelayInDuration = InReader->ReadFloat(DelayInDurationName, Identifier, bIsValid);
	DelayOutDuration = InReader->ReadFloat(DelayOutDurationName, Identifier, bIsValid);
	DelaySpringFrequency = InReader->ReadFloat(DelaySpringFrequencyName, Identifier, bIsValid);
	DelaySpringFalloff = InReader->ReadFloat(DelaySpringFalloffName, Identifier, bIsValid);

	/** Forces */
	OrientationForceRate = InReader->ReadFloat(OrientationForceRateName, Identifier, bIsValid);
	OrientationForceMin = InReader->ReadVector(OrientationForceMinName, Identifier, bIsValid);
	OrientationForceMax = InReader->ReadVector(OrientationForceMaxName, Identifier, bIsValid);
	VortexForceAmount = InReader->ReadFloat(VortexForceAmountName, Identifier, bIsValid);
	VortexForceAxis = InReader->ReadVector(VortexForceAxisName, Identifier, bIsValid);
	CurlNoiseForceStrength = InReader->ReadFloat(CurlNoiseForceStrengthName, Identifier, bIsValid);
	CurlNoiseForceFrequency = InReader->ReadFloat(CurlNoiseForceFrequencyName, Identifier, bIsValid);
	AttractionForceStrength = InReader->ReadFloat(AttractionForceStrengthName, Identifier, bIsValid);
	AttractionForceFalloff = InReader->ReadFloat(AttractionForceFalloffName, Identifier, bIsValid);
	GravityForceAcceleration = InReader->ReadVector(GravityForceAccelerationName, Identifier, bIsValid);
	DragForceLinear = InReader->ReadFloat(DragForceLinearName, Identifier, bIsValid);
	DragForceRotational = InReader->ReadFloat(DragForceRotationalName, Identifier, bIsValid);
	VectorNoiseForceAmount = InReader->ReadFloat(VectorNoiseForceAmountName, Identifier, bIsValid);
}

UNiagaraDataInterfaceArrayMesh* UE::ClonerEffector::FindMeshArrayDataInterface(UCEClonerComponent* InClonerComponent)
{
	if (!InClonerComponent || !InClonerComponent->GetAsset())
	{
		return nullptr;
	}
	return UNiagaraFunctionLibrary::GetDataInterface<UNiagaraDataInterfaceArrayMesh>(InClonerComponent, UE::ClonerEffector::MeshArrayName);
}

#if WITH_EDITOR
FCEExtensionSection UE::ClonerEffector::EditorSection::GetExtensionSectionFromClass(UClass* InClass)
{
	FCEExtensionSection Section;

	if (!InClass)
	{
		return Section;
	}

	while (InClass && !InClass->HasMetaData(TEXT("Section")))
	{
		InClass = InClass->GetSuperClass();
	}

	if (InClass)
	{
		Section.SectionName = FName(InClass->GetMetaData(TEXT("Section")));
		Section.SectionOrder = InClass->GetIntMetaData(TEXT("Priority"));
	}

	return Section;
}
#endif

AStaticMeshActor* UE::ClonerEffector::Conversion::ConvertClonerToStaticMesh(UCEClonerComponent* InCloner)
{
	AStaticMeshActor* NewActor = nullptr;

	if (!IsValid(InCloner))
	{
		return NewActor;
	}

	UWorld* World = InCloner->GetWorld();
	AActor* Owner = InCloner->GetOwner();

	if (!IsValid(World)
		|| !IsValid(Owner)
#if WITH_EDITOR
		|| Owner->bIsEditorPreviewActor
#endif
		)
	{
		return NewActor;
	}

	FCEMeshBuilder ClonerMeshBuilder;
	const FTransform ClonerTransform = InCloner->GetComponentTransform();
	if (!ClonerMeshBuilder.AppendComponent(InCloner, ClonerTransform))
	{
		return NewActor;
	}

	if (ClonerMeshBuilder.GetMeshInstanceCount() == 0)
	{
		return NewActor;
	}

#if WITH_EDITOR
	UPackage* ClonerPackage = InCloner->GetPackage();

	if (!ClonerPackage)
	{
		return NewActor;
	}

	FString PackagePath;
	if (!PickAssetPath(ClonerPackage->GetLoadedPath().GetPackageName(), PackagePath))
	{
		return NewActor;
	}
#endif

	// Spawn attached actor with same flags as this actor
	FActorSpawnParameters SpawnParameters;
	SpawnParameters.Owner = Owner;
	SpawnParameters.ObjectFlags = InCloner->GetFlags();
#if WITH_EDITOR
	SpawnParameters.bTemporaryEditorActor = false;
#endif

	NewActor = World->SpawnActor<AStaticMeshActor>(AStaticMeshActor::StaticClass(), ClonerTransform, SpawnParameters);

	if (!NewActor)
	{
		return NewActor;
	}

	NewActor->SetMobility(EComponentMobility::Movable);

	UStaticMeshComponent* StaticMeshComponent = NewActor->GetStaticMeshComponent();

	UStaticMesh* StaticMesh = nullptr;

#if WITH_EDITOR
	const FString AssetPath = PackagePath + TEXT("SM_") + Owner->GetActorNameOrLabel() + TEXT("_Merged_") + FString::FromInt(NewActor->GetUniqueID());
	StaticMesh = CreateAssetPackage<UStaticMesh>(AssetPath);
#else
	StaticMesh = NewObject<UStaticMesh>(StaticMeshComponent);
#endif

	if (!StaticMesh)
	{
		return NewActor;
	}

	TArray<TWeakObjectPtr<UMaterialInterface>> MeshMaterials;
	FCEMeshBuilder::FCEMeshBuilderParams Params;
	Params.bMergeMaterials = true;
	ClonerMeshBuilder.BuildStaticMesh(StaticMesh, MeshMaterials, Params);

#if WITH_EDITOR
	// Replace material references that are no assets to avoid save issue in new packages
	for (int32 Index = 0; Index < StaticMesh->GetNumSections(/** LOD */0); Index++)
	{
		UMaterialInterface* Material = StaticMesh->GetMaterial(Index);

		if (Material && !Material->IsAsset())
		{
			Material = nullptr;
		}

		StaticMesh->SetMaterial(Index, Material);
	}
#endif

	StaticMeshComponent->SetStaticMesh(StaticMesh);

	for (int32 Index = 0; Index < MeshMaterials.Num(); Index++)
	{
		StaticMeshComponent->SetMaterial(Index, MeshMaterials[Index].Get());
	}

#if WITH_EDITOR
	FActorLabelUtilities::SetActorLabelUnique(NewActor, Owner->GetActorNameOrLabel() + TEXT("_SM_Merged"));
#endif

	return NewActor;
}

ADynamicMeshActor* UE::ClonerEffector::Conversion::ConvertClonerToDynamicMesh(UCEClonerComponent* InCloner)
{
	ADynamicMeshActor* NewActor = nullptr;

	if (!IsValid(InCloner))
	{
		return NewActor;
	}

	UWorld* World = InCloner->GetWorld();
	AActor* Owner = InCloner->GetOwner();

	if (!IsValid(World)
		|| !IsValid(Owner)
#if WITH_EDITOR
		|| Owner->bIsEditorPreviewActor
#endif
		)
	{
		return NewActor;
	}

	FCEMeshBuilder ClonerMeshBuilder;
	const FTransform ClonerTransform = InCloner->GetComponentTransform();
	if (!ClonerMeshBuilder.AppendComponent(InCloner, ClonerTransform))
	{
		return NewActor;
	}

	if (ClonerMeshBuilder.GetMeshInstanceCount() == 0)
	{
		return NewActor;
	}

	// Spawn attached actor with same flags as this actor
	FActorSpawnParameters SpawnParameters;
	SpawnParameters.Owner = Owner;
	SpawnParameters.ObjectFlags = InCloner->GetFlags();
#if WITH_EDITOR
	SpawnParameters.bTemporaryEditorActor = false;
#endif

	NewActor = World->SpawnActor<ADynamicMeshActor>(ADynamicMeshActor::StaticClass(), ClonerTransform, SpawnParameters);

	if (!NewActor)
	{
		return NewActor;
	}

	UDynamicMeshComponent* DynamicMeshComponent = NewActor->GetDynamicMeshComponent();

	TArray<TWeakObjectPtr<UMaterialInterface>> MeshMaterials;
	FCEMeshBuilder::FCEMeshBuilderParams Params;
	Params.bMergeMaterials = true;
	ClonerMeshBuilder.BuildDynamicMesh(DynamicMeshComponent->GetDynamicMesh(), MeshMaterials, Params);

	for (int32 Index = 0; Index < MeshMaterials.Num(); Index++)
	{
		DynamicMeshComponent->SetMaterial(Index, MeshMaterials[Index].Get());
	}

#if WITH_EDITOR
	FActorLabelUtilities::SetActorLabelUnique(NewActor, Owner->GetActorNameOrLabel() + TEXT("_DM_Merged"));
#endif

	return NewActor;
}

TArray<AStaticMeshActor*> UE::ClonerEffector::Conversion::ConvertClonerToStaticMeshes(UCEClonerComponent* InCloner)
{
	TArray<AStaticMeshActor*> NewActors;

	if (!IsValid(InCloner))
	{
		return NewActors;
	}

	UWorld* World = InCloner->GetWorld();
	AActor* Owner = InCloner->GetOwner();

	if (!IsValid(World)
		|| !IsValid(Owner)
#if WITH_EDITOR
		|| Owner->bIsEditorPreviewActor
#endif
		)
	{
		return NewActors;
	}

	FCEMeshBuilder ClonerMeshBuilder;
	const FTransform ClonerTransform = InCloner->GetComponentTransform();
	if (!ClonerMeshBuilder.AppendComponent(InCloner, ClonerTransform))
	{
		return NewActors;
	}

	if (ClonerMeshBuilder.GetMeshInstanceCount() == 0)
	{
		return NewActors;
	}

#if WITH_EDITOR
	UPackage* ClonerPackage = InCloner->GetPackage();

	if (!ClonerPackage)
	{
		return NewActors;
	}

	FString PackagePath;
	if (!PickAssetPath(ClonerPackage->GetLoadedPath().GetPackageName(), PackagePath))
	{
		return NewActors;
	}
#endif

	// Spawn attached actor with same flags as this actor
	FActorSpawnParameters SpawnParameters;
	SpawnParameters.Owner = Owner;
	SpawnParameters.ObjectFlags = InCloner->GetFlags();
#if WITH_EDITOR
	SpawnParameters.bTemporaryEditorActor = false;
#endif

	// Create a Group Actor to hold all actors related to this operation
	AActor* GroupActor = World->SpawnActor<AActor>(SpawnParameters);

	if (!GroupActor)
	{
		return NewActors;
	}

	CreateRootComponent(GroupActor, USceneComponent::StaticClass(), ClonerTransform);

#if WITH_EDITOR
	FActorLabelUtilities::SetActorLabelUnique(GroupActor, Owner->GetActorNameOrLabel() + TEXT("_SM_Instances"));
#endif

	NewActors.Reserve(ClonerMeshBuilder.GetMeshInstanceCount());

#if WITH_EDITOR
	for (uint32 MeshIndex : ClonerMeshBuilder.GetMeshIndexes())
	{
		UStaticMesh* StaticMesh = nullptr;

		const FString AssetPath = PackagePath + TEXT("SM_") + Owner->GetActorNameOrLabel() + TEXT("_") + FString::FromInt(MeshIndex);
		StaticMesh = CreateAssetPackage<UStaticMesh>(AssetPath);

		if (!StaticMesh)
		{
			continue;
		}

		TArray<FCEMeshBuilder::FCEMeshInstanceData> Instances;
		ClonerMeshBuilder.BuildStaticMesh(MeshIndex, StaticMesh, Instances);

		// Replace material references that are no assets to avoid save issue in new packages
		for (int32 Index = 0; Index < StaticMesh->GetNumSections(/** LOD */0); Index++)
		{
			UMaterialInterface* Material = StaticMesh->GetMaterial(Index);

			if (Material && !Material->IsAsset())
			{
				Material = nullptr;
			}

			StaticMesh->SetMaterial(Index, Material);
		}

		for (const FCEMeshBuilder::FCEMeshInstanceData& Instance : Instances)
		{
			if (AStaticMeshActor* StaticMeshActor = World->SpawnActor<AStaticMeshActor>(AStaticMeshActor::StaticClass(), ClonerTransform, SpawnParameters))
			{
				StaticMeshActor->SetMobility(EComponentMobility::Movable);

				UStaticMeshComponent* StaticMeshComponent = StaticMeshActor->GetStaticMeshComponent();

				StaticMeshComponent->SetStaticMesh(StaticMesh);
				StaticMeshActor->SetActorTransform(Instance.Transform);

				for (int32 MaterialIndex = 0; MaterialIndex < Instance.MeshMaterials.Num(); MaterialIndex++)
				{
					StaticMeshComponent->SetMaterial(MaterialIndex, Instance.MeshMaterials[MaterialIndex].Get());
				}

				StaticMeshActor->AttachToActor(GroupActor, FAttachmentTransformRules::KeepRelativeTransform);

				FActorLabelUtilities::SetActorLabelUnique(StaticMeshActor, Owner->GetActorNameOrLabel() + TEXT("_SM_Instance"));

				NewActors.Add(StaticMeshActor);
			}
		}
	}

#else

	for (int32 Index = 0; Index < ClonerMeshBuilder.GetMeshInstanceCount(); Index++)
	{
		if (AStaticMeshActor* StaticMeshActor = World->SpawnActor<AStaticMeshActor>(AStaticMeshActor::StaticClass(), ClonerTransform, SpawnParameters))
		{
			StaticMeshActor->SetMobility(EComponentMobility::Movable);

			UStaticMeshComponent* StaticMeshComponent = StaticMeshActor->GetStaticMeshComponent();

			UStaticMesh* StaticMesh = NewObject<UStaticMesh>(StaticMeshComponent);

			FCEMeshBuilder::FCEMeshInstanceData MeshData;
			ClonerMeshBuilder.BuildStaticMesh(Index, StaticMesh, MeshData);

			StaticMeshComponent->SetStaticMesh(StaticMesh);
			StaticMeshActor->SetActorTransform(MeshData.Transform);

			for (int32 MaterialIndex = 0; MaterialIndex < MeshData.MeshMaterials.Num(); MaterialIndex++)
			{
				StaticMeshComponent->SetMaterial(MaterialIndex, MeshData.MeshMaterials[MaterialIndex].Get());
			}

			StaticMeshActor->AttachToActor(GroupActor, FAttachmentTransformRules::KeepRelativeTransform);

			NewActors.Add(StaticMeshActor);
		}
	}
#endif

	return NewActors;
}

TArray<ADynamicMeshActor*> UE::ClonerEffector::Conversion::ConvertClonerToDynamicMeshes(UCEClonerComponent* InCloner)
{
	TArray<ADynamicMeshActor*> NewActors;

	if (!IsValid(InCloner))
	{
		return NewActors;
	}

	UWorld* World = InCloner->GetWorld();
	AActor* Owner = InCloner->GetOwner();

	if (!IsValid(World)
		|| !IsValid(Owner)
#if WITH_EDITOR
		|| Owner->bIsEditorPreviewActor
#endif
		)
	{
		return NewActors;
	}

	FCEMeshBuilder ClonerMeshBuilder;
	const FTransform ClonerTransform = InCloner->GetComponentTransform();
	if (!ClonerMeshBuilder.AppendComponent(InCloner, ClonerTransform))
	{
		return NewActors;
	}

	if (ClonerMeshBuilder.GetMeshInstanceCount() == 0)
	{
		return NewActors;
	}

	// Spawn attached actor with same flags as this actor
	FActorSpawnParameters SpawnParameters;
	SpawnParameters.Owner = Owner;
	SpawnParameters.ObjectFlags = InCloner->GetFlags();
#if WITH_EDITOR
	SpawnParameters.bTemporaryEditorActor = false;
#endif

	// Create a Group Actor to hold all actors related to this operation
	AActor* GroupActor = World->SpawnActor<AActor>(SpawnParameters);

	if (!GroupActor)
	{
		return NewActors;
	}

	CreateRootComponent(GroupActor, USceneComponent::StaticClass(), ClonerTransform);

#if WITH_EDITOR
	FActorLabelUtilities::SetActorLabelUnique(GroupActor, Owner->GetActorNameOrLabel() + TEXT("_DM_Instances"));
#endif

	for (int32 Index = 0; Index < ClonerMeshBuilder.GetMeshInstanceCount(); Index++)
	{
		if (ADynamicMeshActor* DynamicMeshActor = World->SpawnActor<ADynamicMeshActor>(ADynamicMeshActor::StaticClass(), ClonerTransform, SpawnParameters))
		{
			UDynamicMeshComponent* DynamicMeshComponent = DynamicMeshActor->GetDynamicMeshComponent();

			FCEMeshBuilder::FCEMeshInstanceData MeshData;
			ClonerMeshBuilder.BuildDynamicMesh(Index, DynamicMeshComponent->GetDynamicMesh(), MeshData);

			DynamicMeshActor->SetActorTransform(MeshData.Transform);

			for (int32 MaterialIndex = 0; MaterialIndex < MeshData.MeshMaterials.Num(); MaterialIndex++)
			{
				DynamicMeshComponent->SetMaterial(MaterialIndex, MeshData.MeshMaterials[MaterialIndex].Get());
			}

			DynamicMeshActor->AttachToActor(GroupActor, FAttachmentTransformRules::KeepRelativeTransform);

#if WITH_EDITOR
			FActorLabelUtilities::SetActorLabelUnique(DynamicMeshActor, Owner->GetActorNameOrLabel() + TEXT("_DM_Instance"));
#endif

			NewActors.Add(DynamicMeshActor);
		}
	}

	return NewActors;
}

TArray<AActor*> UE::ClonerEffector::Conversion::ConvertClonerToInstancedStaticMeshes(UCEClonerComponent* InCloner)
{
	TArray<AActor*> NewActors;

	if (!IsValid(InCloner))
	{
		return NewActors;
	}

	UWorld* World = InCloner->GetWorld();
	AActor* Owner = InCloner->GetOwner();

	if (!IsValid(World)
		|| !IsValid(Owner)
#if WITH_EDITOR
		|| Owner->bIsEditorPreviewActor
#endif
		)
	{
		return NewActors;
	}

	FCEMeshBuilder ClonerMeshBuilder;
	const FTransform ClonerTransform = InCloner->GetComponentTransform();
	if (!ClonerMeshBuilder.AppendComponent(InCloner, ClonerTransform))
	{
		return NewActors;
	}

	if (ClonerMeshBuilder.GetMeshInstanceCount() == 0)
	{
		return NewActors;
	}

#if WITH_EDITOR
	UPackage* ClonerPackage = InCloner->GetPackage();

	if (!ClonerPackage)
	{
		return NewActors;
	}

	FString PackagePath;
	if (!PickAssetPath(ClonerPackage->GetLoadedPath().GetPackageName(), PackagePath))
	{
		return NewActors;
	}
#endif

	// Spawn attached actor with same flags as this actor
	FActorSpawnParameters SpawnParameters;
	SpawnParameters.Owner = Owner;
	SpawnParameters.ObjectFlags = InCloner->GetFlags();
#if WITH_EDITOR
	SpawnParameters.bTemporaryEditorActor = false;
#endif

	// Create a Group Actor to hold all actors related to this operation
	AActor* GroupActor = World->SpawnActor<AActor>(SpawnParameters);

	if (!GroupActor)
	{
		return NewActors;
	}

	CreateRootComponent(GroupActor, USceneComponent::StaticClass(), ClonerTransform);

#if WITH_EDITOR
	FActorLabelUtilities::SetActorLabelUnique(GroupActor, Owner->GetActorNameOrLabel() + TEXT("_ISM_Instances"));
#endif

	for (uint32 MeshIndex : ClonerMeshBuilder.GetMeshIndexes())
	{
		if (AActor* ISMActor = World->SpawnActor<AActor>(SpawnParameters))
		{
			UInstancedStaticMeshComponent* ISMComponent = Cast<UInstancedStaticMeshComponent>(CreateRootComponent(ISMActor, UInstancedStaticMeshComponent::StaticClass(), ClonerTransform));

			UStaticMesh* StaticMesh = nullptr;

#if WITH_EDITOR
			const FString AssetPath = PackagePath + TEXT("SM_") + Owner->GetActorNameOrLabel() + TEXT("_") + FString::FromInt(MeshIndex);
			StaticMesh = CreateAssetPackage<UStaticMesh>(AssetPath);
#else
			StaticMesh = NewObject<UStaticMesh>(ISMComponent);
#endif

			if (!StaticMesh)
			{
				continue;
			}

			TArray<FCEMeshBuilder::FCEMeshInstanceData> Instances;
			ClonerMeshBuilder.BuildStaticMesh(MeshIndex, StaticMesh, Instances);

#if WITH_EDITOR
			// Replace material references that are no assets to avoid save issue in new packages
			for (int32 Index = 0; Index < StaticMesh->GetNumSections(/** LOD */0); Index++)
			{
				UMaterialInterface* Material = StaticMesh->GetMaterial(Index);

				if (Material && !Material->IsAsset())
				{
					Material = nullptr;
				}

				StaticMesh->SetMaterial(Index, Material);
			}
#endif

			ISMComponent->SetStaticMesh(StaticMesh);

			for (const FCEMeshBuilder::FCEMeshInstanceData& Instance : Instances)
			{
				ISMComponent->AddInstance(Instance.Transform, /** WorldSpace */true);
			}

			if (!Instances.IsEmpty())
			{
				constexpr int32 InstanceIndex = 0;

				// ISM component do not have a way to set different materials per instance, just pick first instance materials...
				for (int32 MaterialIndex = 0; MaterialIndex < Instances[InstanceIndex].MeshMaterials.Num(); MaterialIndex++)
				{
					ISMComponent->SetMaterial(MaterialIndex, Instances[InstanceIndex].MeshMaterials[MaterialIndex].Get());
				}
			}

			ISMActor->AttachToActor(GroupActor, FAttachmentTransformRules::KeepRelativeTransform);

#if WITH_EDITOR
			FActorLabelUtilities::SetActorLabelUnique(ISMActor, Owner->GetActorNameOrLabel() + TEXT("_ISM_Instance"));
#endif

			NewActors.Add(ISMActor);
		}
	}

	return NewActors;
}

UActorComponent* UE::ClonerEffector::Conversion::CreateRootComponent(AActor* InActor, TSubclassOf<USceneComponent> InComponentClass, const FTransform& InWorldTransform)
{
	USceneComponent* const NewComponent = NewObject<USceneComponent>(InActor
		, InComponentClass.Get()
		, MakeUniqueObjectName(InActor, InComponentClass.Get(), InComponentClass->GetFName())
		, RF_Transactional);

	InActor->SetRootComponent(NewComponent);
	InActor->AddInstanceComponent(NewComponent);
	NewComponent->OnComponentCreated();
	NewComponent->RegisterComponent();

#if WITH_EDITOR
	InActor->RerunConstructionScripts();
#endif

	NewComponent->SetWorldTransform(InWorldTransform);

	return NewComponent;
}

#if WITH_EDITOR
bool UE::ClonerEffector::Conversion::PickAssetPath(const FString& InDefaultPath, FString& OutPickedPath)
{
	TSharedPtr<SDlgPickPath> DialogWidget = SNew(SDlgPickPath)
		.Title(LOCTEXT("PickAssetsLocation", "Choose Asset(s) Location"))
		.DefaultPath(FText::FromString(InDefaultPath));

	if (DialogWidget->ShowModal() != EAppReturnType::Ok)
	{
		OutPickedPath = TEXT("");
		return false;
	}

	OutPickedPath = DialogWidget->GetPath().ToString() + TEXT("/");
	return true;
}

UObject* UE::ClonerEffector::Conversion::CreateAssetPackage(TSubclassOf<UObject> InAssetClass, const FString& InAssetPath)
{
	if (!IsValid(InAssetClass.Get()) || InAssetPath.IsEmpty())
	{
		return nullptr;
	}

	FString PackageName = FPackageName::ObjectPathToPackageName(InAssetPath);
	FAssetToolsModule& AssetToolsModule = FModuleManager::LoadModuleChecked<FAssetToolsModule>("AssetTools");

	FString AssetName;
	AssetToolsModule.Get().CreateUniqueAssetName(PackageName, TEXT(""), PackageName, AssetName);

	FString UserPackageName = InAssetPath;
	FName UserAssetName(*FPackageName::GetLongPackageAssetName(UserPackageName));

	// is input name valid ?
	if (UserAssetName.IsNone())
	{
		// Use default if invalid
		UserPackageName = PackageName;
		UserAssetName = *AssetName;
	}

	// Find/create package
	UPackage* Package = CreatePackage(*UserPackageName);

	if (!Package)
	{
		return nullptr;
	}

	// Create asset object
	UObject* AssetObject = NewObject<UObject>(Package, InAssetClass.Get(), UserAssetName, RF_Public | RF_Standalone);

	// Notify asset registry of new asset
	FAssetRegistryModule::AssetCreated(AssetObject);

	return AssetObject;
}
#endif

#undef LOCTEXT_NAMESPACE
