// Copyright Epic Games, Inc. All Rights Reserved.

#include "Modules/ModuleManager.h"

#include "Cloner/Logs/CEClonerLogs.h"
#include "Effector/Logs/CEEffectorLogs.h"

DEFINE_LOG_CATEGORY(LogCECloner);
DEFINE_LOG_CATEGORY(LogCEEffector);

IMPLEMENT_MODULE(FDefaultModuleImpl, ClonerEffector)