// Copyright Epic Games, Inc. All Rights Reserved.

#include "Cloner/Attachments/CEClonerAttachmentItem.h"

#include "Components/PrimitiveComponent.h"
#include "CEMeshBuilder.h"
#include "GameFramework/Actor.h"
#include "Settings/CEClonerEffectorSettings.h"
#include "UDynamicMesh.h"
#include "Utilities/CEClonerEffectorUtilities.h"

#if WITH_EDITOR
bool FCEClonerAttachmentItem::CheckBoundsChanged(bool bInUpdate)
{
	bool bChanged = false;

	if (ItemActor.IsValid())
	{
		const FBox LocalBounds = GetAttachmentBounds();
		const FVector ActorOrigin = LocalBounds.GetCenter();
		const FVector ActorExtent = LocalBounds.GetExtent();

		bChanged = !ActorOrigin.Equals(Origin) || !ActorExtent.Equals(Extent);

		if (bInUpdate)
		{
			Origin = ActorOrigin;
			Extent = ActorExtent;
		}
	}

	return bChanged;
}

FBox FCEClonerAttachmentItem::GetBakedMeshBounds() const
{
	FBox Bounds(ForceInitToZero);
	if (BakedMesh)
	{
		BakedMesh->ProcessMesh([&Bounds](const FDynamicMesh3& InMesh)
		{
			Bounds = static_cast<FBox>(InMesh.GetBounds(/** Parallel */true));
		});
	}
	return Bounds;
}

FBox FCEClonerAttachmentItem::GetAttachmentBounds() const
{
	FBox Bounds(ForceInitToZero);
	if (const AActor* Actor = ItemActor.Get())
	{
		TArray<UPrimitiveComponent*> PrimitiveComponents;
		Actor->GetComponents(PrimitiveComponents, /** IncludeChildren */false);

		for (const UPrimitiveComponent* Component : PrimitiveComponents)
		{
			if (Component
				&& Component->IsRegistered()
				&& !Component->IsVisualizationComponent())
			{
				FTransform ComponentToActorTransform = Component->GetComponentTransform().GetRelativeTransform(Actor->GetActorTransform());
				FBoxSphereBounds ComponentBounds = Component->CalcBounds(ComponentToActorTransform);

				Bounds += ComponentBounds.GetBox();
			}
		}
	}
	return Bounds;
}
#endif

bool FCEClonerAttachmentItem::CheckMaterialsChanged(bool bInUpdate, TArray<TWeakObjectPtr<UMaterialInterface>>* OutInvalidMaterials)
{
	using namespace UE::ClonerEffector::Utilities;

	bool bMaterialChanged = false;

	if (const AActor* Actor = ItemActor.Get())
	{
		TArray<UPrimitiveComponent*> PrimitiveComponents;
		Actor->GetComponents(PrimitiveComponents, /** IncludeChildrenActors */false);

		int32 MatIdx = 0;
		TArray<TWeakObjectPtr<UMaterialInterface>> NewMaterials;
		NewMaterials.Reserve(PrimitiveComponents.Num());
		UMaterialInterface* DefaultMaterial = LoadObject<UMaterialInterface>(nullptr, UCEClonerEffectorSettings::DefaultMaterialPath);

		if (OutInvalidMaterials)
		{
			OutInvalidMaterials->Empty();
		}

		for (UPrimitiveComponent* PrimitiveComponent : PrimitiveComponents)
		{
			if (!PrimitiveComponent || !FCEMeshBuilder::HasAnyGeometry(PrimitiveComponent))
			{
				continue;
			}

			for (int32 MatIndex = 0; MatIndex < PrimitiveComponent->GetNumMaterials(); MatIndex++)
			{
				UMaterialInterface* PreviousMaterial = PrimitiveComponent->GetMaterial(MatIndex);
				UMaterialInterface* NewMaterial = PreviousMaterial;

				if (FilterSupportedMaterial(NewMaterial, DefaultMaterial) && OutInvalidMaterials)
				{
					OutInvalidMaterials->Add(PreviousMaterial);
				}

				if (!BakedMaterials.IsValidIndex(MatIdx)
					|| BakedMaterials[MatIdx] != NewMaterial)
				{
					bMaterialChanged = true;
				}

				NewMaterials.Add(NewMaterial);
				MatIdx++;
			}
		}

		if (bInUpdate)
		{
			if (NewMaterials.Num() != BakedMaterials.Num())
			{
				MeshStatus = ECEClonerAttachmentStatus::Outdated;
			}

			BakedMaterials = NewMaterials;
		}
	}

	return bMaterialChanged;
}
