// Copyright Epic Games, Inc. All Rights Reserved.

#include "Cloner/Attachments/CEClonerAttachmentRootItem.h"
#include "GameFramework/Actor.h"

FCEClonerAttachmentRootItem::FCEClonerAttachmentRootItem(AActor* InActor)
	: RootActor(InActor)
{
}
