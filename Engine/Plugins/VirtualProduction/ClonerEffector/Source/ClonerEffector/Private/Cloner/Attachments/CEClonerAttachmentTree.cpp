// Copyright Epic Games, Inc. All Rights Reserved.

#include "Cloner/Attachments/CEClonerAttachmentTree.h"

#include "Cloner/Attachments/CEClonerAttachmentTreeBehavior.h"
#include "Cloner/Logs/CEClonerLogs.h"
#include "Components/PrimitiveComponent.h"
#include "GameFramework/Actor.h"
#include "Subsystems/CEClonerSubsystem.h"
#include "Templates/SharedPointer.h"

void FCEClonerAttachmentTree::SetAttachmentRoot(AActor* InActor)
{
	AttachmentRoot = InActor;
	MarkAttachmentOutdated();
}

AActor* FCEClonerAttachmentTree::GetAttachmentRoot() const
{
	return AttachmentRoot;
}

void FCEClonerAttachmentTree::SetBehaviorImplementation(const TSharedRef<ICEClonerAttachmentTreeBehavior>& InImplementation)
{
	if (BehaviorImplementation.IsValid())
	{
		TArray<AActor*> ActorsToDetach;
		ForEachAttachment([this, &ActorsToDetach](AActor* InActor, FCEClonerAttachmentItem& InAttachment)
		{
			ActorsToDetach.Add(InActor);
			return true;
		});

		for (AActor* const Actor : ActorsToDetach)
		{
			DetachItem(Actor);
		}

		BehaviorImplementation->OnDeactivation(*this);
	}

	BehaviorImplementation = InImplementation;

	if (BehaviorImplementation.IsValid())
	{
		BehaviorImplementation->OnActivation(*this);
		MarkAttachmentOutdated();
	}
}

void FCEClonerAttachmentTree::Cleanup()
{
	if (BehaviorImplementation.IsValid())
	{
		BehaviorImplementation->OnDeactivation(*this);
		BehaviorImplementation.Reset();
	}
}

void FCEClonerAttachmentTree::Reset()
{
	ItemAttachmentMap.Reset();
	RootItemAttachments.Reset();
	DirtyItemAttachments.Reset();
	Status = ECEClonerAttachmentStatus::Outdated;
}

void FCEClonerAttachmentTree::MarkAttachmentOutdated()
{
	if (Status == ECEClonerAttachmentStatus::Updated)
	{
		Status = ECEClonerAttachmentStatus::Outdated;
	}
}

void FCEClonerAttachmentTree::MarkCacheOutdated(AActor* InActor)
{
	if (BehaviorImplementation.IsValid())
	{
		BehaviorImplementation->InvalidateMesh(*this, InActor);
	}
}

bool FCEClonerAttachmentTree::UpdateAttachments(bool bInReset)
{
	if (!IsValid(AttachmentRoot)
		|| !BehaviorImplementation.IsValid()
		|| Status == ECEClonerAttachmentStatus::Updating)
	{
		return false;
	}

	if (bInReset)
	{
		Reset();
		Status = ECEClonerAttachmentStatus::Outdated;
	}

	if (Status != ECEClonerAttachmentStatus::Outdated)
	{
		return false;
	}

	UE_LOGF(LogCECloner, Verbose, "%ls : updating attachment tree", *AttachmentRoot->GetActorNameOrLabel())

	Status = ECEClonerAttachmentStatus::Updating;
	const bool bResult = UpdateAttachmentsInternal();
	Status = ECEClonerAttachmentStatus::Updated;

	return bResult;
}

AActor* FCEClonerAttachmentTree::FindRootActor(AActor* InActor)
{
	if (InActor == nullptr)
	{
		return nullptr;
	}

	if (const FCEClonerAttachmentItem* Item = ItemAttachmentMap.Find(InActor))
	{
		if (Item->bRootItem)
		{
			return Item->ItemActor.Get();
		}

		return FindRootActor(Item->ParentActor.Get());
	}

	return nullptr;
}

FCEClonerAttachmentRootItem* FCEClonerAttachmentTree::FindRootItem(AActor* InActor)
{
	if (AActor* RootActor = FindRootActor(InActor))
	{
		return RootItemAttachments.FindByKey(RootActor);
	}

	return nullptr;
}

TArray<AActor*> FCEClonerAttachmentTree::GetRootActors() const
{
	TArray<AActor*> Actors;
	Algo::Transform(RootItemAttachments, Actors, [](const FCEClonerAttachmentRootItem& InRootAttachment)->AActor*
	{
		return InRootAttachment.RootActor.Get();
	});
	return Actors;
}

void FCEClonerAttachmentTree::ForEachAttachment(const TFunctionRef<bool(AActor*, FCEClonerAttachmentItem&)>& InFunctor)
{
	for (TMap<TWeakObjectPtr<AActor>, FCEClonerAttachmentItem>::TIterator It(ItemAttachmentMap); It; ++It)
	{
		if (AActor* Actor = It->Key.Get())
		{
			if (!InFunctor(Actor, It->Value))
			{
				return;
			}
		}
	}
}

void FCEClonerAttachmentTree::DetachItem(AActor* InActor)
{
	if (FCEClonerAttachmentItem* Item = ItemAttachmentMap.Find(InActor))
	{
		DetachItem(*Item);
	}
}

bool FCEClonerAttachmentTree::IsCacheAvailable(bool bInAllowInvalid) const
{
	// Check cached meshes are not async loading
	for (const FCEClonerAttachmentRootItem& RootItem : RootItemAttachments)
	{
		const UStaticMesh* BakedMesh = RootItem.MergedBakedMesh.Get();

		if (!IsValid(BakedMesh))
		{
			if (!bInAllowInvalid)
			{
				return false;
			}

			continue;
		}

		if (BakedMesh->HasAnyInternalFlags(EInternalObjectFlags_AsyncLoading))
		{
			return false;
		}
	}

	return true;
}

void FCEClonerAttachmentTree::GetAttachments(AActor* InActor, TArray<const FCEClonerAttachmentItem*>& OutAttachmentItems, bool bInRecurse) const
{
	if (!InActor)
	{
		return;
	}

	const FCEClonerAttachmentItem* AttachmentItem = ItemAttachmentMap.Find(InActor);

	if (!AttachmentItem)
	{
		return;
	}

	OutAttachmentItems.Add(AttachmentItem);

	if (bInRecurse)
	{
		for (const TWeakObjectPtr<AActor>& ChildActor : AttachmentItem->ChildrenActors)
		{
			GetAttachments(ChildActor.Get(), OutAttachmentItems, bInRecurse);
		}
	}
}

FBox FCEClonerAttachmentTree::GetAttachmentBounds(AActor* InActor, bool bInIncludeChildren) const
{
	TArray<const FCEClonerAttachmentItem*> Attachments;
	GetAttachments(InActor, Attachments, bInIncludeChildren);

	FBox Bounds(ForceInitToZero);
	const FTransform ActorTransform = InActor->GetActorTransform();
	for (const FCEClonerAttachmentItem* Attachment : Attachments)
	{
		if (const AActor* Actor = Attachment->ItemActor.Get())
		{
			TArray<UPrimitiveComponent*> PrimitiveComponents;
			Actor->GetComponents(PrimitiveComponents, /** IncludeChildren */false);

			for (const UPrimitiveComponent* Component : PrimitiveComponents)
			{
				if (Component
					&& Component->IsRegistered()
#if WITH_EDITOR
					&& !Component->IsVisualizationComponent()
#endif
					)
				{
					FTransform ComponentToActorTransform = Component->GetComponentTransform().GetRelativeTransform(ActorTransform);
					FBoxSphereBounds ComponentBounds = Component->CalcBounds(ComponentToActorTransform);

					Bounds += ComponentBounds.GetBox();
					Bounds.IsValid = true;
				}
			}
		}
	}

	return Bounds;
}

#if WITH_EDITOR
TOptional<FVector> FCEClonerAttachmentTree::GetRootCachedMeshSize(AActor* InActor) const
{
	TOptional<FVector> MeshSize;
	const FCEClonerAttachmentRootItem* RootItem = RootItemAttachments.FindByKey(InActor);

	if (!RootItem || !RootItem->MergedBakedMesh.Get())
	{
		return MeshSize;
	}

	MeshSize = RootItem->MeshSize;

	return MeshSize;
}
#endif

bool FCEClonerAttachmentTree::UpdateAttachmentsInternal()
{
	// Invalidate all, to see what is outdated and what is still invalid
	for (TPair<TWeakObjectPtr<AActor>, FCEClonerAttachmentItem>& AttachmentPair : ItemAttachmentMap)
	{
		AttachmentPair.Value.Status = ECEClonerAttachmentStatus::Invalid;
	}

	// Update root attachment items
	TArray<AActor*> RootChildren;
	BehaviorImplementation->GetOrderedChildrenActors(*this, AttachmentRoot, RootChildren);

	TArray<FCEClonerAttachmentRootItem> NewRootItems;
	NewRootItems.Reserve(RootChildren.Num());
	for (int32 RootIdx = 0; RootIdx < RootChildren.Num(); RootIdx++)
	{
		AActor* RootChild = RootChildren[RootIdx];

		if (!IsValid(RootChild))
		{
			continue;
		}

		const FTransform RootTransform = RootChild->GetActorTransform();
		UpdateAttachment(RootChild, nullptr, RootTransform);

		// Lets find the old root idx
		if (const FCEClonerAttachmentRootItem* RootItem = RootItemAttachments.FindByKey(RootChild))
		{
			const int32 OldIdx = RootItemAttachments.Find(*RootItem);

			// Did we rearrange stuff ?
			if (RootIdx != OldIdx)
			{
				bItemAttachmentsDirty = true;
			}

			NewRootItems.Add(*RootItem);
		}
		else
		{
			FCEClonerAttachmentRootItem NewRootItem(RootChild);
			NewRootItems.Add(NewRootItem);
		}
	}

	// Did we remove any root actors ?
	if (RootItemAttachments.Num() != NewRootItems.Num())
	{
		bItemAttachmentsDirty = true;
	}

	// Did we need to update meshes
	TArray<TWeakObjectPtr<AActor>> ClonedActors;
	ItemAttachmentMap.GenerateKeyArray(ClonedActors);
	for (const TWeakObjectPtr<AActor>& ClonedActorWeak : ClonedActors)
	{
		FCEClonerAttachmentItem* ClonedItem = ItemAttachmentMap.Find(ClonedActorWeak);

		if (!ClonedItem)
		{
			continue;
		}

		AActor* ClonedActor = ClonedActorWeak.Get();

		if (ClonedItem->Status == ECEClonerAttachmentStatus::Invalid)
		{
			DetachItem(*ClonedItem);
		}
		else if (ClonedItem->Status == ECEClonerAttachmentStatus::Outdated)
		{
			if (ClonedItem->MeshStatus == ECEClonerAttachmentStatus::Outdated)
			{
				DirtyItemAttachments.Add(ClonedItem->ItemActor);
				BehaviorImplementation->InvalidateMesh(*this, ClonedActor);
			}

			bItemAttachmentsDirty = true;
			ClonedItem->Status = ECEClonerAttachmentStatus::Updated;
		}
	}

	// Did we remove an attachment ?
	if (ClonedActors.Num() != ItemAttachmentMap.Num())
	{
		bItemAttachmentsDirty = true;
	}

	if (!DirtyItemAttachments.IsEmpty())
	{
		bItemAttachmentsDirty = true;
	}

	RootItemAttachments = NewRootItems;

	return bItemAttachmentsDirty;
}

void FCEClonerAttachmentTree::UpdateAttachment(AActor* InActor, AActor* InParent, const FTransform& InActorTransform)
{
	if (!IsValid(InActor))
	{
		return;
	}

	TArray<AActor*> ChildrenActors;
	BehaviorImplementation->GetOrderedChildrenActors(*this, InActor, ChildrenActors);

	const FTransform ActorTransform = InActor->GetActorTransform().GetRelativeTransform(InActorTransform);

	FCEClonerAttachmentItem* AttachmentItem = ItemAttachmentMap.Find(InActor);

	if (!AttachmentItem)
	{
		AttachmentItem = &ItemAttachmentMap.Add(InActor, FCEClonerAttachmentItem());
		AttachmentItem->ItemActor = InActor;
		AttachmentItem->MeshStatus = ECEClonerAttachmentStatus::Outdated;
		AttachmentItem->Status = ECEClonerAttachmentStatus::Outdated;
	}
	else
	{
		AttachmentItem->Status = ECEClonerAttachmentStatus::Updated;
	}

	if (AttachmentItem)
	{
		// Check Root is the same
		const bool bIsRoot = InParent == nullptr;
		if (AttachmentItem->bRootItem != bIsRoot)
		{
			BehaviorImplementation->InvalidateMesh(*this, InActor);
			AttachmentItem->bRootItem = bIsRoot;
			AttachmentItem->Status = ECEClonerAttachmentStatus::Outdated;
		}

		// Check parent is the same
		if (AttachmentItem->ParentActor.Get() != InParent)
		{
			BehaviorImplementation->InvalidateMesh(*this, InParent);
			BehaviorImplementation->InvalidateMesh(*this, AttachmentItem->ParentActor.Get());
			AttachmentItem->ParentActor = InParent;
			AttachmentItem->Status = ECEClonerAttachmentStatus::Outdated;
		}

		// Check transform is the same
		if (!ActorTransform.Equals(AttachmentItem->ActorTransform))
		{
			// invalidate if not root, else change transform in mesh renderer
			if (!bIsRoot)
			{
				BehaviorImplementation->InvalidateMesh(*this, InActor);
			}
			AttachmentItem->ActorTransform = ActorTransform;
			AttachmentItem->Status = ECEClonerAttachmentStatus::Outdated;
		}

#if WITH_EDITOR
		// Check bounds change
		if (AttachmentItem->CheckBoundsChanged(/** CheckAndUpdate */true) && AttachmentItem->bSetupDone)
		{
			BehaviorImplementation->InvalidateMesh(*this, InActor);
			AttachmentItem->Status = ECEClonerAttachmentStatus::Outdated;
		}

		// Check materials change
		TArray<TWeakObjectPtr<UMaterialInterface>> UnsupportedMaterials;
		if (AttachmentItem->CheckMaterialsChanged(/** CheckAndUpdate */true, &UnsupportedMaterials))
		{
			// Show warning for unset materials
			if (!UnsupportedMaterials.IsEmpty())
			{
				if (UCEClonerSubsystem* ClonerSubsystem = UCEClonerSubsystem::Get())
				{
					ClonerSubsystem->FireMaterialWarning(AttachmentRoot, InActor, UnsupportedMaterials);
				}
			}

			BehaviorImplementation->UpdateMesh(*this, InActor);

			if (AttachmentItem->bSetupDone)
			{
				AttachmentItem->Status = ECEClonerAttachmentStatus::Outdated;
			}
		}

		// Check dynamic mesh needs update
		if (!AttachmentItem->BakedMesh && AttachmentItem->bSetupDone)
		{
			AttachmentItem->MeshStatus = ECEClonerAttachmentStatus::Outdated;
			AttachmentItem->Status = ECEClonerAttachmentStatus::Outdated;
		}
#endif

		if (AttachmentItem->ChildrenActors.Num() != ChildrenActors.Num())
		{
			BehaviorImplementation->InvalidateMesh(*this, InActor);
		}

		AttachmentItem->ChildrenActors.Empty(ChildrenActors.Num());
		Algo::Transform(ChildrenActors, AttachmentItem->ChildrenActors,
			[](AActor* InActor)
			{
				return InActor;
			}
		);

		for (AActor* ChildActor : ChildrenActors)
		{
			UpdateAttachment(ChildActor, InActor, InActorTransform);
		}

		if (!AttachmentItem->bSetupDone)
		{
			AttachmentItem->bSetupDone = true;
			BehaviorImplementation->OnItemAttached(*this, *AttachmentItem);
			OnItemAttachedDelegate.ExecuteIfBound(InActor, *AttachmentItem);
		}
	}
}

void FCEClonerAttachmentTree::DetachItem(FCEClonerAttachmentItem& InItem)
{
	AActor* Actor = InItem.ItemActor.Get(/** EvenIfPendingKill */true);
	BehaviorImplementation->InvalidateMesh(*this, Actor);
	BehaviorImplementation->OnItemDetached(*this, InItem);
	OnItemDetachedDelegate.ExecuteIfBound(Actor, InItem);
	ItemAttachmentMap.Remove(Actor);
	bItemAttachmentsDirty = true;
}

