// Copyright Epic Games, Inc. All Rights Reserved.

#include "Cloner/Attachments/CEClonerAttachmentTreeBehavior.h"

#include "CEMeshBuilder.h"
#include "Cloner/CEClonerComponent.h"
#include "Cloner/Attachments/CEClonerAttachmentTree.h"
#include "Cloner/Attachments/CEClonerSceneTreeCustomResolver.h"
#include "Cloner/Logs/CEClonerLogs.h"
#include "Components/DynamicMeshComponent.h"
#include "Engine/Engine.h"
#include "Engine/World.h"
#include "GameFramework/Actor.h"
#include "Materials/Material.h"
#include "Misc/EnumerateRange.h"
#include "Subsystems/CEClonerSubsystem.h"
#include "Utilities/CEClonerEffectorUtilities.h"

#define LOCTEXT_NAMESPACE "CEClonerAttachmentImplementation"

void FCEClonerAttachmentGroupBehavior::OnActivation(FCEClonerAttachmentTree& InTree)
{
	const AActor* AttachmentRoot = InTree.GetAttachmentRoot();

	if (AttachmentRoot)
	{
		if (const UWorld* World = AttachmentRoot->GetWorld())
		{
			WorldActorDestroyedDelegate = World->AddOnActorDestroyedHandler(FOnActorDestroyed::FDelegate::CreateSPLambda(this, [this, &InTree](AActor* InActor)
			{
				OnWorldActorDestroyed(InTree, InActor);
			}));
		}
	}

	USceneComponent::MarkRenderStateDirtyEvent.AddSPLambda(this, [this, &InTree](UActorComponent& InComponent)
	{
		OnCheckMaterialChanged(InTree, InComponent.GetOwner());

#if WITH_EDITOR
		OnRenderStateDirty(InTree, InComponent);
#endif
	});

#if WITH_EDITOR
	FCoreUObjectDelegates::OnObjectPropertyChanged.AddSPLambda(this, [this, &InTree](UObject* InObject, FPropertyChangedEvent& InEvent)
	{
		OnCheckMaterialChanged(InTree, InObject);
	});

	UMaterial::OnMaterialCompilationFinished().AddSPLambda(this, [this, &InTree](UMaterialInterface* InMaterial)
	{
		OnCheckMaterialChanged(InTree, InMaterial);
	});

	UCEClonerSubsystem* ClonerSubsystem = UCEClonerSubsystem::Get();
	if (!!AttachmentRoot && !!ClonerSubsystem)
	{
		if (const TSharedPtr<ICEClonerSceneTreeCustomResolver> CustomSceneTreeResolver = ClonerSubsystem->FindCustomLevelSceneTreeResolver(AttachmentRoot->GetLevel()))
		{
			CustomSceneTreeResolver->OnActorHierarchyChanged().AddSPLambda(this, [this, &InTree](AActor* InActor)
			{
				OnLevelHierarchyChanged(InTree, InActor);
			});
		}
	}

	if (GEngine)
	{
		GEngine->OnLevelActorAttached().AddSPLambda(this, [this, &InTree](AActor* InActor, const AActor* InParent)
		{
			OnLevelHierarchyChanged(InTree, InActor);
		});
	
		GEngine->OnLevelActorDetached().AddSPLambda(this, [this, &InTree](AActor* InActor, const AActor* InParent)
		{
			OnLevelHierarchyChanged(InTree, InActor);
		});
	}
#endif
}

void FCEClonerAttachmentGroupBehavior::OnDeactivation(FCEClonerAttachmentTree& InTree)
{
	const AActor* AttachmentRoot = InTree.GetAttachmentRoot();

	if (AttachmentRoot)
	{
		if (const UWorld* World = AttachmentRoot->GetWorld())
		{
			World->RemoveOnActorDestroyedHandler(WorldActorDestroyedDelegate);
		}
	}

	USceneComponent::MarkRenderStateDirtyEvent.RemoveAll(this);

#if WITH_EDITOR
	FCoreUObjectDelegates::OnObjectPropertyChanged.RemoveAll(this);

	UMaterial::OnMaterialCompilationFinished().RemoveAll(this);

	UCEClonerSubsystem* ClonerSubsystem = UCEClonerSubsystem::Get();
	if (!!AttachmentRoot && !!ClonerSubsystem)
	{
		if (const TSharedPtr<ICEClonerSceneTreeCustomResolver> CustomSceneTreeResolver = ClonerSubsystem->FindCustomLevelSceneTreeResolver(AttachmentRoot->GetLevel()))
		{
			CustomSceneTreeResolver->OnActorHierarchyChanged().RemoveAll(this);
		}
	}
	
	if (GEngine)
	{
		GEngine->OnLevelActorAttached().RemoveAll(this);
		GEngine->OnLevelActorDetached().RemoveAll(this);
	}
#endif
}

void FCEClonerAttachmentGroupBehavior::GetOrderedChildrenActors(FCEClonerAttachmentTree& InTree, AActor* InActor, TArray<AActor*>& OutChildren) const
{
	UCEClonerSubsystem* ClonerSubsystem = UCEClonerSubsystem::Get();
	if (!ClonerSubsystem)
	{
		return;
	}

	const TSharedPtr<ICEClonerSceneTreeCustomResolver> CustomSceneTreeResolver = ClonerSubsystem->FindCustomLevelSceneTreeResolver(InActor->GetLevel());

	if (!CustomSceneTreeResolver.IsValid() || !CustomSceneTreeResolver->GetDirectChildrenActor(InActor, OutChildren))
	{
		InActor->GetAttachedActors(OutChildren, /** Reset */true, /** IncludeChildren */false);
	}

	OutChildren.RemoveAll([](const AActor* InActor)
	{
		return !IsValid(InActor);
	});
}

void FCEClonerAttachmentGroupBehavior::InvalidateMesh(FCEClonerAttachmentTree& InTree, AActor* InActor)
{
#if WITH_EDITOR
	if (!InActor)
	{
		return;
	}

	if (const FCEClonerAttachmentItem* FoundItem = InTree.ItemAttachmentMap.Find(InActor))
	{
		if (FoundItem->bRootItem || !FoundItem->ParentActor.IsValid())
		{
			if (FCEClonerAttachmentRootItem* RootItem = InTree.RootItemAttachments.FindByKey(InActor))
			{
				RootItem->Reset();
				InTree.bItemAttachmentsDirty = true;
			}
		}
		else
		{
			InvalidateMesh(InTree, FoundItem->ParentActor.Get());
		}
	}
#endif
}

void FCEClonerAttachmentGroupBehavior::OnItemAttached(FCEClonerAttachmentTree& InTree, FCEClonerAttachmentItem& InItem)
{
	if (AActor* Actor = InItem.ItemActor.Get())
	{
		BindActorDelegates(InTree, Actor);

#if WITH_EDITOR
		// Ensure cached mesh bounds matches with attachment bounds
		if (InItem.bRootItem)
		{
			const TOptional<FVector> BakedMeshSize = InTree.GetRootCachedMeshSize(Actor);

			// Refresh cloner attachment when mesh size is not set
			if (!BakedMeshSize.IsSet())
			{
				InItem.MeshStatus = ECEClonerAttachmentStatus::Outdated;
				InvalidateMesh(InTree, Actor);
				InTree.DirtyItemAttachments.Add(Actor);
				InTree.MarkAttachmentOutdated();
				return;
			}

			// Leave the opportunity to dynamic mesh to initialize and build their geometry
			FTSTicker::GetCoreTicker().AddTicker(FTickerDelegate::CreateSPLambda(this, [this, &InTree, BakedMeshSize, Actor](float)->bool
			{
				if (IsValid(Actor))
				{
					const FVector AttachmentSize = InTree.GetAttachmentBounds(Actor, /** IncludeChildren */true).GetSize();

					if (!BakedMeshSize->Equals(AttachmentSize, 1.f))
					{
						const AActor* ClonerActor = InTree.GetAttachmentRoot();

						UE::ClonerEffector::Utilities::ShowWarning(FText::Format(
							LOCTEXT("AttachmentCachedBoundsMismatch", "Cloner {0} : {1} cached size vs actor size mismatch, see logs"),
							FText::FromString(ClonerActor->GetActorNameOrLabel()),
							FText::FromString(Actor->GetActorNameOrLabel())));

						UE_LOGF(LogCECloner, Warning, "%ls : %ls bounds mismatch, cached size (%ls) vs actor size (%ls), verify attached actor", *ClonerActor->GetActorNameOrLabel(), *Actor->GetActorNameOrLabel(), *BakedMeshSize->ToString(), *AttachmentSize.ToString());
					}
				}

				return false;
			}));
		}
#endif
	}
}

void FCEClonerAttachmentGroupBehavior::OnItemDetached(FCEClonerAttachmentTree& InTree, FCEClonerAttachmentItem& InItem)
{
	if (AActor* Actor = InItem.ItemActor.Get())
	{
		UnbindActorDelegates(InTree, Actor);
	}
}

void FCEClonerAttachmentGroupBehavior::UpdateMesh(FCEClonerAttachmentTree& InTree, AActor* InActor)
{
	RefreshBakedMeshRootMaterials(InTree, InActor);
}

void FCEClonerAttachmentFlatBehavior::GetOrderedChildrenActors(FCEClonerAttachmentTree& InTree, AActor* InActor, TArray<AActor*>& OutChildren) const
{
	if (InActor != InTree.GetAttachmentRoot())
	{
		return;
	}

	UCEClonerSubsystem* ClonerSubsystem = UCEClonerSubsystem::Get();
	if (!ClonerSubsystem)
	{
		return;
	}

	const TSharedPtr<ICEClonerSceneTreeCustomResolver> CustomSceneTreeResolver = ClonerSubsystem->FindCustomLevelSceneTreeResolver(InActor->GetLevel());

	if (CustomSceneTreeResolver.IsValid())
	{
		TFunction<void(AActor*)> AddOrderedChildrenActorsRecursively = [&AddOrderedChildrenActorsRecursively, &CustomSceneTreeResolver, &OutChildren](AActor* InActor)
		{
			TArray<AActor*> Children;
			CustomSceneTreeResolver->GetDirectChildrenActor(InActor, Children);

			for (AActor* ChildActor : Children)
			{
				if (!ChildActor)
				{
					continue;
				}

				TArray<UPrimitiveComponent*> PrimitiveComponents;
				ChildActor->GetComponents(PrimitiveComponents, /** IncludeChildrenActors */false);

				for (UPrimitiveComponent* PrimitiveComponent : PrimitiveComponents)
				{
					// Skip invalid component or ones without any geometry
					if (!!PrimitiveComponent && FCEMeshBuilder::HasAnyGeometry(PrimitiveComponent))
					{
						OutChildren.Add(ChildActor);
						break;
					}
				}

				AddOrderedChildrenActorsRecursively(ChildActor);
			}
		};

		AddOrderedChildrenActorsRecursively(InActor);
	}
	else
	{
		InActor->GetAttachedActors(OutChildren, /** Reset */true, /** IncludeChildren */true);
	}

	OutChildren.RemoveAll([](const AActor* InActor)
	{
		return !IsValid(InActor);
	});
}

void FCEClonerAttachmentGroupBehavior::BindActorDelegates(FCEClonerAttachmentTree& InTree, AActor* InActor)
{
	if (!IsValid(InActor))
	{
		return;
	}

#if WITH_EDITOR
	// Detect static mesh change
	TArray<UStaticMeshComponent*> StaticMeshComponents;
	InActor->GetComponents(StaticMeshComponents, false);
	for (UStaticMeshComponent* StaticMeshComponent : StaticMeshComponents)
	{
		if (!StaticMeshComponent->OnStaticMeshChanged().IsBoundToObject(this))
		{
			StaticMeshComponent->OnStaticMeshChanged().AddSPLambda(this, [this, &InTree, InActor](UStaticMeshComponent*)
			{
				OnMeshChanged(InTree, InActor);
			});
		}
	}

	// Detect dynamic mesh change
	TArray<UDynamicMeshComponent*> DynamicMeshComponents;
	InActor->GetComponents(DynamicMeshComponents, false);
	for (UDynamicMeshComponent* DynamicMeshComponent : DynamicMeshComponents)
	{
		if (!DynamicMeshComponent->OnMeshChanged.IsBoundToObject(this))
		{
			DynamicMeshComponent->OnMeshChanged.AddSPLambda(this, [this, &InTree, InActor]()
			{
				OnMeshChanged(InTree, InActor);
			});
		}
	}
#endif

	// Detect components transform
	TArray<USceneComponent*> SceneComponents;
	constexpr bool bIncludeChildren = false;
	InActor->GetComponents(SceneComponents, bIncludeChildren);
	for (USceneComponent* SceneComponent : SceneComponents)
	{
		if (!SceneComponent->TransformUpdated.IsBoundToObject(this))
		{
			SceneComponent->TransformUpdated.AddSPLambda(this, [this, &InTree](USceneComponent* InComponent, EUpdateTransformFlags InFlags, ETeleportType InTeleport)
			{
				OnComponentTransformed(InTree, InComponent, InFlags, InTeleport);
			});
		}
	}
}

void FCEClonerAttachmentGroupBehavior::UnbindActorDelegates(FCEClonerAttachmentTree& InTree, AActor* InActor)
{
	if (!InActor)
	{
		return;
	}

#if WITH_EDITOR
	TArray<UStaticMeshComponent*> StaticMeshComponents;
	InActor->GetComponents(StaticMeshComponents, false);
	for (UStaticMeshComponent* StaticMeshComponent : StaticMeshComponents)
	{
		StaticMeshComponent->OnStaticMeshChanged().RemoveAll(this);
	}

	TArray<UDynamicMeshComponent*> DynamicMeshComponents;
	InActor->GetComponents(DynamicMeshComponents, false);
	for (UDynamicMeshComponent* DynamicMeshComponent : DynamicMeshComponents)
	{
		DynamicMeshComponent->OnMeshChanged.RemoveAll(this);
	}
#endif

	TArray<USceneComponent*> SceneComponents;
	constexpr bool bIncludeChildren = false;
	InActor->GetComponents(SceneComponents, bIncludeChildren);
	for (USceneComponent* SceneComponent : SceneComponents)
	{
		SceneComponent->TransformUpdated.RemoveAll(this);
	}
}

void FCEClonerAttachmentGroupBehavior::OnComponentTransformed(FCEClonerAttachmentTree& InTree, USceneComponent* InComponent, EUpdateTransformFlags InFlags, ETeleportType InTeleport)
{
	if (!InComponent || !InComponent->GetOwner() || InFlags == EUpdateTransformFlags::PropagateFromParent)
	{
		return;
	}

	AActor* Owner = InComponent->GetOwner();

	FCEClonerAttachmentItem* Item = InTree.ItemAttachmentMap.Find(Owner);

	if (!Item || !Item->bSetupDone)
	{
		return;
	}

	const AActor* RootActor = InTree.FindRootActor(Owner);

	if (!RootActor)
	{
		return;
	}

	// Skip update if root component has moved, since we can simply offset the mesh
	if (RootActor == Owner && RootActor->GetRootComponent() == InComponent)
	{
		InTree.bItemAttachmentsDirty = true;
		return;
	}

	bool bComponentSupported = FCEMeshBuilder::IsComponentSupported(InComponent);

	if (!bComponentSupported)
	{
		for (const TObjectPtr<USceneComponent>& ChildComponent : InComponent->GetAttachChildren())
		{
			if (FCEMeshBuilder::IsComponentSupported(ChildComponent))
			{
				bComponentSupported = true;
				break;
			}
		}
	}

	if (!bComponentSupported)
	{
		return;
	}

	const AActor* ClonerActor = InTree.GetAttachmentRoot();

	if (!ClonerActor)
	{
		return;
	}

	UE_LOGF(LogCECloner, Verbose, "%ls : Transform state changed for %ls", *ClonerActor->GetActorNameOrLabel(), *Owner->GetActorNameOrLabel());

	Item->MeshStatus = ECEClonerAttachmentStatus::Outdated;
	InvalidateMesh(InTree, Owner);
	InTree.DirtyItemAttachments.Add(Item->ItemActor);
	InTree.MarkAttachmentOutdated();
}

void FCEClonerAttachmentGroupBehavior::OnWorldActorDestroyed(FCEClonerAttachmentTree& InTree, AActor* InActor)
{
	if (InActor && InTree.ItemAttachmentMap.Contains(InActor))
	{
		InTree.DetachItem(InActor);
	}
}

#if WITH_EDITOR
void FCEClonerAttachmentGroupBehavior::OnMeshChanged(FCEClonerAttachmentTree& InTree, AActor* InActor)
{
	if (!InActor)
	{
		return;
	}

	const AActor* ClonerActor = InTree.GetAttachmentRoot();

	if (!ClonerActor)
	{
		return;
	}

	FCEClonerAttachmentItem* Item = InTree.ItemAttachmentMap.Find(InActor);

	if (!Item || !Item->bSetupDone || Item->MeshStatus == ECEClonerAttachmentStatus::Outdated)
	{
		return;
	}

	UE_LOGF(LogCECloner, Verbose, "%ls : Detected mesh change for %ls", *ClonerActor->GetActorNameOrLabel(), *InActor->GetActorNameOrLabel());

	Item->MeshStatus = ECEClonerAttachmentStatus::Outdated;
	InvalidateMesh(InTree, InActor);
	InTree.DirtyItemAttachments.Add(Item->ItemActor);
	InTree.MarkAttachmentOutdated();
}

void FCEClonerAttachmentGroupBehavior::OnRenderStateDirty(FCEClonerAttachmentTree& InTree, UActorComponent& InComponent)
{
	AActor* Owner = InComponent.GetOwner();
	const AActor* ClonerActor = InTree.GetAttachmentRoot();

	if (!Owner || !ClonerActor || Owner->GetLevel() != ClonerActor->GetLevel())
	{
		return;
	}

	// Does it contain geometry that we can convert
	if (!FCEMeshBuilder::IsComponentSupported(&InComponent))
	{
		return;
	}

	FCEClonerAttachmentItem* Item = InTree.ItemAttachmentMap.Find(Owner);

	if (!Item || !Item->bSetupDone || !Item->CheckBoundsChanged(/** CheckAndUpdate */false))
	{
		return;
	}

	UE_LOGF(LogCECloner, Verbose, "%ls : Render state changed for %ls", *ClonerActor->GetActorNameOrLabel(), *Owner->GetActorNameOrLabel());

	// Rebind delegates as new components might be available
	BindActorDelegates(InTree, Owner);

	Item->MeshStatus = ECEClonerAttachmentStatus::Outdated;
	InvalidateMesh(InTree, Owner);
	InTree.DirtyItemAttachments.Add(Item->ItemActor);
	InTree.MarkAttachmentOutdated();
}

void FCEClonerAttachmentGroupBehavior::OnLevelHierarchyChanged(FCEClonerAttachmentTree& InTree, AActor* InActor)
{
	if (IsValid(InActor))
	{
		const AActor* AttachmentRoot = InTree.GetAttachmentRoot();

		if (InTree.ItemAttachmentMap.Contains(InActor) || InActor->IsAttachedTo(AttachmentRoot))
		{
			InTree.MarkAttachmentOutdated();
		}
	}
}
#endif

void FCEClonerAttachmentGroupBehavior::OnCheckMaterialChanged(FCEClonerAttachmentTree& InTree, UObject* InObject)
{
	if (!IsValid(InObject))
	{
		return;
	}

	const AActor* ClonerActor = InTree.GetAttachmentRoot();

	if (!ClonerActor)
	{
		return;
	}

	AActor* ActorChanged = Cast<AActor>(InObject);
	ActorChanged = ActorChanged ? ActorChanged : InObject->GetTypedOuter<AActor>();

	if (!ActorChanged)
	{
		return;
	}

	FCEClonerAttachmentItem* AttachmentItem = InTree.ItemAttachmentMap.Find(ActorChanged);

	if (!AttachmentItem || !AttachmentItem->bSetupDone)
	{
		return;
	}

	TArray<TWeakObjectPtr<UMaterialInterface>> UnsupportedMaterials;
	if (AttachmentItem->CheckMaterialsChanged(/** Update */true, &UnsupportedMaterials))
	{
		// Show warning for unset materials
		if (!UnsupportedMaterials.IsEmpty())
		{
			if (UCEClonerSubsystem* ClonerSubsystem = UCEClonerSubsystem::Get())
			{
				ClonerSubsystem->FireMaterialWarning(ClonerActor, ActorChanged, UnsupportedMaterials);
			}
		}

		UE_LOGF(LogCECloner, Verbose, "%ls : Detected material change for %ls", *ClonerActor->GetActorNameOrLabel(), *ActorChanged->GetActorNameOrLabel());

		if (AttachmentItem->MeshStatus == ECEClonerAttachmentStatus::Outdated)
		{
			InvalidateMesh(InTree, ActorChanged);
			InTree.MarkAttachmentOutdated();
		}
		else
		{
			RefreshBakedMeshRootMaterials(InTree, ActorChanged);
		}
	}
}

void FCEClonerAttachmentGroupBehavior::RefreshBakedMeshRootMaterials(FCEClonerAttachmentTree& InTree, AActor* InActor)
{
	FCEClonerAttachmentRootItem* RootItem = InTree.FindRootItem(InActor);
	if (!RootItem || !RootItem->MergedBakedMesh)
	{
		return;
	}

	TArray<const FCEClonerAttachmentItem*> AttachmentItems;
	InTree.GetAttachments(RootItem->RootActor.Get(), AttachmentItems, /** Recurse */true);
	if (AttachmentItems.IsEmpty())
	{
		return;
	}

	TArray<TWeakObjectPtr<UMaterialInterface>> RootMaterials;
	TArray<FStaticMaterial>& StaticMaterials = RootItem->MergedBakedMesh->GetStaticMaterials();

	int32 MaterialIndexOffset = 0;
	for (const FCEClonerAttachmentItem* AttachmentItem : AttachmentItems)
	{
		if (!AttachmentItem)
		{
			continue;
		}

		for (TConstEnumerateRef<TWeakObjectPtr<UMaterialInterface>> BakedMaterialWeak : EnumerateRange(AttachmentItem->BakedMaterials))
		{
			if (UMaterialInterface* Material = BakedMaterialWeak->Get())
			{
				const int32 MaterialSlotIndex = MaterialIndexOffset + BakedMaterialWeak.GetIndex();

				if (StaticMaterials.IsValidIndex(MaterialSlotIndex))
				{
					StaticMaterials[MaterialSlotIndex].MaterialInterface = Material;
				}
			}
		}

		// Increment offset by material count for next attachment
		MaterialIndexOffset += AttachmentItem->BakedMaterials.Num();
	}

	InTree.bItemAttachmentsDirty = true;
}

#undef LOCTEXT_NAMESPACE
