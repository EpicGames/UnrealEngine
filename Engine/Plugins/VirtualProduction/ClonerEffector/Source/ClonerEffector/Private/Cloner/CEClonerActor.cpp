// Copyright Epic Games, Inc. All Rights Reserved.

#include "Cloner/CEClonerActor.h"

#include "Cloner/CEClonerComponent.h"
#include "Cloner/Extensions/CEClonerCollisionExtension.h"
#include "Cloner/Extensions/CEClonerConstraintExtension.h"
#include "Cloner/Extensions/CEClonerEffectorExtension.h"
#include "Cloner/Extensions/CEClonerEmitterSpawnExtension.h"
#include "Cloner/Extensions/CEClonerLifetimeExtension.h"
#include "Cloner/Extensions/CEClonerMeshRendererExtension.h"
#include "Cloner/Extensions/CEClonerProgressExtension.h"
#include "Cloner/Extensions/CEClonerRangeExtension.h"
#include "Cloner/Extensions/CEClonerStepExtension.h"
#include "Cloner/Layouts/CEClonerCircleLayout.h"
#include "Cloner/Layouts/CEClonerCylinderLayout.h"
#include "Cloner/Layouts/CEClonerGridLayout.h"
#include "Cloner/Layouts/CEClonerHoneycombLayout.h"
#include "Cloner/Layouts/CEClonerLayoutBase.h"
#include "Cloner/Layouts/CEClonerLineLayout.h"
#include "Cloner/Layouts/CEClonerMeshLayout.h"
#include "Cloner/Layouts/CEClonerSphereRandomLayout.h"
#include "Cloner/Layouts/CEClonerSphereUniformLayout.h"
#include "Cloner/Layouts/CEClonerSplineLayout.h"
#include "Cloner/Logs/CEClonerLogs.h"
#include "Containers/Ticker.h"
#include "Effector/CEEffectorActor.h"
#include "Materials/MaterialInterface.h"
#include "UObject/ConstructorHelpers.h"

#if WITH_EDITOR
#include "Editor.h"
#include "Selection.h"
#include "Settings/LevelEditorViewportSettings.h"
#endif

struct FCEClonerActorVersion
{
	enum Type : int32
	{
		PreVersioning = 0,

		/** Migrating all logic and properties to component */
		ComponentMigration,

		VersionPlusOne,
		LatestVersion = VersionPlusOne - 1
	};

	static constexpr FGuid GUID = FGuid(0x9271D8A2, 0xBF4146B6, 0xA20FC0A3, 0x9D8295B4);
};

FCustomVersionRegistration GRegisterCEClonerActorVersion(FCEClonerActorVersion::GUID, static_cast<int32>(FCEClonerActorVersion::LatestVersion), TEXT("CEClonerActorVersion"));

// Sets default values
ACEClonerActor::ACEClonerActor()
{
	SetCanBeDamaged(false);
	PrimaryActorTick.bCanEverTick = false;

	ClonerComponent = CreateDefaultSubobject<UCEClonerComponent>(TEXT("AvaClonerComponent"));
	SetRootComponent(ClonerComponent);

	if (!IsTemplate())
	{
#if WITH_EDITOR
		if (GEditor)
		{
			GEditor->GetSelectedActors()->SelectionChangedEvent.AddUObject(this, &ACEClonerActor::OnEditorSelectionChanged);
		}
#endif
	}
}

#if WITH_EDITOR
void ACEClonerActor::SpawnDefaultActorAttached()
{
	if (GIsEditor && ClonerComponent && ClonerComponent->GetAttachmentCount() == 0)
	{
		ClonerComponent->CreateDefaultActorAttached();
	}
}
#endif

void ACEClonerActor::Serialize(FArchive& InArchive)
{
	InArchive.UsingCustomVersion(FCEClonerActorVersion::GUID);

	Super::Serialize(InArchive);

	const int32 Version = InArchive.CustomVer(FCEClonerActorVersion::GUID);

	if (Version < FCEClonerActorVersion::ComponentMigration)
	{
		MigrateToVersion = FCEClonerActorVersion::ComponentMigration;
	}
}

#if WITH_EDITOR
FString ACEClonerActor::GetDefaultActorLabel() const
{
	return DefaultLabel;
}
#endif

void ACEClonerActor::PostLoad()
{
	Super::PostLoad();

	MigrateDeprecatedProperties();
}

void ACEClonerActor::MigrateDeprecatedProperties()
{
	if (MigrateToVersion == INDEX_NONE)
	{
		return;
	}

	UE_LOGF(LogCECloner, Warning, "%ls : Cloner migrating from version %i to latest %i, please re-save this asset", *GetActorNameOrLabel(), MigrateToVersion, FCEClonerActorVersion::LatestVersion);

	PRAGMA_DISABLE_DEPRECATION_WARNINGS

	if (MigrateToVersion <= FCEClonerActorVersion::ComponentMigration)
	{
		ClonerComponent->SetEnabled(bEnabled);
		ClonerComponent->SetSeed(Seed);
		ClonerComponent->SetColor(Color);
		ClonerComponent->SetLayoutName(LayoutName);

#if WITH_EDITOR
		ClonerComponent->SetVisualizerSpriteVisible(bVisualizerSpriteVisible);
#endif

		if (UCEClonerMeshRendererExtension* MeshRendererExtension = ClonerComponent->FindOrAddExtension<UCEClonerMeshRendererExtension>())
		{
			MeshRendererExtension->SetVisualizeEffectors(bVisualizeEffectors);
			MeshRendererExtension->SetMeshRenderMode(MeshRenderMode);
			MeshRendererExtension->SetMeshFacingMode(MeshFacingMode);
			MeshRendererExtension->SetMeshCastShadows(bMeshCastShadows);
			MeshRendererExtension->SetDefaultMeshes(DefaultMeshes);
			MeshRendererExtension->SetUseOverrideMaterial(bUseOverrideMaterial);
			MeshRendererExtension->SetOverrideMaterial(OverrideMaterial);
		}

		if (UCEClonerCollisionExtension* CollisionExtension = ClonerComponent->FindOrAddExtension<UCEClonerCollisionExtension>())
		{
			CollisionExtension->SetSurfaceCollisionEnabled(bSurfaceCollisionEnabled);
			CollisionExtension->SetParticleCollisionEnabled(bParticleCollisionEnabled);
			CollisionExtension->SetCollisionVelocityEnabled(bCollisionVelocityEnabled);
			CollisionExtension->SetCollisionIterations(CollisionIterations);
			CollisionExtension->SetCollisionGridResolution(CollisionGridResolution);
			CollisionExtension->SetCollisionGridSize(CollisionGridSize);
			CollisionExtension->SetCollisionRadiusMode(CollisionRadiusMode);
			CollisionExtension->SetMassMin(MassMin);
			CollisionExtension->SetMassMax(MassMax);
		}

		if (UCEClonerEmitterSpawnExtension* SpawnExtension = ClonerComponent->FindOrAddExtension<UCEClonerEmitterSpawnExtension>())
		{
			SpawnExtension->SetSpawnLoopMode(SpawnLoopMode);
			SpawnExtension->SetSpawnLoopIterations(SpawnLoopIterations);
			SpawnExtension->SetSpawnLoopIterations(SpawnLoopInterval);
			SpawnExtension->SetSpawnBehaviorMode(SpawnBehaviorMode);
			SpawnExtension->SetSpawnRate(SpawnRate);
		}

		if (UCEClonerLifetimeExtension* LifetimeExtension = ClonerComponent->FindOrAddExtension<UCEClonerLifetimeExtension>())
		{
			LifetimeExtension->SetLifetimeEnabled(bLifetimeEnabled);
			LifetimeExtension->SetLifetimeMin(LifetimeMin);
			LifetimeExtension->SetLifetimeMax(LifetimeMax);
			LifetimeExtension->SetLifetimeScaleEnabled(bLifetimeScaleEnabled);
			LifetimeExtension->SetLifetimeScaleCurve(LifetimeScaleCurve);
		}

		if (UCEClonerStepExtension* StepExtension = ClonerComponent->FindOrAddExtension<UCEClonerStepExtension>())
		{
			StepExtension->SetDeltaStepEnabled(bDeltaStepEnabled);
			StepExtension->SetDeltaStepRotation(DeltaStepRotation);
			StepExtension->SetDeltaStepScale(DeltaStepScale);
		}

		if (UCEClonerRangeExtension* RangeExtension = ClonerComponent->FindOrAddExtension<UCEClonerRangeExtension>())
		{
			RangeExtension->SetRangeEnabled(bRangeEnabled);
			RangeExtension->SetRangeMirrored(false);
			RangeExtension->SetRangeOffsetMin(RangeOffsetMin);
			RangeExtension->SetRangeOffsetMax(RangeOffsetMax);
			RangeExtension->SetRangeRotationMin(RangeRotationMin);
			RangeExtension->SetRangeRotationMax(RangeRotationMax);
			RangeExtension->SetRangeScaleMin(RangeScaleMin);
			RangeExtension->SetRangeScaleMax(RangeScaleMax);
			RangeExtension->SetRangeScaleUniform(bRangeScaleUniform);
			RangeExtension->SetRangeScaleUniformMin(RangeScaleUniformMin);
			RangeExtension->SetRangeScaleUniformMax(RangeScaleUniformMax);
		}

		if (UCEClonerProgressExtension* ProgressExtension = ClonerComponent->FindOrAddExtension<UCEClonerProgressExtension>())
		{
			ProgressExtension->SetProgress(Progress);
			ProgressExtension->SetInvertProgress(bInvertProgress);
		}

		if (UCEClonerEffectorExtension* EffectorExtension = ClonerComponent->FindOrAddExtension<UCEClonerEffectorExtension>())
		{
			for (const TWeakObjectPtr<ACEEffectorActor>& EffectorWeak : EffectorsWeak)
			{
				if (ACEEffectorActor* Effector = EffectorWeak.Get())
				{
					// Register to channel before linking
					Effector->RegisterToChannel();
					EffectorExtension->LinkEffector(Effector);
				}
			}
		}

		for (const TPair<FName, TObjectPtr<UCEClonerLayoutBase>>& LayoutPair : LayoutInstances)
		{
			if (UCEClonerGridLayout* PrevGridLayout = Cast<UCEClonerGridLayout>(LayoutPair.Value))
			{
				if (UCEClonerGridLayout* GridLayout = ClonerComponent->FindOrAddLayout<UCEClonerGridLayout>())
				{
					GridLayout->SetCountX(PrevGridLayout->GetCountX());
					GridLayout->SetCountY(PrevGridLayout->GetCountY());
					GridLayout->SetCountZ(PrevGridLayout->GetCountZ());
					GridLayout->SetSpacingX(PrevGridLayout->GetSpacingX());
					GridLayout->SetSpacingY(PrevGridLayout->GetSpacingY());
					GridLayout->SetSpacingZ(PrevGridLayout->GetSpacingZ());
					GridLayout->SetTwistAxis(PrevGridLayout->GetTwistAxis());
					GridLayout->SetTwistFactor(PrevGridLayout->GetTwistFactor() * 100);
				}

				if (UCEClonerConstraintExtension* ConstraintExtension = ClonerComponent->FindOrAddExtension<UCEClonerConstraintExtension>())
				{
					ConstraintExtension->SetConstraint(PrevGridLayout->GetConstraint());
					ConstraintExtension->SetInvertConstraint(PrevGridLayout->GetInvertConstraint());
					ConstraintExtension->SetSphereRadius(PrevGridLayout->GetSphereConstraint().Radius);
					ConstraintExtension->SetSphereCenter(PrevGridLayout->GetSphereConstraint().Center);
					ConstraintExtension->SetCylinderRadius(PrevGridLayout->GetCylinderConstraint().Radius);
					ConstraintExtension->SetCylinderHeight(PrevGridLayout->GetCylinderConstraint().Height);
					ConstraintExtension->SetCylinderCenter(PrevGridLayout->GetCylinderConstraint().Center);
					ConstraintExtension->SetTextureAsset(PrevGridLayout->GetTextureConstraint().Texture.Get());
					ConstraintExtension->SetTexturePlane(PrevGridLayout->GetTextureConstraint().Plane);
					ConstraintExtension->SetTextureSampleMode(PrevGridLayout->GetTextureConstraint().Channel);
					ConstraintExtension->SetTextureCompareMode(PrevGridLayout->GetTextureConstraint().CompareMode);
					ConstraintExtension->SetTextureThreshold(PrevGridLayout->GetTextureConstraint().Threshold);
				}
			}
			else if (UCEClonerLineLayout* PrevLineLayout = Cast<UCEClonerLineLayout>(LayoutPair.Value))
			{
				if (UCEClonerLineLayout* LineLayout = ClonerComponent->FindOrAddLayout<UCEClonerLineLayout>())
				{
					LineLayout->SetCount(PrevLineLayout->GetCount());
					LineLayout->SetSpacing(PrevLineLayout->GetSpacing());
					LineLayout->SetAxis(PrevLineLayout->GetAxis());
					LineLayout->SetDirection(PrevLineLayout->GetDirection());
					LineLayout->SetRotation(PrevLineLayout->GetRotation());
				}
			}
			else if (UCEClonerCircleLayout* PrevCircleLayout = Cast<UCEClonerCircleLayout>(LayoutPair.Value))
			{
				if (UCEClonerCircleLayout* CircleLayout = ClonerComponent->FindOrAddLayout<UCEClonerCircleLayout>())
				{
					CircleLayout->SetCount(PrevCircleLayout->GetCount());
					CircleLayout->SetRadius(PrevCircleLayout->GetRadius());
					CircleLayout->SetAngleStart(PrevCircleLayout->GetAngleStart());
					CircleLayout->SetAngleRatio(PrevCircleLayout->GetAngleRatio());
					CircleLayout->SetOrientMesh(PrevCircleLayout->GetOrientMesh());
					CircleLayout->SetPlane(PrevCircleLayout->GetPlane());
					CircleLayout->SetRotation(PrevCircleLayout->GetRotation());
					CircleLayout->SetScale(PrevCircleLayout->GetScale());
				}
			}
			else if (UCEClonerCylinderLayout* PrevCylinderLayout = Cast<UCEClonerCylinderLayout>(LayoutPair.Value))
			{
				if (UCEClonerCylinderLayout* CylinderLayout = ClonerComponent->FindOrAddLayout<UCEClonerCylinderLayout>())
				{
					CylinderLayout->SetBaseCount(PrevCylinderLayout->GetBaseCount());
					CylinderLayout->SetHeightCount(PrevCylinderLayout->GetHeightCount());
					CylinderLayout->SetHeight(PrevCylinderLayout->GetHeight());
					CylinderLayout->SetRadius(PrevCylinderLayout->GetRadius());
					CylinderLayout->SetAngleStart(PrevCylinderLayout->GetAngleStart());
					CylinderLayout->SetAngleRatio(PrevCylinderLayout->GetAngleRatio());
					CylinderLayout->SetOrientMesh(PrevCylinderLayout->GetOrientMesh());
					CylinderLayout->SetPlane(PrevCylinderLayout->GetPlane());
					CylinderLayout->SetRotation(PrevCylinderLayout->GetRotation());
					CylinderLayout->SetScale(PrevCylinderLayout->GetScale());
				}
			}
			else if (UCEClonerSphereUniformLayout* PrevSphereUniformLayout = Cast<UCEClonerSphereUniformLayout>(LayoutPair.Value))
			{
				if (UCEClonerSphereUniformLayout* SphereLayout = ClonerComponent->FindOrAddLayout<UCEClonerSphereUniformLayout>())
				{
					SphereLayout->SetCount(PrevSphereUniformLayout->GetCount());
					SphereLayout->SetRadius(PrevSphereUniformLayout->GetRadius());
					SphereLayout->SetRatio(PrevSphereUniformLayout->GetRatio());
					SphereLayout->SetOrientMesh(PrevSphereUniformLayout->GetOrientMesh());
					SphereLayout->SetRotation(PrevSphereUniformLayout->GetRotation());
					SphereLayout->SetScale(PrevSphereUniformLayout->GetScale());
				}
			}
			else if (UCEClonerHoneycombLayout* PrevHoneycombLayout = Cast<UCEClonerHoneycombLayout>(LayoutPair.Value))
			{
				if (UCEClonerHoneycombLayout* HoneycombLayout = ClonerComponent->FindOrAddLayout<UCEClonerHoneycombLayout>())
				{
					HoneycombLayout->SetPlane(PrevHoneycombLayout->GetPlane());
					HoneycombLayout->SetWidthCount(PrevHoneycombLayout->GetWidthCount());
					HoneycombLayout->SetHeightCount(PrevHoneycombLayout->GetHeightCount());
					HoneycombLayout->SetWidthOffset(PrevHoneycombLayout->GetWidthOffset());
					HoneycombLayout->SetHeightOffset(PrevHoneycombLayout->GetHeightOffset());
					HoneycombLayout->SetHeightSpacing(PrevHoneycombLayout->GetHeightSpacing());
					HoneycombLayout->SetWidthSpacing(PrevHoneycombLayout->GetWidthSpacing());
					HoneycombLayout->SetTwistAxis(PrevHoneycombLayout->GetTwistAxis());
					HoneycombLayout->SetTwistFactor(PrevHoneycombLayout->GetTwistFactor() * 100);
				}
			}
			else if (UCEClonerMeshLayout* PrevMeshLayout = Cast<UCEClonerMeshLayout>(LayoutPair.Value))
			{
				if (UCEClonerMeshLayout* MeshLayout = ClonerComponent->FindOrAddLayout<UCEClonerMeshLayout>())
				{
					MeshLayout->SetCount(PrevMeshLayout->GetCount());
					MeshLayout->SetAsset(PrevMeshLayout->GetAsset());
					MeshLayout->SetSampleData(PrevMeshLayout->GetSampleData());
					MeshLayout->SetSampleActorWeak(PrevMeshLayout->GetSampleActor());
				}
			}
			else if (UCEClonerSplineLayout* PrevSplineLayout = Cast<UCEClonerSplineLayout>(LayoutPair.Value))
			{
				if (UCEClonerSplineLayout* SplineLayout = ClonerComponent->FindOrAddLayout<UCEClonerSplineLayout>())
				{
					SplineLayout->SetCount(PrevSplineLayout->GetCount());
					SplineLayout->SetSplineActorWeak(PrevSplineLayout->GetSplineActor());
					SplineLayout->SetOrientMesh(PrevSplineLayout->GetOrientMesh());
				}
			}
			else if (UCEClonerSphereRandomLayout* PrevSphereRandomLayout = Cast<UCEClonerSphereRandomLayout>(LayoutPair.Value))
			{
				if (UCEClonerSphereRandomLayout* SphereLayout = ClonerComponent->FindOrAddLayout<UCEClonerSphereRandomLayout>())
				{
					SphereLayout->SetCount(PrevSphereRandomLayout->GetCount());
					SphereLayout->SetLatitude(PrevSphereRandomLayout->GetLatitude());
					SphereLayout->SetLongitude(PrevSphereRandomLayout->GetLongitude());
					SphereLayout->SetDistribution(PrevSphereRandomLayout->GetDistribution());
					SphereLayout->SetRadius(PrevSphereRandomLayout->GetRadius());
					SphereLayout->SetOrientMesh(PrevSphereRandomLayout->GetOrientMesh());
					SphereLayout->SetRotation(PrevSphereRandomLayout->GetRotation());
					SphereLayout->SetScale(PrevSphereRandomLayout->GetScale());
				}
			}
		}
	}

	PRAGMA_ENABLE_DEPRECATION_WARNINGS

	MigrateToVersion = INDEX_NONE;
}

#if WITH_EDITOR
void ACEClonerActor::OnEditorSelectionChanged(UObject* InSelection)
{
	if (const USelection* ActorSelection = Cast<USelection>(InSelection))
	{
		if (ULevelEditorViewportSettings* ViewportSettings = GetMutableDefault<ULevelEditorViewportSettings>())
		{
			if (ActorSelection->Num() == 1 && ActorSelection->GetSelectedObject(0) == this)
			{
				UseSelectionOutline = ViewportSettings->bUseSelectionOutline;
				ViewportSettings->bUseSelectionOutline = false;
			}
			else if (UseSelectionOutline.IsSet())
			{
				ViewportSettings->bUseSelectionOutline = UseSelectionOutline.GetValue();
				UseSelectionOutline.Reset();
			}
		}
	}
}
#endif
