// Copyright Epic Games, Inc. All Rights Reserved.

#include "Cloner/CEClonerComponent.h"

#include "Async/Async.h"
#include "Cloner/Attachments/CEClonerAttachmentTreeBehavior.h"
#include "Cloner/CEClonerActor.h"
#include "Cloner/Extensions/CEClonerExtensionBase.h"
#include "Cloner/Layouts/CEClonerLayoutBase.h"
#include "Components/BillboardComponent.h"
#include "Components/SkeletalMeshComponent.h"
#include "DataInterface/NiagaraDataInterfaceArrayMesh.h"
#include "Engine/Level.h"
#include "Engine/StaticMeshActor.h"
#include "Engine/Texture2D.h"
#include "Engine/World.h"
#include "Logs/CEClonerLogs.h"
#include "Materials/Material.h"
#include "Materials/MaterialInstanceDynamic.h"
#include "Misc/EnumerateRange.h"
#include "NiagaraFunctionLibrary.h"
#include "NiagaraMeshRendererProperties.h"
#include "NiagaraSystem.h"
#include "Settings/CEClonerEffectorSettings.h"
#include "Subsystems/CEClonerSubsystem.h"
#include "UDynamicMesh.h"
#include "UObject/Package.h"
#include "Utilities/CEClonerEffectorUtilities.h"

#if WITH_EDITOR
#include "Editor/EditorEngine.h"
#include "Engine/StaticMesh.h"
#include "Misc/MessageDialog.h"
#include "Misc/ScopedSlowTask.h"
#include "ScopedTransaction.h"
#include "StaticMeshCompiler.h"
#include "StaticMeshResources.h"
#include "UObject/ObjectSaveContext.h"
#endif

#define LOCTEXT_NAMESPACE "CEClonerComponent"

#if WITH_EDITOR
namespace UE::ClonerEffector::Private
{
	static void SanitizeBakedMeshBounds(UStaticMesh* InMesh)
	{
		if (!InMesh || InMesh->IsCompiling())
		{
			return;
		}

		FStaticMeshRenderData* RenderData = InMesh->GetRenderData();
		if (RenderData && RenderData->Bounds.ContainsNaN())
		{
			UE_LOGF(LogCECloner, Log, "%ls : NaN bounds detected on baked mesh, resetting to zero bounds...", *InMesh->GetPathName());
			// Zeroing bounds as fallback to avoid NaN
			RenderData->Bounds = FBoxSphereBounds(ForceInit);
			InMesh->CalculateExtendedBounds();
		}
	}
}
#endif

UCEClonerComponent::FOnClonerInitialized UCEClonerComponent::OnClonerInitializedDelegate;
UCEClonerComponent::FOnClonerLayoutLoaded UCEClonerComponent::OnClonerLayoutLoadedDelegate;
UCEClonerComponent::FOnClonerMeshUpdated UCEClonerComponent::OnClonerMeshUpdatedDelegate;
UCEClonerComponent::FOnClonerAttachmentChanged UCEClonerComponent::OnClonerActorAttachedDelegate;
UCEClonerComponent::FOnClonerAttachmentChanged UCEClonerComponent::OnClonerActorDetachedDelegate;

UCEClonerComponent::UCEClonerComponent()
	: UNiagaraComponent()
{
	CastShadow = true;
	bReceivesDecals = true;
	bAutoActivate = true;
	bHiddenInGame = false;

#if WITH_EDITOR
	// Do not show bounding box around cloner for better visibility
	SetIsVisualizationComponent(true);

	// Disable use of bounds to focus to avoid de-zoom
	SetIgnoreBoundsForEditorFocus(true);
#endif

	bIsEditorOnly = false;

	// Show sprite for this component to visualize it when empty
#if WITH_EDITORONLY_DATA
	bVisualizeComponent = true;
#endif

	if (!IsTemplate())
	{
		UCEClonerSubsystem::OnClonerSetEnabled().AddUObject(this, &UCEClonerComponent::OnClonerSetEnabled);

		// Apply default layout
		const TArray<FName> LayoutNames = GetClonerLayoutNames();
		LayoutName = !LayoutNames.IsEmpty() ? LayoutNames[0] : NAME_None;

		// Apply default behavior
		const TArray<FName> BehaviorNames = GetClonerTreeBehaviorNames();
		TreeBehaviorName = !BehaviorNames.IsEmpty() ? BehaviorNames[0] : NAME_None;

		// Bind attachment tree events
		ClonerTree.OnItemAttached().BindUObject(this, &UCEClonerComponent::OnTreeItemAttached);
		ClonerTree.OnItemDetached().BindUObject(this, &UCEClonerComponent::OnTreeItemDetached);
	}
}

void UCEClonerComponent::PostInitProperties()
{
	Super::PostInitProperties();

	if (HasAnyFlags(RF_ClassDefaultObject))
	{
		// Register new type def for niagara

		constexpr ENiagaraTypeRegistryFlags MeshFlags =
			ENiagaraTypeRegistryFlags::AllowAnyVariable |
			ENiagaraTypeRegistryFlags::AllowParameter;

		FNiagaraTypeRegistry::Register(FNiagaraTypeDefinition(StaticEnum<ECEClonerMeshRenderMode>()), MeshFlags);
		FNiagaraTypeRegistry::Register(FNiagaraTypeDefinition(StaticEnum<ECEClonerGridConstraint>()), MeshFlags);
		FNiagaraTypeRegistry::Register(FNiagaraTypeDefinition(StaticEnum<ECEClonerPlane>()), MeshFlags);
		FNiagaraTypeRegistry::Register(FNiagaraTypeDefinition(StaticEnum<ECEClonerAxis>()), MeshFlags);
		FNiagaraTypeRegistry::Register(FNiagaraTypeDefinition(StaticEnum<ECEClonerEasing>()), MeshFlags);
		FNiagaraTypeRegistry::Register(FNiagaraTypeDefinition(StaticEnum<ECEClonerMeshAsset>()), MeshFlags);
		FNiagaraTypeRegistry::Register(FNiagaraTypeDefinition(StaticEnum<ECEClonerMeshSampleData>()), MeshFlags);
		FNiagaraTypeRegistry::Register(FNiagaraTypeDefinition(StaticEnum<ECEClonerEffectorType>()), MeshFlags);
		FNiagaraTypeRegistry::Register(FNiagaraTypeDefinition(StaticEnum<ECEClonerTextureSampleChannel>()), MeshFlags);
		FNiagaraTypeRegistry::Register(FNiagaraTypeDefinition(StaticEnum<ECEClonerCompareMode>()), MeshFlags);
		FNiagaraTypeRegistry::Register(FNiagaraTypeDefinition(StaticEnum<ECEClonerEffectorMode>()), MeshFlags);
		FNiagaraTypeRegistry::Register(FNiagaraTypeDefinition(StaticEnum<ECEClonerSpawnLoopMode>()), MeshFlags);
		FNiagaraTypeRegistry::Register(FNiagaraTypeDefinition(StaticEnum<ECEClonerSpawnBehaviorMode>()), MeshFlags);
		FNiagaraTypeRegistry::Register(FNiagaraTypeDefinition(StaticEnum<ECEClonerEffectorPushDirection>()), MeshFlags);
	}
}

void UCEClonerComponent::PostLoad()
{
	Super::PostLoad();

	InitializeCloner();
}

#if WITH_EDITOR
void UCEClonerComponent::PreSave(FObjectPreSaveContext InSaveContext)
{
	Super::PreSave(InSaveContext);

	// The baked meshes are built with UStaticMesh::Build(), this launches an mesh compilation (async or sync).
	// While that build is ongoing, FStaticMeshRenderData::Bounds is left uninitialized (NaN). 
	// Saving the level in that window serializes the NaN bounds and produces a warning on the next load. 
	// To avoid that, we force any pending compilation to finish here so the render data (and its bounds) 
	// are valid by the time the mesh is serialized.
	TArray<UStaticMesh*> CompilingMeshes;
	CompilingMeshes.Reserve(ClonerTree.RootItemAttachments.Num());
	for (const FCEClonerAttachmentRootItem& RootItem : ClonerTree.RootItemAttachments)
	{
		if (UStaticMesh* BakedMesh = RootItem.MergedBakedMesh.Get(); BakedMesh && BakedMesh->IsCompiling())
		{
			CompilingMeshes.Add(BakedMesh);
		}
	}

	if (!CompilingMeshes.IsEmpty())
	{
		FStaticMeshCompilingManager::Get().FinishCompilation(CompilingMeshes);
	}
	// Used MIDs could get gc'd on save, causing the mesh renderer to have an array of dangling pointers.
	// MarkRenderStateDirty to update to the latest state.
	MarkRenderStateDirty();
}

void UCEClonerComponent::PreDuplicate(FObjectDuplicationParameters& InParams)
{
	Super::PreDuplicate(InParams);

	ForceUpdateCloner();
}

void UCEClonerComponent::PostDuplicate(bool bInPIE)
{
	SetAsset(nullptr);

	Super::PostDuplicate(bInPIE);

	RegisterTicker();

	ForceUpdateCloner();
}

void UCEClonerComponent::PostEditImport()
{
	SetAsset(nullptr);

	Super::PostEditImport();

	RegisterTicker();

	ForceUpdateCloner();
}

void UCEClonerComponent::PostEditUndo()
{
	Super::PostEditUndo();

	// Reregister ticker in case this object was destroyed then undo
	RegisterTicker();

	ClonerTree.MarkAttachmentOutdated();
}

const TCEPropertyChangeDispatcher<UCEClonerComponent> UCEClonerComponent::PropertyChangeDispatcher =
{
	/** General */
	{ GET_MEMBER_NAME_CHECKED(UCEClonerComponent, bEnabled), &UCEClonerComponent::OnEnabledChanged },
	{ GET_MEMBER_NAME_CHECKED(UCEClonerComponent, Seed), &UCEClonerComponent::OnSeedChanged },
	{ GET_MEMBER_NAME_CHECKED(UCEClonerComponent, Color), &UCEClonerComponent::OnColorChanged },
	{ GET_MEMBER_NAME_CHECKED(UCEClonerComponent, GlobalScale), &UCEClonerComponent::OnGlobalScaleChanged },
	{ GET_MEMBER_NAME_CHECKED(UCEClonerComponent, GlobalRotation), &UCEClonerComponent::OnGlobalRotationChanged },
	{ GET_MEMBER_NAME_CHECKED(UCEClonerComponent, TreeBehaviorName), &UCEClonerComponent::OnTreeBehaviorNameChanged },
	{ GET_MEMBER_NAME_CHECKED(UCEClonerComponent, bVisualizerSpriteVisible), &UCEClonerComponent::OnVisualizerSpriteVisibleChanged },
	/** Layout */
	{ GET_MEMBER_NAME_CHECKED(UCEClonerComponent, LayoutName), &UCEClonerComponent::OnLayoutNameChanged },
};

void UCEClonerComponent::PostEditChangeProperty(FPropertyChangedEvent& InPropertyChangedEvent)
{
	Super::PostEditChangeProperty(InPropertyChangedEvent);

	PropertyChangeDispatcher.OnPropertyChanged(this, InPropertyChangedEvent);
}
#endif

void UCEClonerComponent::OnComponentCreated()
{
	Super::OnComponentCreated();

	InitializeCloner();
}

void UCEClonerComponent::OnComponentDestroyed(bool bInDestroyingHierarchy)
{
	Super::OnComponentDestroyed(bInDestroyingHierarchy);

	ClonerTree.Cleanup();

	FWorldDelegates::LevelAddedToWorld.RemoveAll(this);

	FTSTicker::RemoveTicker(ClonerTickerHandle);
	ClonerTickerHandle.Reset();

	FTSTicker::RemoveTicker(ConversionTickerHandle);
	ConversionTickerHandle.Reset();
}

void UCEClonerComponent::UpdateClonerRenderState()
{
	/**
	 * Perform a mesh update when asset is valid,
	 * An update is not already ongoing,
	 * Meshes are out of date after an attachment tree update,
	 * Tree is up to date
	 */
	if (!GetAsset()
		|| IsGarbageCollectingAndLockingUObjectHashTables()
		|| bClonerMeshesUpdating
		|| !ClonerTree.bItemAttachmentsDirty
		|| ClonerTree.Status != ECEClonerAttachmentStatus::Updated)
	{
		return;
	}

#if WITH_EDITOR
	UpdateDirtyMeshesAsync();
#else
	OnDirtyMeshesUpdated(true);
#endif
}

#if WITH_EDITOR
FName UCEClonerComponent::GetActiveExtensionsPropertyName()
{
	return GET_MEMBER_NAME_CHECKED(UCEClonerComponent, ActiveExtensions);
}

FName UCEClonerComponent::GetActiveLayoutPropertyName()
{
	return GET_MEMBER_NAME_CHECKED(UCEClonerComponent, ActiveLayout);
}

FName UCEClonerComponent::GetLayoutNamePropertyName()
{
	return GET_MEMBER_NAME_CHECKED(UCEClonerComponent, LayoutName);
}

void UCEClonerComponent::UpdateDirtyMeshesAsync()
{
	if (bClonerMeshesUpdating)
	{
		return;
	}

	bClonerMeshesUpdating = true;

	if (ConversionTickerHandle.IsValid())
	{
		FTSTicker::RemoveTicker(ConversionTickerHandle);
	}

	UE_LOGF(LogCECloner, Verbose, "%ls : Updating %i dirty actor meshes...", *GetOwner()->GetActorNameOrLabel(), ClonerTree.DirtyItemAttachments.Num());

	const double StartTime = FPlatformTime::Seconds();
	const int32 DirtyItemCount = ClonerTree.DirtyItemAttachments.Num();
	ConversionTickerHandle = FTSTicker::GetCoreTicker().AddTicker(FTickerDelegate::CreateWeakLambda(this, [This = this, StartTime, DirtyItemCount](float InDelta)->bool
		{
			if (!This)
			{
				return false;
			}

			FGCScopeGuard GCGuard;
			const double StartTaskTime = FPlatformTime::Seconds();

			{
				// update actor baked dynamic meshes
				for (TSet<TWeakObjectPtr<AActor>>::TIterator It(This->ClonerTree.DirtyItemAttachments); It; ++It)
				{
					const TWeakObjectPtr<AActor>& Attachment = *It;

					if (AActor* DirtyActor = Attachment.Get())
					{
						This->UpdateActorDynamicMesh(DirtyActor);
					}

					It.RemoveCurrent();

					if ((FPlatformTime::Seconds() - StartTaskTime) > InDelta)
					{
						// Continue next tick
						return true;
					}
				}
			}

			{
				// Update actors baked static mesh
				for (const FCEClonerAttachmentRootItem& RootItem : This->ClonerTree.RootItemAttachments)
				{
					const UStaticMesh* RootStaticMesh = RootItem.MergedBakedMesh.Get();

					if (!RootStaticMesh)
					{
						if (AActor* RootActor = RootItem.RootActor.Get())
						{
							This->UpdateActorStaticMesh(RootActor);
						}
					}

					if ((FPlatformTime::Seconds() - StartTaskTime) > InDelta)
					{
						// Continue next tick
						return true;
					}
				}
			}

			UE_LOGF(LogCECloner, Verbose, "%ls : Updated %i dirty actor meshes in %.3fs", *This->GetOwner()->GetActorNameOrLabel(), DirtyItemCount, FPlatformTime::Seconds() - StartTime);

			// update niagara asset
			This->OnDirtyMeshesUpdated(/** Success */true);

			// Stop
			return false;
		})
	);
}

void UCEClonerComponent::UpdateActorDynamicMesh(AActor* InActor)
{
	if (!InActor)
	{
		return;
	}

	const AActor* ClonerActor = GetOwner();

	if (!ClonerActor)
	{
		return;
	}

	FCEClonerAttachmentItem* AttachmentItem = ClonerTree.ItemAttachmentMap.Find(InActor);

	if (!AttachmentItem || AttachmentItem->MeshStatus != ECEClonerAttachmentStatus::Outdated)
	{
		return;
	}

	AttachmentItem->MeshStatus = ECEClonerAttachmentStatus::Updating;

	UDynamicMesh* Mesh = NewObject<UDynamicMesh>();
	TArray<TWeakObjectPtr<UMaterialInterface>> MeshMaterials;

	MeshBuilder.AppendActor(InActor, InActor->GetActorTransform());
	MeshBuilder.BuildDynamicMesh(Mesh, MeshMaterials);
	MeshBuilder.Reset();

	AttachmentItem->BakedMesh = Mesh;

	TArray<TWeakObjectPtr<UMaterialInterface>> UnsetMaterials;
	UMaterialInterface* DefaultMaterial = LoadObject<UMaterialInterface>(nullptr, UCEClonerEffectorSettings::DefaultMaterialPath);
	if (UE::ClonerEffector::Utilities::FilterSupportedMaterials(MeshMaterials, UnsetMaterials, DefaultMaterial))
	{
		if (UCEClonerSubsystem* ClonerSubsystem = UCEClonerSubsystem::Get())
		{
			ClonerSubsystem->FireMaterialWarning(ClonerActor, InActor, UnsetMaterials);	
		}
	}

	AttachmentItem->BakedMaterials = MoveTemp(MeshMaterials);

	// Was the mesh invalidated during the update process, then leave it outdated
	if (AttachmentItem->MeshStatus == ECEClonerAttachmentStatus::Updating) //-V547
	{
		AttachmentItem->MeshStatus = ECEClonerAttachmentStatus::Updated;
		ClonerTree.MarkCacheOutdated(InActor);
	}

	UE_LOGF(LogCECloner, Verbose, "%ls : Updated actor dynamic mesh : %ls - %i", *ClonerActor->GetActorNameOrLabel(), *InActor->GetActorNameOrLabel(), Mesh->GetTriangleCount());
}

void UCEClonerComponent::UpdateActorStaticMesh(AActor* InRootActor)
{
	if (!InRootActor)
	{
		return;
	}

	const AActor* ClonerActor = GetOwner();

	if (!ClonerActor)
	{
		return;
	}

	FCEClonerAttachmentRootItem* RootItem = ClonerTree.RootItemAttachments.FindByKey(InRootActor);

	if (!RootItem)
	{
		return;
	}

	const FCEClonerAttachmentItem* ActorAttachmentItem = ClonerTree.ItemAttachmentMap.Find(InRootActor);

	if (!ActorAttachmentItem)
	{
		return;
	}

	TArray<const FCEClonerAttachmentItem*> AttachmentItems;
	ClonerTree.GetAttachments(InRootActor, AttachmentItems, /** Recurse */true);

	uint32 Hash = HashCombineFast(ClonerActor->GetUniqueID(), InRootActor->GetUniqueID());
	for (const FCEClonerAttachmentItem* AttachmentItem : AttachmentItems)
	{
		if (!AttachmentItem)
		{
			continue;
		}

		const UDynamicMesh* BakedDynamicMesh = AttachmentItem->BakedMesh;
		if (!BakedDynamicMesh)
		{
			continue;
		}

		Hash = HashCombineFast(Hash, BakedDynamicMesh->GetUniqueID());
		MeshBuilder.AppendMesh(AttachmentItem->BakedMesh, AttachmentItem->BakedMaterials, AttachmentItem->ActorTransform);
	}

	// Avoid dirtying asset by creating it transient first,
	// and avoid bounds log spamming by renaming asset with prefix LandscapeNaniteMesh
	const FName PreObjectName(FString::Printf(TEXT("LandscapeNaniteMesh_%i_%i"), Hash, AttachmentItems.Num()));
	UStaticMesh* Mesh = NewObject<UStaticMesh>(GetTransientPackage(), PreObjectName);

	// Rename the asset + outer once build is done
	Mesh->OnPostMeshBuild().AddUObject(this, &UCEClonerComponent::OnActorStaticMeshPostBuild);

	TArray<TWeakObjectPtr<UMaterialInterface>> MeshMaterials;
	ClonerTree.bItemAttachmentsDirty = MeshBuilder.BuildStaticMesh(Mesh, MeshMaterials);
	MeshBuilder.Reset();

	RootItem->MergedBakedMesh = Mesh;
#if WITH_EDITORONLY_DATA
	RootItem->MeshSize = ClonerTree.GetAttachmentBounds(InRootActor, /** IncludeChildren */true).GetSize();
#endif

	UE_LOGF(LogCECloner, Verbose, "%ls : Updated actor static mesh : %ls - %i - %i", *ClonerActor->GetActorNameOrLabel(), *InRootActor->GetActorNameOrLabel(), Mesh->GetNumTriangles(/** LOD */0), AttachmentItems.Num());
}

void UCEClonerComponent::OnActorStaticMeshPostBuild(UStaticMesh* InMesh)
{
	if (InMesh)
	{
		InMesh->OnPostMeshBuild().RemoveAll(this);
		const FName MeshName = MakeUniqueObjectName(this, UStaticMesh::StaticClass(), TEXT("ClonerMesh"));
		InMesh->Rename(*MeshName.ToString(), this, REN_NonTransactional | REN_DoNotDirty);
		// Done compiling, if bounds are NaN, they will be zero out, to avoid saving a mesh with NaN and triggering warning during deserialize
		// UStaticMesh::Build can run async or synchronous, this will get called when build is done in both cases
		UE::ClonerEffector::Private::SanitizeBakedMeshBounds(InMesh);
	}
}
#endif // WITH_EDITOR

void UCEClonerComponent::OnDirtyMeshesUpdated(bool bInSuccess)
{
	ConversionTickerHandle.Reset();
	bClonerMeshesUpdating = false;

	// Update niagara parameters
	if (bInSuccess)
	{
		UpdateClonerMeshes();
	}
}

void UCEClonerComponent::GetPreloadDependencies(TArray<UObject*>& OutDeps)
{
	Super::GetPreloadDependencies(OutDeps);

	for (const FCEClonerAttachmentRootItem& RootItem : ClonerTree.RootItemAttachments)
	{
		if (UStaticMesh* BakedMesh = RootItem.MergedBakedMesh.Get())
		{
			OutDeps.Add(BakedMesh);
		}
	}
}

void UCEClonerComponent::InitializeCloner()
{
	AActor* Owner = GetOwner();
	if (bClonerInitialized
		|| IsTemplate()
		|| !Owner)
	{
		return;
	}

	bClonerInitialized = true;

#if WITH_EDITOR
	if (GIsEditor)
	{
		OnVisualizerSpriteVisibleChanged();
	}

	// Skip init for preview actor
	if (Owner->bIsEditorPreviewActor)
	{
		return;
	}
#endif

	ClonerTree.SetAttachmentRoot(Owner);
	OnTreeBehaviorNameChanged();

	// When level is streamed in, wait until actor hierarchy and resources are ready before initializing
	if (const ULevel* Level = GetComponentLevel())
	{
		if (!Level->IsPersistentLevel() && Level->HasAnyInternalFlags(EInternalObjectFlags_AsyncLoading))
		{
			FWorldDelegates::LevelAddedToWorld.AddUObject(this, &UCEClonerComponent::OnLevelLoaded);
		}
		else
		{
			PostInitializeCloner();
		}
	}
}

void UCEClonerComponent::OnLevelLoaded(ULevel* InLevel, UWorld* InWorld)
{
	const ULevel* Level = GetComponentLevel();

	if (!IsValid(Level) || Level != InLevel)
	{
		return;
	}

	FWorldDelegates::LevelAddedToWorld.RemoveAll(this);

	PostInitializeCloner();
}

void UCEClonerComponent::PostInitializeCloner()
{
	// Register a custom ticker to avoid using the component tick that needs the simulation to be solo
	RegisterTicker();
	OnClonerInitializedDelegate.Broadcast(this);
}

void UCEClonerComponent::RegisterTicker()
{
	if (!bClonerInitialized)
	{
		return;
	}

	// Register custom ticker to avoid using component tick and niagara solo mode
	FTSTicker::RemoveTicker(ClonerTickerHandle);
	ClonerTickerHandle = FTSTicker::GetCoreTicker().AddTicker(FTickerDelegate::CreateUObject(this, &UCEClonerComponent::TickCloner));
}

bool UCEClonerComponent::CheckResourcesReady()
{
	const ULevel* Level = GetComponentLevel();

	// Check level is not async loading
	if (!IsValid(Level) || Level->HasAnyInternalFlags(EInternalObjectFlags_AsyncLoading))
	{
		return false;
	}

	// Check cached meshes are not async loading
	if (!ClonerTree.IsCacheAvailable(/** AllowInvalid */true))
	{
		return false;
	}

	// Check cached active system is not async loading
	for (const TObjectPtr<UCEClonerLayoutBase>& Layout : LayoutInstances)
	{
		if (Layout && Layout->GetLayoutName() == LayoutName)
		{
			if (Layout->GetSystem() && Layout->GetSystem()->HasAnyInternalFlags(EInternalObjectFlags_AsyncLoading))
			{
				return false;
			}
		}
	}

	return true;
}

bool UCEClonerComponent::TickCloner(float InDelta)
{
	if (!bClonerInitialized)
	{
		return false;
	}

	if (!bClonerResourcesReady)
	{
		if (!CheckResourcesReady())
		{
			return true;
		}

		OnEnabledChanged();
		bClonerResourcesReady = true;
#if WITH_EDITOR
		// All baked meshes are loaded here thanks to CheckResourcesReady.
		// Repair any mesh serialized with NaN bounds so the next save persists finite bounds and the warning stops recurring.
		for (const FCEClonerAttachmentRootItem& RootItem : ClonerTree.RootItemAttachments)
		{
			UE::ClonerEffector::Private::SanitizeBakedMeshBounds(RootItem.MergedBakedMesh.Get());
		}
#endif
	}

	if (bEnabled)
	{
		// Update attachment tree
		ClonerTree.UpdateAttachments();
		UpdateClonerRenderState();

		// Update layout parameters
		if (ActiveLayout && ActiveLayout->IsLayoutDirty())
		{
			ActiveLayout->UpdateLayoutParameters();
		}

		// Update extension parameters
		for (const TObjectPtr<UCEClonerExtensionBase>& ActiveExtension : ActiveExtensions)
		{
			if (ActiveExtension && ActiveExtension->IsExtensionDirty())
			{
				ActiveExtension->UpdateExtensionParameters();
			}
		}

		// Is a simulation reset needed
		if (bNeedsRefresh)
		{
			bNeedsRefresh = false;
			RequestClonerUpdate(/**Immediate*/true);
		}
	}

	return true;
}

void UCEClonerComponent::SetEnabled(bool bInEnable)
{
	if (bInEnable == bEnabled)
	{
		return;
	}

	bEnabled = bInEnable;
	OnEnabledChanged();
}

void UCEClonerComponent::SetSeed(int32 InSeed)
{
	if (InSeed == Seed)
	{
		return;
	}

	Seed = InSeed;
	OnSeedChanged();
}

void UCEClonerComponent::SetColor(const FLinearColor& InColor)
{
	if (InColor.Equals(Color))
	{
		return;
	}

	Color = InColor;
	OnColorChanged();
}

void UCEClonerComponent::SetGlobalScale(const FVector& InScale)
{
	if (GlobalScale.Equals(InScale))
	{
		return;
	}

	GlobalScale = InScale.ComponentMax(FVector(UE_KINDA_SMALL_NUMBER));;
	OnGlobalScaleChanged();
}

void UCEClonerComponent::SetGlobalRotation(const FRotator& InRotation)
{
	if (GlobalRotation.Equals(InRotation))
	{
		return;
	}

	GlobalRotation = InRotation;
	OnGlobalRotationChanged();
}

void UCEClonerComponent::SetLayoutName(FName InLayoutName)
{
	if (LayoutName == InLayoutName)
	{
		return;
	}

	const TArray<FName> LayoutNames = GetClonerLayoutNames();
	if (!LayoutNames.Contains(InLayoutName))
	{
		return;
	}

	LayoutName = InLayoutName;
	OnLayoutNameChanged();
}

void UCEClonerComponent::SetLayoutClass(TSubclassOf<UCEClonerLayoutBase> InLayoutClass)
{
	if (!InLayoutClass.Get())
	{
		return;
	}

	if (const UCEClonerSubsystem* ClonerSubsystem = UCEClonerSubsystem::Get())
	{
		const FName NewLayoutName = ClonerSubsystem->FindLayoutName(InLayoutClass);

		if (!NewLayoutName.IsNone())
		{
			SetLayoutName(NewLayoutName);
		}
	}
}

TSubclassOf<UCEClonerLayoutBase> UCEClonerComponent::GetLayoutClass() const
{
	return ActiveLayout ? ActiveLayout->GetClass() : nullptr;
}

#if WITH_EDITOR
void UCEClonerComponent::SetTreeBehaviorName(FName InBehaviorName)
{
	if (TreeBehaviorName == InBehaviorName)
	{
		return;
	}

	const TArray<FName> BehaviorNames = GetClonerTreeBehaviorNames();
	if (!BehaviorNames.Contains(InBehaviorName))
	{
		return;
	}

	TreeBehaviorName = InBehaviorName;
	OnTreeBehaviorNameChanged();
}

void UCEClonerComponent::SetVisualizerSpriteVisible(bool bInVisible)
{
	if (bVisualizerSpriteVisible == bInVisible)
	{
		return;
	}

	bVisualizerSpriteVisible = bInVisible;
	OnVisualizerSpriteVisibleChanged();
}
#endif

int32 UCEClonerComponent::GetMeshCount() const
{
	if (const UNiagaraDataInterfaceArrayMesh* const MeshArrayDI = UE::ClonerEffector::FindMeshArrayDataInterface(const_cast<UCEClonerComponent*>(this)))
	{
		return MeshArrayDI->MeshData.Num();
	}
	return 0;
}

int32 UCEClonerComponent::GetAttachmentCount() const
{
	return ClonerTree.ItemAttachmentMap.Num();
}

#if WITH_EDITOR
void UCEClonerComponent::ForceUpdateCloner()
{
	ClonerTree.MarkAttachmentOutdated();
	ClonerTree.UpdateAttachments();
	UpdateClonerRenderState();
	OnLayoutNameChanged();
}

void UCEClonerComponent::OpenClonerSettings()
{
	if (const UCEClonerEffectorSettings* ClonerSettings = GetDefault<UCEClonerEffectorSettings>())
	{
		ClonerSettings->OpenEditorSettingsWindow();
	}
}

void UCEClonerComponent::CreateDefaultActorAttached()
{
	const UCEClonerEffectorSettings* ClonerEffectorSettings = GetDefault<UCEClonerEffectorSettings>();
	if (!ClonerEffectorSettings || !ClonerEffectorSettings->GetSpawnDefaultActorAttached())
	{
		return;
	}

	// Only spawn if world is valid and not a preview actor
	UWorld* World = GetWorld();
	AActor* Owner = GetOwner();

	if (!IsValid(World)
		|| !IsValid(Owner)
		|| Owner->bIsEditorPreviewActor)
	{
		return;
	}

	// Only spawn if no actor is attached below it
	TArray<AActor*> AttachedActors;
	constexpr bool bReset = true;
	constexpr bool bRecursive = false;
	Owner->GetAttachedActors(AttachedActors, bReset, bRecursive);

	if (!AttachedActors.IsEmpty())
	{
		return;
	}

	UStaticMesh* DefaultStaticMesh = ClonerEffectorSettings->GetDefaultStaticMesh();
	UMaterialInterface* DefaultMaterial = ClonerEffectorSettings->GetDefaultMaterial();

	if (!DefaultStaticMesh || !DefaultMaterial)
	{
		return;
	}

	FScopedTransaction Transaction(LOCTEXT("CreateDefaultActorAttached", "Create cloner default actor attached"), !GIsTransacting);

	Modify();

	// Spawn attached actor with same flags as this actor
	FActorSpawnParameters SpawnParameters;
	SpawnParameters.OverrideLevel = Owner->GetLevel();
	SpawnParameters.ObjectFlags = GetFlags() | RF_Transactional;
	SpawnParameters.bTemporaryEditorActor = false;

	const FVector ClonerLocation = GetComponentLocation();
	const FRotator ClonerRotation = FRotator::ZeroRotator;

	if (AStaticMeshActor* DefaultActorAttached = World->SpawnActor<AStaticMeshActor>(ClonerLocation, ClonerRotation, SpawnParameters))
	{
		UStaticMeshComponent* StaticMeshComponent = DefaultActorAttached->GetStaticMeshComponent();
		StaticMeshComponent->SetStaticMesh(DefaultStaticMesh);
		StaticMeshComponent->SetMaterial(0, DefaultMaterial);

		DefaultActorAttached->SetMobility(EComponentMobility::Movable);
		DefaultActorAttached->AttachToActor(GetOwner(), FAttachmentTransformRules::KeepWorldTransform);

		FActorLabelUtilities::SetActorLabelUnique(DefaultActorAttached, TEXT("DefaultClone"));
	}
}

void UCEClonerComponent::ConvertToStaticMesh()
{
	if (!IsValid(this) || !bEnabled)
	{
		return;
	}

	FScopedSlowTask SlowTask(0.0f, LOCTEXT("ConvertToStaticMesh", "Converting cloner to static mesh"));
	SlowTask.MakeDialog();

	if (UE::ClonerEffector::Conversion::ConvertClonerToStaticMesh(this))
	{
		UE_LOGF(LogCECloner, Log, "%ls : ConvertToStaticMesh Completed", *GetOwner()->GetActorNameOrLabel())
	}
	else
	{
		UE_LOGF(LogCECloner, Warning, "%ls : ConvertToStaticMesh Failed", *GetOwner()->GetActorNameOrLabel())
	}
}

void UCEClonerComponent::ConvertToDynamicMesh()
{
	if (!IsValid(this) || !bEnabled)
	{
		return;
	}

	FScopedSlowTask SlowTask(0.0f, LOCTEXT("ConvertToDynamicMesh", "Converting cloner to dynamic mesh"));
	SlowTask.MakeDialog();

	if (UE::ClonerEffector::Conversion::ConvertClonerToDynamicMesh(this))
	{
		UE_LOGF(LogCECloner, Log, "%ls : ConvertToDynamicMesh Completed", *GetOwner()->GetActorNameOrLabel())
	}
	else
	{
		UE_LOGF(LogCECloner, Warning, "%ls : ConvertToDynamicMesh Failed", *GetOwner()->GetActorNameOrLabel())
	}
}

void UCEClonerComponent::ConvertToStaticMeshes()
{
	if (!IsValid(this) || !bEnabled)
	{
		return;
	}

	FScopedSlowTask SlowTask(0.0f, LOCTEXT("ConvertToStaticMeshes", "Converting cloner to static meshes"));
	SlowTask.MakeDialog();

	if (!UE::ClonerEffector::Conversion::ConvertClonerToStaticMeshes(this).IsEmpty())
	{
		UE_LOGF(LogCECloner, Log, "%ls : ConvertToStaticMeshes Completed", *GetOwner()->GetActorNameOrLabel())
	}
	else
	{
		UE_LOGF(LogCECloner, Warning, "%ls : ConvertToStaticMeshes Failed", *GetOwner()->GetActorNameOrLabel())
	}
}

void UCEClonerComponent::ConvertToDynamicMeshes()
{
	if (!IsValid(this) || !bEnabled)
	{
		return;
	}

	FScopedSlowTask SlowTask(0.0f, LOCTEXT("ConvertToDynamicMeshes", "Converting cloner to dynamic meshes"));
	SlowTask.MakeDialog();

	if (!UE::ClonerEffector::Conversion::ConvertClonerToDynamicMeshes(this).IsEmpty())
	{
		UE_LOGF(LogCECloner, Log, "%ls : ConvertToDynamicMeshes Completed", *GetOwner()->GetActorNameOrLabel())
	}
	else
	{
		UE_LOGF(LogCECloner, Warning, "%ls : ConvertToDynamicMeshes Failed", *GetOwner()->GetActorNameOrLabel())
	}
}

void UCEClonerComponent::ConvertToInstancedStaticMeshes()
{
	if (!IsValid(this) || !bEnabled)
	{
		return;
	}

	FScopedSlowTask SlowTask(0.0f, LOCTEXT("ConvertToInstancedStaticMeshes", "Converting cloner to instanced static meshes"));
	SlowTask.MakeDialog();

	if (!UE::ClonerEffector::Conversion::ConvertClonerToInstancedStaticMeshes(this).IsEmpty())
	{
		UE_LOGF(LogCECloner, Log, "%ls : ConvertToInstancedStaticMeshes Completed", *GetOwner()->GetActorNameOrLabel())
	}
	else
	{
		UE_LOGF(LogCECloner, Warning, "%ls : ConvertToInstancedStaticMeshes Failed", *GetOwner()->GetActorNameOrLabel())
	}
}
#endif

void UCEClonerComponent::RequestClonerUpdate(bool bInImmediate)
{
	if (!bEnabled)
	{
		return;
	}

	if (bInImmediate)
	{
		bNeedsRefresh = false;

		FNiagaraUserRedirectionParameterStore& UserParameterStore = GetOverrideParameters();
		UserParameterStore.PostGenericEditChange();
	}
	else
	{
		bNeedsRefresh = true;
	}
}

void UCEClonerComponent::OnEnabledChanged()
{
	if (bEnabled)
	{
		OnClonerEnabled();
	}
	else
	{
		OnClonerDisabled();
	}
}

void UCEClonerComponent::OnClonerEnabled()
{
	for (const TObjectPtr<UCEClonerExtensionBase>& ActiveExtension : ActiveExtensions)
	{
		ActiveExtension->ActivateExtension();
	}

	OnLayoutNameChanged();
}

void UCEClonerComponent::OnClonerDisabled()
{
	for (const TObjectPtr<UCEClonerExtensionBase>& ActiveExtension : ActiveExtensions)
	{
		ActiveExtension->DeactivateExtension();
	}

	DeactivateImmediate();
	SetAsset(nullptr);
}

void UCEClonerComponent::OnClonerSetEnabled(const UWorld* InWorld, bool bInEnabled, bool bInTransact)
{
	if (GetWorld() == InWorld)
	{
#if WITH_EDITOR
		if (bInTransact)
		{
			Modify();
		}
#endif

		SetEnabled(bInEnabled);
	}
}

void UCEClonerComponent::OnSeedChanged()
{
	if (!bEnabled)
	{
		return;
	}

	SetRandomSeedOffset(Seed);

	RequestClonerUpdate();
}

void UCEClonerComponent::OnColorChanged()
{
	SetColorParameter(TEXT("EffectorDefaultColor"), Color);
}

void UCEClonerComponent::OnGlobalScaleChanged()
{
	GlobalScale = GlobalScale.ComponentMax(FVector(UE_KINDA_SMALL_NUMBER));
	ClonerTree.bItemAttachmentsDirty = true;
}

void UCEClonerComponent::OnGlobalRotationChanged()
{
	ClonerTree.bItemAttachmentsDirty = true;
}

void UCEClonerComponent::OnTreeBehaviorNameChanged()
{
	const TArray<FName> BehaviorNames = GetClonerTreeBehaviorNames();

	// Set default if value does not exists
	if (!BehaviorNames.Contains(TreeBehaviorName) && !BehaviorNames.IsEmpty())
	{
		TreeBehaviorName = BehaviorNames[0];
	}

	if (const UCEClonerSubsystem* ClonerSubsystem = UCEClonerSubsystem::Get())
	{
		if (const TSharedPtr<ICEClonerAttachmentTreeBehavior> TreeBehavior = ClonerSubsystem->CreateAttachmentTreeBehavior(TreeBehaviorName))
		{
			ClonerTree.SetBehaviorImplementation(TreeBehavior.ToSharedRef());
		}
	}
}

void UCEClonerComponent::OnLayoutNameChanged()
{
	if (!bEnabled)
	{
		return;
	}

	const TArray<FName> LayoutNames = GetClonerLayoutNames();

	// Set default if value does not exists
	if (!LayoutNames.Contains(LayoutName) && !LayoutNames.IsEmpty())
	{
		LayoutName = LayoutNames[0];
	}

	UCEClonerLayoutBase* NewActiveLayout = FindOrAddLayout(LayoutName);

	// Apply layout
	SetClonerActiveLayout(NewActiveLayout);
}

#if WITH_EDITOR
void UCEClonerComponent::OnVisualizerSpriteVisibleChanged()
{
	if (GetWorld() != nullptr)
	{
		if (UTexture2D* SpriteTexture = LoadObject<UTexture2D>(nullptr, SpriteTexturePath))
		{
			CreateSpriteComponent(SpriteTexture);

			if (SpriteComponent)
			{
				if (SpriteComponent->Sprite != SpriteTexture)
				{
					SpriteComponent->SetSprite(SpriteTexture);
				}

				SpriteComponent->SetVisibility(bVisualizerSpriteVisible, /** PropagateToChildren */false);
			}
		}
	}
}
#endif

void UCEClonerComponent::OnTreeItemAttached(AActor* InActor, FCEClonerAttachmentItem& InItem)
{
	if (InActor)
	{
		UE::ClonerEffector::Utilities::SetActorVisibility(InActor, /** Visible */false);
		OnClonerActorAttachedDelegate.Broadcast(this, InActor);
	}
}

void UCEClonerComponent::OnTreeItemDetached(AActor* InActor, FCEClonerAttachmentItem& InItem)
{
	if (InActor)
	{
		UE::ClonerEffector::Utilities::SetActorVisibility(InActor, /** Visible */true);
		OnClonerActorDetachedDelegate.Broadcast(this, InActor);
	}
}

UCEClonerLayoutBase* UCEClonerComponent::FindOrAddLayout(TSubclassOf<UCEClonerLayoutBase> InClass)
{
	const UCEClonerSubsystem* Subsystem = UCEClonerSubsystem::Get();

	if (!Subsystem)
	{
		return nullptr;
	}

	const FName ClassLayoutName = Subsystem->FindLayoutName(InClass);

	if (ClassLayoutName.IsNone())
	{
		return nullptr;
	}

	return FindOrAddLayout(ClassLayoutName);
}

UCEClonerLayoutBase* UCEClonerComponent::FindOrAddLayout(FName InLayoutName)
{
	if (IsTemplate())
	{
		return nullptr;
	}

	UCEClonerSubsystem* Subsystem = UCEClonerSubsystem::Get();
	if (!Subsystem)
	{
		return nullptr;
	}

	// Check cached layout instances
	UCEClonerLayoutBase* NewActiveLayout = nullptr;
	for (const TObjectPtr<UCEClonerLayoutBase>& LayoutInstance : LayoutInstances)
	{
		if (LayoutInstance && LayoutInstance->GetLayoutName() == InLayoutName)
		{
			NewActiveLayout = LayoutInstance;
			break;
		}
	}

	// Create new layout instance and cache it
	if (!NewActiveLayout)
	{
		NewActiveLayout = Subsystem->CreateNewLayout(InLayoutName, this);

		if (NewActiveLayout)
		{
			LayoutInstances.Add(NewActiveLayout);
		}
	}

	return NewActiveLayout;
}

UCEClonerExtensionBase* UCEClonerComponent::FindOrAddExtension(TSubclassOf<UCEClonerExtensionBase> InClass)
{
	const UCEClonerSubsystem* Subsystem = UCEClonerSubsystem::Get();

	if (!Subsystem)
	{
		return nullptr;
	}

	const FName ExtensionName = Subsystem->FindExtensionName(InClass);

	if (ExtensionName.IsNone())
	{
		return nullptr;
	}

	return FindOrAddExtension(ExtensionName);
}

UCEClonerExtensionBase* UCEClonerComponent::FindOrAddExtension(FName InExtensionName)
{
	// Check cached extension instances
	UCEClonerExtensionBase* NewActiveExtension = nullptr;
	for (TObjectPtr<UCEClonerExtensionBase>& ExtensionInstance : ExtensionInstances)
	{
		if (ExtensionInstance && ExtensionInstance->GetExtensionName() == InExtensionName)
		{
			NewActiveExtension = ExtensionInstance;
			break;
		}
	}

	// Create new extension instance and cache it
	if (!NewActiveExtension)
	{
		UCEClonerSubsystem* Subsystem = UCEClonerSubsystem::Get();
		if (!Subsystem)
		{
			return nullptr;
		}

		NewActiveExtension = Subsystem->CreateNewExtension(InExtensionName, this);
		ExtensionInstances.Add(NewActiveExtension);
	}

	return NewActiveExtension;
}

TArray<FName> UCEClonerComponent::GetClonerLayoutNames() const
{
	TArray<FName> LayoutNames;

	if (const UCEClonerSubsystem* Subsystem = UCEClonerSubsystem::Get())
	{
		LayoutNames = Subsystem->GetLayoutNames().Array();
	}

	return LayoutNames;
}

TArray<FName> UCEClonerComponent::GetClonerTreeBehaviorNames() const
{
	TArray<FName> LayoutNames;

	if (const UCEClonerSubsystem* Subsystem = UCEClonerSubsystem::Get())
	{
		LayoutNames = Subsystem->GetAttachmentTreeBehaviorNames();
	}

	return LayoutNames;
}

void UCEClonerComponent::RefreshClonerMeshes()
{
	if (!bClonerMeshesUpdating && !ClonerTree.bItemAttachmentsDirty)
	{
		UpdateClonerMeshes();
	}
}

UCEClonerExtensionBase* UCEClonerComponent::GetExtension(TSubclassOf<UCEClonerExtensionBase> InExtensionClass) const
{
	const UCEClonerSubsystem* Subsystem = UCEClonerSubsystem::Get();

	if (!Subsystem)
	{
		return nullptr;
	}

	const FName ExtensionName = Subsystem->FindExtensionName(InExtensionClass.Get());

	if (ExtensionName.IsNone())
	{
		return nullptr;
	}

	return GetExtension(ExtensionName);
}

UCEClonerExtensionBase* UCEClonerComponent::GetExtension(FName InExtensionName) const
{
	for (const TObjectPtr<UCEClonerExtensionBase>& ExtensionInstance : ExtensionInstances)
	{
		if (ExtensionInstance && ExtensionInstance->GetExtensionName() == InExtensionName)
		{
			return ExtensionInstance;
		}
	}

	return nullptr;
}

TArray<AActor*> UCEClonerComponent::GetClonerRootActors() const
{
	return ClonerTree.GetRootActors();
}

void UCEClonerComponent::OnActiveLayoutLoaded(UCEClonerLayoutBase* InLayout, bool bInSuccess)
{
	if (!InLayout)
	{
		return;
	}

	InLayout->OnLayoutLoadedDelegate().RemoveAll(this);

	if (!bInSuccess)
	{
		UE_LOGF(LogCECloner, Warning, "%ls : Cloner layout system failed to load %ls - %ls", *GetOwner()->GetActorNameOrLabel(), *InLayout->GetLayoutName().ToString(), *InLayout->GetLayoutAssetPath());
		return;
	}

	UE_LOGF(LogCECloner, Verbose, "%ls : Cloner layout system loaded %ls - %ls", *GetOwner()->GetActorNameOrLabel(), *InLayout->GetLayoutName().ToString(), *InLayout->GetLayoutAssetPath());

	OnClonerLayoutLoadedDelegate.Broadcast(this, InLayout);

	ActivateLayout(InLayout);
}

void UCEClonerComponent::ActivateLayout(UCEClonerLayoutBase* InLayout)
{
	// Must be valid and loaded
	if (!InLayout || !InLayout->IsLayoutLoaded())
	{
		return;
	}

	// Should match current active layout name
	if (LayoutName != InLayout->GetLayoutName())
	{
		return;
	}

	// Deactivate previous layout
	if (ActiveLayout && ActiveLayout->IsLayoutActive())
	{
		ActiveLayout->DeactivateLayout();
	}

	// Activate new layout
	InLayout->ActivateLayout();

	ActiveLayout = InLayout;

	UE_LOGF(LogCECloner, Verbose, "%ls : Cloner layout system changed %ls - %ls", *GetOwner()->GetActorNameOrLabel(), *InLayout->GetLayoutName().ToString(), *InLayout->GetLayoutAssetPath());

	OnActiveLayoutChanged();

	ClonerTree.bItemAttachmentsDirty = true;
}

void UCEClonerComponent::OnActiveLayoutChanged()
{
	UCEClonerLayoutBase* Layout = GetActiveLayout();
	if (!Layout)
	{
		return;
	}

	OnSeedChanged();
	OnColorChanged();

	Layout->MarkLayoutDirty();

	TSet<TObjectPtr<UCEClonerExtensionBase>> PrevActiveExtensions(ActiveExtensions);
	ActiveExtensions.Empty();

	for (const TSubclassOf<UCEClonerExtensionBase>& ExtensionClass : Layout->GetSupportedExtensions())
	{
		if (UCEClonerExtensionBase* Extension = FindOrAddExtension(ExtensionClass.Get()))
		{
			if (!PrevActiveExtensions.Contains(Extension))
			{
				Extension->ActivateExtension();
			}

			Extension->MarkExtensionDirty();

			ActiveExtensions.Add(Extension);
			PrevActiveExtensions.Remove(Extension);
		}
	}

	for (const TObjectPtr<UCEClonerExtensionBase>& InactiveExtension : PrevActiveExtensions)
	{
		InactiveExtension->DeactivateExtension();
	}

	Algo::StableSort(ActiveExtensions, [](const TObjectPtr<UCEClonerExtensionBase>& InExtensionA, const TObjectPtr<UCEClonerExtensionBase>& InExtensionB)
	{
		return InExtensionA->GetExtensionPriority() > InExtensionB->GetExtensionPriority();
	});
}

void UCEClonerComponent::UpdateClonerMeshes()
{
	const AActor* ClonerActor = GetOwner();

	if (!ClonerActor)
	{
		return;
	}

	const UNiagaraDataInterfaceArrayMesh* const MeshArrayDI = UE::ClonerEffector::FindMeshArrayDataInterface(this);
	if (!MeshArrayDI)
	{
		return;
	}

	UNiagaraSystem* ActiveSystem = GetAsset();

	if (!ActiveSystem || !ActiveLayout)
	{
		return;
	}

	if (ActiveLayout->GetSystem() != ActiveSystem)
	{
		UE_LOGF(LogCECloner, Warning, "%ls : Invalid system for cloner layout", *ClonerActor->GetActorNameOrLabel());
		return;
	}

	UNiagaraMeshRendererProperties* MeshRenderer = ActiveLayout->GetMeshRenderer();

	if (!MeshRenderer)
	{
		UE_LOGF(LogCECloner, Warning, "%ls : Invalid mesh renderer for cloner system", *ClonerActor->GetActorNameOrLabel());
		return;
	}

	TArray<FNiagaraMeshRendererMeshPropertiesBase> MeshData = MeshArrayDI->MeshData;
	bool bMeshChanged = MeshData.Num() != ClonerTree.RootItemAttachments.Num();

	if (ClonerTree.bItemAttachmentsDirty)
	{
		// Resize mesh array properly
		MeshData.SetNum(ClonerTree.RootItemAttachments.Num());

		// Set baked meshes in mesh renderer array
		for (TConstEnumerateRef<FCEClonerAttachmentRootItem> RootItem : EnumerateRange(ClonerTree.RootItemAttachments))
		{
			FNiagaraMeshRendererMeshPropertiesBase& MeshProperties = MeshData[RootItem.GetIndex()];

			UStaticMesh* TargetStaticMesh = RootItem->MergedBakedMesh.Get();
			if (TargetStaticMesh && TargetStaticMesh->GetNumTriangles(0) <= 0)
			{
				TargetStaticMesh = nullptr;
			}
			bMeshChanged = bMeshChanged || MeshProperties.Mesh != TargetStaticMesh;
			MeshProperties.Mesh = TargetStaticMesh;

			if (const AActor* RootActor = RootItem->RootActor.Get())
			{
				if (const USceneComponent* SceneComponent = RootActor->GetRootComponent())
				{
					const FRotator TargetRotation = GlobalRotation + SceneComponent->GetRelativeRotation();
					const FVector TargetScale = GlobalScale * SceneComponent->GetRelativeScale3D();

					bMeshChanged = bMeshChanged 
						|| !MeshProperties.Rotation.Equals(TargetRotation)
						|| !MeshProperties.Scale.Equals(TargetScale);

					MeshProperties.Rotation = TargetRotation;
					MeshProperties.Scale = TargetScale;
				}
			}
		}

		ClonerTree.bItemAttachmentsDirty = !ClonerTree.DirtyItemAttachments.IsEmpty();
	}

	for (const TObjectPtr<UCEClonerExtensionBase>& ActiveExtension : ActiveExtensions)
	{
		// Extensions can override mesh renderer meshes array
		bMeshChanged |= ActiveExtension->OnUpdateClonerMeshes(MeshData);
	}

	bMeshChanged = bMeshChanged || ClonerTree.RootItemAttachments.Num() != MeshData.Num();
	if (!bMeshChanged)
	{
		return;
	}

	for (const TObjectPtr<UCEClonerExtensionBase>& ActiveExtension : ActiveExtensions)
	{
		ActiveExtension->OnClonerMeshesUpdated();
	}

	UE_LOGF(LogCECloner, Verbose, "%ls : Cloner mesh updated - %i cached meshes - %i rendered meshes", *ClonerActor->GetActorNameOrLabel(), ClonerTree.RootItemAttachments.Num(), MeshData.Num());

	// Set new number of meshes in renderer
	SetIntParameter(TEXT("MeshNum"), MeshData.Num());

	UNiagaraDataInterfaceArrayMesh::SetNiagaraArrayMesh(this, UE::ClonerEffector::MeshArrayName, MeshData);
	OnClonerMeshUpdatedDelegate.Broadcast(this);
}

void UCEClonerComponent::SetClonerActiveLayout(UCEClonerLayoutBase* InLayout)
{
	if (!InLayout)
	{
		return;
	}

	const AActor* ClonerActor = GetOwner();
	if (!ClonerActor)
	{
		return;
	}

	if (!InLayout->IsLayoutLoaded())
	{
		if (!InLayout->OnLayoutLoadedDelegate().IsBoundToObject(this))
		{
			InLayout->OnLayoutLoadedDelegate().AddUObject(this, &UCEClonerComponent::OnActiveLayoutLoaded);
		}

		InLayout->LoadLayout();

		return;
	}

	ActivateLayout(InLayout);
}

#undef LOCTEXT_NAMESPACE
