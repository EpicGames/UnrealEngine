// Copyright Epic Games, Inc. All Rights Reserved.

#include "Cloner/Extensions/CEClonerCollisionExtension.h"

#include "Cloner/CEClonerComponent.h"
#include "Components/StaticMeshComponent.h"
#include "DataInterface/NiagaraDataInterfaceArrayMesh.h"
#include "Engine/World.h"
#include "Materials/MaterialInstanceDynamic.h"
#include "NiagaraDataInterfaceArrayFloat.h"
#include "NiagaraFunctionLibrary.h"
#include "NiagaraMeshRendererProperties.h"
#include "NiagaraSystem.h"

UCEClonerCollisionExtension::UCEClonerCollisionExtension()
	: UCEClonerExtensionBase(
			TEXT("Collisions")
			, 0
		)
{}

void UCEClonerCollisionExtension::SetSurfaceCollisionEnabled(bool bInSurfaceCollisionEnabled)
{
	if (bSurfaceCollisionEnabled == bInSurfaceCollisionEnabled)
	{
		return;
	}

	bSurfaceCollisionEnabled = bInSurfaceCollisionEnabled;
	MarkExtensionDirty();
}

void UCEClonerCollisionExtension::SetParticleCollisionEnabled(bool bInParticleCollisionEnabled)
{
	if (bParticleCollisionEnabled == bInParticleCollisionEnabled)
	{
		return;
	}

	bParticleCollisionEnabled = bInParticleCollisionEnabled;
	MarkExtensionDirty();
}

void UCEClonerCollisionExtension::SetCollisionVelocityEnabled(bool bInCollisionVelocityEnabled)
{
	if (bCollisionVelocityEnabled == bInCollisionVelocityEnabled)
	{
		return;
	}

	bCollisionVelocityEnabled = bInCollisionVelocityEnabled;
	MarkExtensionDirty();
}

void UCEClonerCollisionExtension::SetCollisionIterations(int32 InCollisionIterations)
{
	InCollisionIterations = FMath::Max(InCollisionIterations, 1);
	if (CollisionIterations == InCollisionIterations)
	{
		return;
	}

	CollisionIterations = InCollisionIterations;
	MarkExtensionDirty();
}

void UCEClonerCollisionExtension::SetCollisionGridResolution(int32 InCollisionGridResolution)
{
	InCollisionGridResolution = FMath::Max(InCollisionGridResolution, 1);
	if (CollisionGridResolution == InCollisionGridResolution)
	{
		return;
	}

	CollisionGridResolution = InCollisionGridResolution;
	MarkExtensionDirty();
}

void UCEClonerCollisionExtension::SetCollisionGridSize(const FVector& InCollisionGridSize)
{
	const FVector NewCollisionGridSize = InCollisionGridSize.ComponentMax(FVector::ZeroVector);
	if (CollisionGridSize.Equals(NewCollisionGridSize))
	{
		return;
	}

	CollisionGridSize = NewCollisionGridSize;
	MarkExtensionDirty();
}

void UCEClonerCollisionExtension::SetCollisionRadiusMode(ECEClonerCollisionRadiusMode InMode)
{
	if (CollisionRadiusMode == InMode)
	{
		return;
	}

	CollisionRadiusMode = InMode;
	MarkExtensionDirty();
}

void UCEClonerCollisionExtension::SetMassMin(float InMassMin)
{
	InMassMin = FMath::Max(InMassMin, 1);
	if (FMath::IsNearlyEqual(MassMin, InMassMin))
	{
		return;
	}

	MassMin = InMassMin;
	MarkExtensionDirty();
}

void UCEClonerCollisionExtension::SetMassMax(float InMassMax)
{
	InMassMax = FMath::Max(InMassMax, 1);
	if (FMath::IsNearlyEqual(MassMax, InMassMax))
	{
		return;
	}

	MassMax = InMassMax;
	MarkExtensionDirty();
}

void UCEClonerCollisionExtension::OnExtensionParametersChanged(UCEClonerComponent* InComponent)
{
	Super::OnExtensionParametersChanged(InComponent);

	MassMin = FMath::Clamp(MassMin, 1, MassMax);
	MassMax = FMath::Max(MassMax, MassMin);

	InComponent->SetBoolParameter(TEXT("SurfaceCollisionEnabled"), bSurfaceCollisionEnabled);
	InComponent->SetIntParameter(TEXT("CollisionIterations"), bParticleCollisionEnabled ? CollisionIterations : 0);
	InComponent->SetBoolParameter(TEXT("CollisionVelocityEnabled"), bParticleCollisionEnabled ? bCollisionVelocityEnabled : false);
	InComponent->SetIntParameter(TEXT("CollisionGridResolution"), CollisionGridResolution);
	InComponent->SetVectorParameter(TEXT("CollisionGridSize"), CollisionGridSize);
	InComponent->SetFloatParameter(TEXT("MassMin"), MassMin);
	InComponent->SetFloatParameter(TEXT("MassMax"), MassMax);

	// Adjust size based on attached mesh count
	CollisionRadii.SetNum(InComponent->GetMeshCount());

	if (const UNiagaraDataInterfaceArrayMesh* const MeshArrayDI = UE::ClonerEffector::FindMeshArrayDataInterface(InComponent))
	{
		if (CollisionRadiusMode != ECEClonerCollisionRadiusMode::Manual)
		{
			for (int32 Idx = 0; Idx < CollisionRadii.Num(); Idx++)
			{
				const FNiagaraMeshRendererMeshPropertiesBase& MeshProperties = MeshArrayDI->MeshData[Idx];
				FBoxSphereBounds MeshBounds(ForceInitToZero);
				const FTransform BoundTransform(MeshProperties.Rotation, MeshProperties.PivotOffset, MeshProperties.Scale);

				if (MeshProperties.Mesh)
				{
					MeshBounds = MeshProperties.Mesh->GetBounds().TransformBy(BoundTransform);
				}

				switch (CollisionRadiusMode)
				{
					case ECEClonerCollisionRadiusMode::MinExtent:
						CollisionRadii[Idx] = MeshBounds.BoxExtent.GetMin();
					break;
					case ECEClonerCollisionRadiusMode::MaxExtent:
						CollisionRadii[Idx] = MeshBounds.BoxExtent.GetMax();
					break;
					default:
					case ECEClonerCollisionRadiusMode::ExtentLength:
						CollisionRadii[Idx] = MeshBounds.SphereRadius;
					break;
				}
			}
		}

		const FNiagaraUserRedirectionParameterStore& ExposedParameters = InComponent->GetOverrideParameters();

		static const FNiagaraVariable CollisionRadiiVar(FNiagaraTypeDefinition(UNiagaraDataInterfaceArrayFloat::StaticClass()), TEXT("CollisionRadii"));

		if (UNiagaraDataInterfaceArrayFloat* CollisionRadiiDI = Cast<UNiagaraDataInterfaceArrayFloat>(ExposedParameters.GetDataInterface(CollisionRadiiVar)))
		{
			CollisionRadiiDI->GetArrayReference() = CollisionRadii;
		}
	}

#if WITH_EDITOR
	if (ShouldInitVisualizerComponent(InComponent) && InitVisualizerComponent())
	{
		CollisionVisualizerComponent->SetVisibility(bParticleCollisionEnabled && bPreviewCollisionGrid);
		// Divide by 100 because cube size is 100x100x100
		CollisionVisualizerComponent->SetWorldScale3D(CollisionGridSize / 100.f);
		CollisionVisualizerMaterial->SetScalarParameterValue(TEXT("GridTile"), CollisionGridResolution);
	}
#endif
}

void UCEClonerCollisionExtension::OnClonerMeshesUpdated()
{
	Super::OnClonerMeshesUpdated();

	MarkExtensionDirty();
}

void UCEClonerCollisionExtension::OnExtensionDeactivated()
{
	Super::OnExtensionDeactivated();

#if WITH_EDITOR
	DestroyVisualizerComponent();
#endif
}

#if WITH_EDITOR

bool UCEClonerCollisionExtension::ShouldInitVisualizerComponent(TNotNull<const UCEClonerComponent*> InComponent) const
{
	const UWorld* World = InComponent->GetWorld();
	return World && (World->WorldType == EWorldType::Editor || World->WorldType == EWorldType::EditorPreview);
}

bool UCEClonerCollisionExtension::InitVisualizerComponent()
{
	if (CollisionVisualizerComponent)
	{
		return !!CollisionVisualizerMaterial;
	}

	UCEClonerComponent* ClonerComponent = GetClonerComponent();

	if (!ClonerComponent)
	{
		return false;
	}

	AActor* ClonerActor = ClonerComponent->GetOwner();

	if (!ClonerActor)
	{
		return false;
	}

	CollisionVisualizerComponent = NewObject<UStaticMeshComponent>(
		ClonerActor
		, UStaticMeshComponent::StaticClass()
		, MakeUniqueObjectName(ClonerActor, UStaticMeshComponent::StaticClass(), TEXT("ClonerCollisionVisualizerComponent"))
		, RF_Transient);
	
	CollisionVisualizerComponent->OnComponentCreated();
	CollisionVisualizerComponent->SetupAttachment(ClonerComponent);
	CollisionVisualizerComponent->RegisterComponent();
	
	CollisionVisualizerComponent->SetIsVisualizationComponent(true);
	CollisionVisualizerComponent->SetHiddenInGame(true);
	CollisionVisualizerComponent->SetCastShadow(false);
	CollisionVisualizerComponent->SetCollisionEnabled(ECollisionEnabled::Type::NoCollision);

	// Setup Mesh
	if (UStaticMesh* CubeMesh = LoadObject<UStaticMesh>(nullptr, TEXT("/Script/Engine.StaticMesh'/Engine/BasicShapes/Cube.Cube'")))
	{
		CollisionVisualizerComponent->SetStaticMesh(CubeMesh);
	}

	// Setup Material
	if (UMaterialInterface* VisualizerMaterial = LoadObject<UMaterialInterface>(nullptr, TEXT("/Script/Engine.Material'/ClonerEffector/Materials/M_ClonerGrid.M_ClonerGrid'")))
	{
		CollisionVisualizerMaterial = UMaterialInstanceDynamic::Create(
			VisualizerMaterial,
			CollisionVisualizerComponent
		);
		
		CollisionVisualizerComponent->SetMaterial(0, CollisionVisualizerMaterial);
	}

	return !!CollisionVisualizerMaterial;
}

void UCEClonerCollisionExtension::DestroyVisualizerComponent()
{
	if (!CollisionVisualizerComponent)
	{
		return;
	}

	CollisionVisualizerComponent->DestroyComponent();
	CollisionVisualizerComponent = nullptr;

	CollisionVisualizerMaterial->MarkAsGarbage();
	CollisionVisualizerMaterial = nullptr;
}

const TCEPropertyChangeDispatcher<UCEClonerCollisionExtension> UCEClonerCollisionExtension::PropertyChangeDispatcher =
{
	/** Collision */
	{ GET_MEMBER_NAME_CHECKED(UCEClonerCollisionExtension, bSurfaceCollisionEnabled), &UCEClonerCollisionExtension::OnExtensionPropertyChanged },
	{ GET_MEMBER_NAME_CHECKED(UCEClonerCollisionExtension, bParticleCollisionEnabled), &UCEClonerCollisionExtension::OnExtensionPropertyChanged },
	{ GET_MEMBER_NAME_CHECKED(UCEClonerCollisionExtension, bCollisionVelocityEnabled), &UCEClonerCollisionExtension::OnExtensionPropertyChanged },
	{ GET_MEMBER_NAME_CHECKED(UCEClonerCollisionExtension, CollisionRadiusMode), &UCEClonerCollisionExtension::OnExtensionPropertyChanged },
	{ GET_MEMBER_NAME_CHECKED(UCEClonerCollisionExtension, CollisionRadii), &UCEClonerCollisionExtension::OnExtensionPropertyChanged },
	{ GET_MEMBER_NAME_CHECKED(UCEClonerCollisionExtension, CollisionIterations), &UCEClonerCollisionExtension::OnExtensionPropertyChanged },
	{ GET_MEMBER_NAME_CHECKED(UCEClonerCollisionExtension, CollisionGridResolution), &UCEClonerCollisionExtension::OnExtensionPropertyChanged },
	{ GET_MEMBER_NAME_CHECKED(UCEClonerCollisionExtension, CollisionGridSize), &UCEClonerCollisionExtension::OnExtensionPropertyChanged },
	{ GET_MEMBER_NAME_CHECKED(UCEClonerCollisionExtension, MassMin), &UCEClonerCollisionExtension::OnExtensionPropertyChanged },
	{ GET_MEMBER_NAME_CHECKED(UCEClonerCollisionExtension, MassMax), &UCEClonerCollisionExtension::OnExtensionPropertyChanged },
	{ GET_MEMBER_NAME_CHECKED(UCEClonerCollisionExtension, bPreviewCollisionGrid), &UCEClonerCollisionExtension::OnExtensionPropertyChanged },
};

void UCEClonerCollisionExtension::PostEditChangeProperty(FPropertyChangedEvent& InPropertyChangedEvent)
{
	Super::PostEditChangeProperty(InPropertyChangedEvent);

	PropertyChangeDispatcher.OnPropertyChanged(this, InPropertyChangedEvent);
}
#endif
