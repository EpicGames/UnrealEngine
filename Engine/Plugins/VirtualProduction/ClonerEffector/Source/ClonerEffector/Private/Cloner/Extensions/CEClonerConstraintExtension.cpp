// Copyright Epic Games, Inc. All Rights Reserved.

#include "Cloner/Extensions/CEClonerConstraintExtension.h"

#include "Cloner/CEClonerComponent.h"
#include "Cloner/Layouts/CEClonerGridLayout.h"
#include "NiagaraDataInterfaceTexture.h"
#include "NiagaraSystem.h"

UCEClonerConstraintExtension::UCEClonerConstraintExtension()
	: UCEClonerExtensionBase(
		TEXT("Constraint")
		, 0
	)
{}

void UCEClonerConstraintExtension::SetConstraint(ECEClonerGridConstraint InConstraint)
{
	if (Constraint == InConstraint)
	{
		return;
	}

	Constraint = InConstraint;
	MarkExtensionDirty();
}

void UCEClonerConstraintExtension::SetInvertConstraint(bool bInInvertConstraint)
{
	if (bInvertConstraint == bInInvertConstraint)
	{
		return;
	}

	bInvertConstraint = bInInvertConstraint;
	MarkExtensionDirty();
}

void UCEClonerConstraintExtension::SetSphereRadius(float InSphereRadius)
{
	if (FMath::IsNearlyEqual(SphereRadius, InSphereRadius))
	{
		return;
	}

	SphereRadius = InSphereRadius;
	MarkExtensionDirty();
}

void UCEClonerConstraintExtension::SetSphereCenter(const FVector& InSphereCenter)
{
	if (SphereCenter.Equals(InSphereCenter))
	{
		return;
	}

	SphereCenter = InSphereCenter;
	MarkExtensionDirty();
}

void UCEClonerConstraintExtension::SetCylinderRadius(float InCylinderRadius)
{
	if (FMath::IsNearlyEqual(CylinderRadius, InCylinderRadius))
	{
		return;
	}

	CylinderRadius = InCylinderRadius;
	MarkExtensionDirty();
}

void UCEClonerConstraintExtension::SetCylinderHeight(float InCylinderHeight)
{
	if (FMath::IsNearlyEqual(CylinderHeight, InCylinderHeight))
	{
		return;
	}

	CylinderHeight = InCylinderHeight;
	MarkExtensionDirty();
}

void UCEClonerConstraintExtension::SetCylinderCenter(const FVector& InCylinderCenter)
{
	if (CylinderCenter.Equals(InCylinderCenter))
	{
		return;
	}

	CylinderCenter = InCylinderCenter;
	MarkExtensionDirty();
}

void UCEClonerConstraintExtension::SetTextureAsset(UTexture* InTexture)
{
	if (TextureAsset == InTexture)
	{
		return;
	}

	TextureAsset = InTexture;
	MarkExtensionDirty();
}

void UCEClonerConstraintExtension::SetTextureSampleMode(ECEClonerTextureSampleChannel InSampleMode)
{
	if (TextureSampleMode == InSampleMode)
	{
		return;
	}

	TextureSampleMode = InSampleMode;
	MarkExtensionDirty();
}

void UCEClonerConstraintExtension::SetTexturePlane(ECEClonerPlane InPlane)
{
	if (TexturePlane == InPlane)
	{
		return;
	}

	TexturePlane = InPlane;
	MarkExtensionDirty();
}

void UCEClonerConstraintExtension::SetTextureCompareMode(ECEClonerCompareMode InMode)
{
	if (TextureCompareMode == InMode)
	{
		return;
	}

	TextureCompareMode = InMode;
	MarkExtensionDirty();
}

void UCEClonerConstraintExtension::SetTextureThreshold(float InThreshold)
{
	InThreshold = FMath::Max(0, InThreshold);
	if (FMath::IsNearlyEqual(TextureThreshold, InThreshold))
	{
		return;
	}

	TextureThreshold = InThreshold;
	MarkExtensionDirty();
}

void UCEClonerConstraintExtension::SetTextureOffset(const FVector2D& InOffset)
{
	if (TextureOffset.Equals(InOffset))
	{
		return;
	}

	TextureOffset = InOffset;
	MarkExtensionDirty();
}

void UCEClonerConstraintExtension::SetTextureRotation(float InRotation)
{
	if (FMath::IsNearlyEqual(TextureRotation, InRotation))
	{
		return;
	}

	TextureRotation = InRotation;
	MarkExtensionDirty();
}

void UCEClonerConstraintExtension::SetTextureScale(const FVector2D& InScale)
{
	if (TextureScale.Equals(InScale))
	{
		return;
	}

	TextureScale = InScale;
	MarkExtensionDirty();
}

void UCEClonerConstraintExtension::SetTextureClamp(bool bInClamp)
{
	if (bTextureClamp == bInClamp)
	{
		return;
	}

	bTextureClamp = bInClamp;
	MarkExtensionDirty();
}

#if WITH_EDITOR
const TCEPropertyChangeDispatcher<UCEClonerConstraintExtension> UCEClonerConstraintExtension::PropertyChangeDispatcher =
{
	{ GET_MEMBER_NAME_CHECKED(UCEClonerConstraintExtension, Constraint), &UCEClonerConstraintExtension::OnExtensionPropertyChanged },
	{ GET_MEMBER_NAME_CHECKED(UCEClonerConstraintExtension, bInvertConstraint), &UCEClonerConstraintExtension::OnExtensionPropertyChanged },
	{ GET_MEMBER_NAME_CHECKED(UCEClonerConstraintExtension, SphereRadius), &UCEClonerConstraintExtension::OnExtensionPropertyChanged },
	{ GET_MEMBER_NAME_CHECKED(UCEClonerConstraintExtension, SphereCenter), &UCEClonerConstraintExtension::OnExtensionPropertyChanged },
	{ GET_MEMBER_NAME_CHECKED(UCEClonerConstraintExtension, CylinderRadius), &UCEClonerConstraintExtension::OnExtensionPropertyChanged },
	{ GET_MEMBER_NAME_CHECKED(UCEClonerConstraintExtension, CylinderHeight), &UCEClonerConstraintExtension::OnExtensionPropertyChanged },
	{ GET_MEMBER_NAME_CHECKED(UCEClonerConstraintExtension, CylinderCenter), &UCEClonerConstraintExtension::OnExtensionPropertyChanged },
	{ GET_MEMBER_NAME_CHECKED(UCEClonerConstraintExtension, TextureAsset), &UCEClonerConstraintExtension::OnExtensionPropertyChanged },
	{ GET_MEMBER_NAME_CHECKED(UCEClonerConstraintExtension, TextureSampleMode), &UCEClonerConstraintExtension::OnExtensionPropertyChanged },
	{ GET_MEMBER_NAME_CHECKED(UCEClonerConstraintExtension, TexturePlane), &UCEClonerConstraintExtension::OnExtensionPropertyChanged },
	{ GET_MEMBER_NAME_CHECKED(UCEClonerConstraintExtension, TextureCompareMode), &UCEClonerConstraintExtension::OnExtensionPropertyChanged },
	{ GET_MEMBER_NAME_CHECKED(UCEClonerConstraintExtension, TextureThreshold), &UCEClonerConstraintExtension::OnExtensionPropertyChanged },
	{ GET_MEMBER_NAME_CHECKED(UCEClonerConstraintExtension, TextureOffset), &UCEClonerConstraintExtension::OnExtensionPropertyChanged },
	{ GET_MEMBER_NAME_CHECKED(UCEClonerConstraintExtension, TextureRotation), &UCEClonerConstraintExtension::OnExtensionPropertyChanged },
	{ GET_MEMBER_NAME_CHECKED(UCEClonerConstraintExtension, TextureScale), &UCEClonerConstraintExtension::OnExtensionPropertyChanged },
	{ GET_MEMBER_NAME_CHECKED(UCEClonerConstraintExtension, bTextureClamp), &UCEClonerConstraintExtension::OnExtensionPropertyChanged },
};

void UCEClonerConstraintExtension::PostEditChangeProperty(FPropertyChangedEvent& InPropertyChangedEvent)
{
	Super::PostEditChangeProperty(InPropertyChangedEvent);

	PropertyChangeDispatcher.OnPropertyChanged(this, InPropertyChangedEvent);
}
#endif

void UCEClonerConstraintExtension::OnExtensionParametersChanged(UCEClonerComponent* InComponent)
{
	Super::OnExtensionParametersChanged(InComponent);

	FNiagaraUserRedirectionParameterStore& ExposedParameters = InComponent->GetOverrideParameters();
	static const FNiagaraVariable ConstraintVar(FNiagaraTypeDefinition(StaticEnum<ECEClonerGridConstraint>()), TEXT("Constraint"));
	ExposedParameters.SetParameterValue<int32>(static_cast<int32>(Constraint), ConstraintVar);

	InComponent->SetBoolParameter(TEXT("ConstraintInvert"), Constraint != ECEClonerGridConstraint::None ? bInvertConstraint : false);

	// Sphere
	InComponent->SetVectorParameter(TEXT("ConstraintSphereCenter"), SphereCenter);

	InComponent->SetFloatParameter(TEXT("ConstraintSphereRadius"), SphereRadius);

	// Cylinder
	InComponent->SetVectorParameter(TEXT("ConstraintCylinderCenter"), CylinderCenter);

	InComponent->SetFloatParameter(TEXT("ConstraintCylinderHeight"), CylinderHeight);

	InComponent->SetFloatParameter(TEXT("ConstraintCylinderRadius"), CylinderRadius);

	// Texture
	static const FNiagaraVariable ConstraintTextureSamplerVar(FNiagaraTypeDefinition(UNiagaraDataInterfaceTexture::StaticClass()), TEXT("ConstraintTextureSampler"));
	if (UNiagaraDataInterfaceTexture* TextureSamplerDI = Cast<UNiagaraDataInterfaceTexture>(ExposedParameters.GetDataInterface(ConstraintTextureSamplerVar)))
	{
		TextureSamplerDI->SetTexture(TextureAsset.Get());
	}

	static const FNiagaraVariable ConstraintTexturePlaneVar(FNiagaraTypeDefinition(StaticEnum<ECEClonerPlane>()), TEXT("ConstraintTexturePlane"));
	ExposedParameters.SetParameterValue<int32>(static_cast<int32>(TexturePlane), ConstraintTexturePlaneVar);

	static const FNiagaraVariable ConstraintTextureChannelVar(FNiagaraTypeDefinition(StaticEnum<ECEClonerTextureSampleChannel>()), TEXT("ConstraintTextureChannel"));
	ExposedParameters.SetParameterValue<int32>(static_cast<int32>(TextureSampleMode), ConstraintTextureChannelVar);

	static const FNiagaraVariable ConstraintTextureCompareModeVar(FNiagaraTypeDefinition(StaticEnum<ECEClonerCompareMode>()), TEXT("ConstraintTextureCompareMode"));
	ExposedParameters.SetParameterValue<int32>(static_cast<int32>(TextureCompareMode), ConstraintTextureCompareModeVar);
	
	InComponent->SetFloatParameter(TEXT("ConstraintTextureThreshold"), FMath::Max(0, TextureThreshold));

	InComponent->SetVariableVec2(TEXT("ConstraintTextureOffset"), TextureOffset);

	InComponent->SetFloatParameter(TEXT("ConstraintTextureRotation"), TextureRotation);

	InComponent->SetVariableVec2(TEXT("ConstraintTextureScale"), TextureScale);

	InComponent->SetVariableBool(TEXT("ConstraintTextureClamp"), bTextureClamp);
}

bool UCEClonerConstraintExtension::IsLayoutSupported(const UCEClonerLayoutBase* InLayout) const
{
	return InLayout->IsA<UCEClonerGridLayout>();
}
