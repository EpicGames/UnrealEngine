// Copyright Epic Games, Inc. All Rights Reserved.

#include "Cloner/Extensions/CEClonerDisplacementExtension.h"

#include "Cloner/CEClonerComponent.h"
#include "Cloner/Layouts/CEClonerGridLayout.h"
#include "NiagaraDataInterfaceTexture.h"

UCEClonerDisplacementExtension::UCEClonerDisplacementExtension()
	: UCEClonerExtensionBase(
		TEXT("Displacement")
		, 0
	)
{}

void UCEClonerDisplacementExtension::SetDisplacementEnabled(bool bInEnabled)
{
	if (bDisplacementEnabled == bInEnabled)
	{
		return;
	}

	bDisplacementEnabled = bInEnabled;
	MarkExtensionDirty();
}

void UCEClonerDisplacementExtension::SetDisplacementInvert(bool bInInvert)
{
	if (bDisplacementInvert == bInInvert)
	{
		return;
	}

	bDisplacementInvert = bInInvert;
	MarkExtensionDirty();
}

void UCEClonerDisplacementExtension::SetDisplacementOffsetMax(const FVector& InMax)
{
	if (DisplacementOffsetMax.Equals(InMax))
	{
		return;
	}

	DisplacementOffsetMax = InMax;
	MarkExtensionDirty();
}

void UCEClonerDisplacementExtension::SetDisplacementRotationMax(const FRotator& InMax)
{
	if (DisplacementRotationMax.Equals(InMax))
	{
		return;
	}

	DisplacementRotationMax = InMax;
	MarkExtensionDirty();
}

void UCEClonerDisplacementExtension::SetDisplacementScaleMax(const FVector& InMax)
{
	if (DisplacementScaleMax.Equals(InMax))
	{
		return;
	}

	DisplacementScaleMax = InMax.ComponentMax(FVector(UE_KINDA_SMALL_NUMBER));
	MarkExtensionDirty();
}

void UCEClonerDisplacementExtension::SetDisplacementTextureAsset(UTexture* InTexture)
{
	if (DisplacementTextureAsset == InTexture)
	{
		return;
	}

	DisplacementTextureAsset = InTexture;
	MarkExtensionDirty();
}

void UCEClonerDisplacementExtension::SetDisplacementTextureSampleMode(ECEClonerTextureSampleChannel InSampleMode)
{
	if (DisplacementTextureSampleMode == InSampleMode)
	{
		return;
	}

	DisplacementTextureSampleMode = InSampleMode;
	MarkExtensionDirty();
}

void UCEClonerDisplacementExtension::SetDisplacementTexturePlane(ECEClonerPlane InPlane)
{
	if (DisplacementTexturePlane == InPlane)
	{
		return;
	}

	DisplacementTexturePlane = InPlane;
	MarkExtensionDirty();
}

void UCEClonerDisplacementExtension::SetDisplacementTextureOffset(const FVector2D& InOffset)
{
	if (DisplacementTextureOffset.Equals(InOffset))
	{
		return;
	}

	DisplacementTextureOffset = InOffset;
	MarkExtensionDirty();
}

void UCEClonerDisplacementExtension::SetDisplacementTextureRotation(float InRotation)
{
	if (FMath::IsNearlyEqual(DisplacementTextureRotation, InRotation))
	{
		return;
	}

	DisplacementTextureRotation = InRotation;
	MarkExtensionDirty();
}

void UCEClonerDisplacementExtension::SetDisplacementTextureScale(const FVector2D& InScale)
{
	if (DisplacementTextureScale.Equals(InScale))
	{
		return;
	}

	DisplacementTextureScale = InScale;
	MarkExtensionDirty();
}

void UCEClonerDisplacementExtension::SetDisplacementTextureClamp(bool bInClamp)
{
	if (bDisplacementTextureClamp == bInClamp)
	{
		return;
	}

	bDisplacementTextureClamp = bInClamp;
	MarkExtensionDirty();
}

#if WITH_EDITOR
const TCEPropertyChangeDispatcher<UCEClonerDisplacementExtension> UCEClonerDisplacementExtension::PropertyChangeDispatcher =
{
	{ GET_MEMBER_NAME_CHECKED(UCEClonerDisplacementExtension, bDisplacementEnabled), &UCEClonerDisplacementExtension::OnExtensionPropertyChanged },
	{ GET_MEMBER_NAME_CHECKED(UCEClonerDisplacementExtension, bDisplacementInvert), &UCEClonerDisplacementExtension::OnExtensionPropertyChanged },
	{ GET_MEMBER_NAME_CHECKED(UCEClonerDisplacementExtension, DisplacementOffsetMax), &UCEClonerDisplacementExtension::OnExtensionPropertyChanged },
	{ GET_MEMBER_NAME_CHECKED(UCEClonerDisplacementExtension, DisplacementRotationMax), &UCEClonerDisplacementExtension::OnExtensionPropertyChanged },
	{ GET_MEMBER_NAME_CHECKED(UCEClonerDisplacementExtension, DisplacementScaleMax), &UCEClonerDisplacementExtension::OnExtensionPropertyChanged },
	{ GET_MEMBER_NAME_CHECKED(UCEClonerDisplacementExtension, DisplacementTextureAsset), &UCEClonerDisplacementExtension::OnExtensionPropertyChanged },
	{ GET_MEMBER_NAME_CHECKED(UCEClonerDisplacementExtension, DisplacementTextureSampleMode), &UCEClonerDisplacementExtension::OnExtensionPropertyChanged },
	{ GET_MEMBER_NAME_CHECKED(UCEClonerDisplacementExtension, DisplacementTexturePlane), &UCEClonerDisplacementExtension::OnExtensionPropertyChanged },
	{ GET_MEMBER_NAME_CHECKED(UCEClonerDisplacementExtension, DisplacementTextureOffset), &UCEClonerDisplacementExtension::OnExtensionPropertyChanged },
	{ GET_MEMBER_NAME_CHECKED(UCEClonerDisplacementExtension, DisplacementTextureRotation), &UCEClonerDisplacementExtension::OnExtensionPropertyChanged },
	{ GET_MEMBER_NAME_CHECKED(UCEClonerDisplacementExtension, DisplacementTextureScale), &UCEClonerDisplacementExtension::OnExtensionPropertyChanged },
	{ GET_MEMBER_NAME_CHECKED(UCEClonerDisplacementExtension, bDisplacementTextureClamp), &UCEClonerDisplacementExtension::OnExtensionPropertyChanged },
};

void UCEClonerDisplacementExtension::PostEditChangeProperty(FPropertyChangedEvent& InPropertyChangedEvent)
{
	Super::PostEditChangeProperty(InPropertyChangedEvent);

	PropertyChangeDispatcher.OnPropertyChanged(this, InPropertyChangedEvent);
}
#endif

void UCEClonerDisplacementExtension::OnExtensionParametersChanged(UCEClonerComponent* InComponent)
{
	Super::OnExtensionParametersChanged(InComponent);

	FNiagaraUserRedirectionParameterStore& ExposedParameters = InComponent->GetOverrideParameters();

	InComponent->SetBoolParameter(TEXT("DisplacementEnabled"), bDisplacementEnabled);

	InComponent->SetBoolParameter(TEXT("DisplacementInvert"), bDisplacementInvert);

	InComponent->SetVectorParameter(TEXT("DisplacementOffsetMax"), DisplacementOffsetMax);
	
	InComponent->SetVectorParameter(TEXT("DisplacementRotationMax"), FVector(DisplacementRotationMax.Yaw, DisplacementRotationMax.Pitch, DisplacementRotationMax.Roll));
	
	InComponent->SetVectorParameter(TEXT("DisplacementScaleMax"), DisplacementScaleMax.ComponentMax(FVector(UE_KINDA_SMALL_NUMBER)));

	static const FNiagaraVariable TextureVar(FNiagaraTypeDefinition(UNiagaraDataInterfaceTexture::StaticClass()), TEXT("DisplacementTexture"));
	if (UNiagaraDataInterfaceTexture* TextureSamplerDI = Cast<UNiagaraDataInterfaceTexture>(ExposedParameters.GetDataInterface(TextureVar)))
	{
		TextureSamplerDI->SetTexture(DisplacementTextureAsset.Get());
	}

	static const FNiagaraVariable TexturePlaneVar(FNiagaraTypeDefinition(StaticEnum<ECEClonerPlane>()), TEXT("DisplacementTexturePlane"));
	ExposedParameters.SetParameterValue<int32>(static_cast<int32>(DisplacementTexturePlane), TexturePlaneVar);

	static const FNiagaraVariable TextureChannelVar(FNiagaraTypeDefinition(StaticEnum<ECEClonerTextureSampleChannel>()), TEXT("DisplacementTextureChannel"));
	ExposedParameters.SetParameterValue<int32>(static_cast<int32>(DisplacementTextureSampleMode), TextureChannelVar);

	InComponent->SetVariableVec2(TEXT("DisplacementTextureOffset"), DisplacementTextureOffset);

	InComponent->SetFloatParameter(TEXT("DisplacementTextureRotation"), DisplacementTextureRotation);

	InComponent->SetVariableVec2(TEXT("DisplacementTextureScale"), DisplacementTextureScale);

	InComponent->SetVariableBool(TEXT("DisplacementTextureClamp"), bDisplacementTextureClamp);
}

bool UCEClonerDisplacementExtension::IsLayoutSupported(const UCEClonerLayoutBase* InLayout) const
{
	return InLayout->IsA<UCEClonerGridLayout>();
}