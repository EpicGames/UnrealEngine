// Copyright Epic Games, Inc. All Rights Reserved.

#include "Cloner/Extensions/CEClonerEffectorExtension.h"

#include "Cloner/CEClonerComponent.h"
#include "Cloner/Logs/CEClonerLogs.h"
#include "Effector/CEEffectorComponent.h"
#include "NiagaraDataInterfaceArrayInt.h"
#include "Subsystems/CEClonerSubsystem.h"
#include "Subsystems/CEEffectorSubsystem.h"

#if WITH_EDITOR
#include "Editor.h"
#include "Editor/EditorEngine.h"
#endif

#if WITH_EDITOR
FName UCEClonerEffectorExtension::GetEffectorActorsWeakName()
{
	return GET_MEMBER_NAME_CHECKED(UCEClonerEffectorExtension, EffectorActorsWeak);
}
#endif

UCEClonerEffectorExtension::UCEClonerEffectorExtension()
	: UCEClonerExtensionBase(
		TEXT("Effector")
		, 0
	)
{}

bool UCEClonerEffectorExtension::LinkEffector(AActor* InEffectorActor)
{
	if (!IsValid(InEffectorActor)
		|| EffectorActorsWeak.Contains(InEffectorActor)
		|| !InEffectorActor->FindComponentByClass<UCEEffectorComponent>())
	{
		return false;
	}

	EffectorActorsWeak.Add(InEffectorActor);

	OnEffectorsChanged();

	UE_LOGF(LogCECloner, Verbose, "%ls : Effector %ls linked to Cloner", *GetClonerComponent()->GetOwner()->GetActorNameOrLabel(), *InEffectorActor->GetActorNameOrLabel());

	return true;
}

bool UCEClonerEffectorExtension::UnlinkEffector(AActor* InEffectorActor)
{
	if (!InEffectorActor)
	{
		return false;
	}

	if (EffectorActorsWeak.Remove(InEffectorActor) > 0)
	{
		OnEffectorsChanged();

		UE_LOGF(LogCECloner, Verbose, "%ls : Effector %ls unlinked from Cloner", *GetClonerComponent()->GetOwner()->GetActorNameOrLabel(), *InEffectorActor->GetActorNameOrLabel());
	}

	return true;
}

bool UCEClonerEffectorExtension::IsEffectorLinked(const AActor* InEffectorActor) const
{
	return InEffectorActor && EffectorActorsWeak.Contains(InEffectorActor);
}

int32 UCEClonerEffectorExtension::GetEffectorCount() const
{
	return EffectorActorsWeak.Num();
}

void UCEClonerEffectorExtension::OnExtensionActivated()
{
	Super::OnExtensionActivated();

	UCEEffectorSubsystem::OnEffectorIdentifierChanged().RemoveAll(this);
	UCEEffectorSubsystem::OnEffectorIdentifierChanged().AddUObject(this, &UCEClonerEffectorExtension::OnEffectorIdentifierChanged);
}

void UCEClonerEffectorExtension::OnExtensionDeactivated()
{
	Super::OnExtensionDeactivated();

	UCEEffectorSubsystem::OnEffectorIdentifierChanged().RemoveAll(this);
}

void UCEClonerEffectorExtension::OnExtensionParametersChanged(UCEClonerComponent* InComponent)
{
	Super::OnExtensionParametersChanged(InComponent);

	OnEffectorsChanged();
}

void UCEClonerEffectorExtension::OnEffectorIdentifierChanged(UCEEffectorComponent* InEffector, int32 InOldIdentifier, int32 InNewIdentifier)
{
	if (EffectorActorsWeak.Contains(InEffector->GetOwner()))
	{
		OnEffectorsChanged();

		if (InOldIdentifier != INDEX_NONE && InNewIdentifier == INDEX_NONE)
		{
#if WITH_EDITOR
			if (GUndo)
			{
				Modify();
			}
#endif

			EffectorActorsWeak.Remove(InEffector->GetOwner());
		}
	}
}

void UCEClonerEffectorExtension::OnEffectorsChanged()
{
	UNiagaraComponent* Component = GetClonerComponent();

	if (!Component)
	{
		return;
	}

	const FNiagaraUserRedirectionParameterStore& ExposedParameters = Component->GetOverrideParameters();

	static const FNiagaraVariable EffectorIndexDIVar(FNiagaraTypeDefinition(UNiagaraDataInterfaceArrayInt32::StaticClass()), TEXT("EffectorIndexArray"));
	UNiagaraDataInterfaceArrayInt32* EffectorIndexArrayDI = Cast<UNiagaraDataInterfaceArrayInt32>(ExposedParameters.GetDataInterface(EffectorIndexDIVar));

	if (!EffectorIndexArrayDI)
	{
		return;
	}

	// Remove duplicates
	TSet<TWeakObjectPtr<UCEEffectorComponent>> SetEffectorsWeak;
	SetEffectorsWeak.Reserve(EffectorActorsWeak.Num());

	for (TArray<TWeakObjectPtr<AActor>>::TIterator It(EffectorActorsWeak); It; ++It)
	{
		if (const AActor* EffectorActor = It->Get())
		{
			TArray<UCEEffectorComponent*> EffectorComponents;
			EffectorActor->GetComponents(EffectorComponents, /** Recurse */false);

			for (UCEEffectorComponent* EffectorComponent : EffectorComponents)
			{
				if (EffectorComponent->GetChannelIdentifier() != INDEX_NONE)
				{
					SetEffectorsWeak.Add(EffectorComponent);
				}
			}
		}
	}

	// Removed effectors
	for (const TWeakObjectPtr<UCEEffectorComponent>& EffectorWeak : EffectorsInternalWeak.Difference(SetEffectorsWeak))
	{
		if (UCEEffectorComponent* Effector = EffectorWeak.Get())
		{
			Effector->OnClonerUnlinked(this);
		}
	}

	// Added effectors
	for (const TWeakObjectPtr<UCEEffectorComponent>& EffectorWeak : SetEffectorsWeak.Difference(EffectorsInternalWeak))
	{
		if (UCEEffectorComponent* Effector = EffectorWeak.Get())
		{
			Effector->OnClonerLinked(this);
		}
	}

	EffectorsInternalWeak = SetEffectorsWeak;

	TArray<int32> EffectorIndexes;
	EffectorIndexes.Reserve(SetEffectorsWeak.Num());

	for (const TWeakObjectPtr<UCEEffectorComponent>& EffectorWeak : SetEffectorsWeak)
	{
		if (const UCEEffectorComponent* Effector = EffectorWeak.Get())
		{
			const int32 ChannelIdentifier = Effector->GetChannelIdentifier();

			if (ChannelIdentifier != INDEX_NONE)
			{
				EffectorIndexes.AddUnique(ChannelIdentifier);
			}
		}
	}

	TArray<int32>& EffectorIndexArray = EffectorIndexArrayDI->GetArrayReference();
	EffectorIndexArray.Empty(EffectorIndexes.Num());

	for (const int32 EffectorIndex : EffectorIndexes)
	{
		EffectorIndexArray.Add(EffectorIndex);
	}

	// Apply changes
	MarkExtensionDirty();
}

void UCEClonerEffectorExtension::OnEffectorActorsChanged()
{
	for (TArray<TWeakObjectPtr<AActor>>::TIterator It(EffectorActorsWeak); It; ++It)
	{
		if (const AActor* EffectorActor = It->Get())
		{
			if (!EffectorActor->FindComponentByClass<UCEEffectorComponent>())
			{
				It.RemoveCurrent();
			}
		}
	}

	OnEffectorsChanged();
}

#if WITH_EDITOR
void UCEClonerEffectorExtension::CreateLinkedEffector()
{
	UCEClonerComponent* ClonerComponent = GetClonerComponent();
	UCEClonerSubsystem* ClonerSubsystem = UCEClonerSubsystem::Get();

	if (ClonerComponent && ClonerSubsystem)
	{
		constexpr UCEClonerSubsystem::ECEClonerActionFlags Flags = UCEClonerSubsystem::ECEClonerActionFlags::All;
		ClonerSubsystem->CreateLinkedEffectors({ClonerComponent}, Flags);
	}
}

const TCEPropertyChangeDispatcher<UCEClonerEffectorExtension> UCEClonerEffectorExtension::PropertyChangeDispatcher =
{
	{ GET_MEMBER_NAME_CHECKED(UCEClonerEffectorExtension, EffectorActorsWeak), &UCEClonerEffectorExtension::OnEffectorActorsChanged }
};

void UCEClonerEffectorExtension::PostEditChangeProperty(FPropertyChangedEvent& InPropertyChangedEvent)
{
	Super::PostEditChangeProperty(InPropertyChangedEvent);

	PropertyChangeDispatcher.OnPropertyChanged(this, InPropertyChangedEvent);
}

void UCEClonerEffectorExtension::PostEditUndo()
{
	Super::PostEditUndo();

	OnEffectorsChanged();
}
#endif
