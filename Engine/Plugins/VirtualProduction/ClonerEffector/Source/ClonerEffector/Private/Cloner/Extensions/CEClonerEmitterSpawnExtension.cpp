// Copyright Epic Games, Inc. All Rights Reserved.

#include "Cloner/Extensions/CEClonerEmitterSpawnExtension.h"

#include "Cloner/CEClonerComponent.h"
#include "NiagaraEmitter.h"
#include "NiagaraSystem.h"
#include "NiagaraUserRedirectionParameterStore.h"

UCEClonerEmitterSpawnExtension::UCEClonerEmitterSpawnExtension()
	: UCEClonerExtensionBase(
		TEXT("Spawn")
		, 0
	)
{}

void UCEClonerEmitterSpawnExtension::SetSpawnLoopMode(ECEClonerSpawnLoopMode InMode)
{
	if (SpawnLoopMode == InMode)
	{
		return;
	}

	SpawnLoopMode = InMode;
	MarkExtensionDirty();
}

void UCEClonerEmitterSpawnExtension::SetSpawnLoopIterations(int32 InIterations)
{
	InIterations = FMath::Max(InIterations, 1);
	if (SpawnLoopIterations == InIterations)
	{
		return;
	}

	SpawnLoopIterations = InIterations;
	MarkExtensionDirty();
}

void UCEClonerEmitterSpawnExtension::SetSpawnLoopInterval(float InInterval)
{
	InInterval = FMath::Max(InInterval, 0.f);
	if (SpawnLoopInterval == InInterval)
	{
		return;
	}

	SpawnLoopInterval = InInterval;
	MarkExtensionDirty();
}

void UCEClonerEmitterSpawnExtension::SetSpawnBehaviorMode(ECEClonerSpawnBehaviorMode InMode)
{
	if (SpawnBehaviorMode == InMode)
	{
		return;
	}

	SpawnBehaviorMode = InMode;
	MarkExtensionDirty();
}

void UCEClonerEmitterSpawnExtension::SetSpawnRate(float InRate)
{
	InRate = FMath::Max(InRate, 0.f);
	if (SpawnRate == InRate)
	{
		return;
	}

	SpawnRate = InRate;
	MarkExtensionDirty();
}

void UCEClonerEmitterSpawnExtension::SetSpawnMaxFrameCount(int32 InCount)
{
	InCount = FMath::Clamp(InCount, 0, SpawnMaxTotalCount);
	if (SpawnMaxFrameCount == InCount)
	{
		return;
	}

	SpawnMaxFrameCount = InCount;
	MarkExtensionDirty();
}

void UCEClonerEmitterSpawnExtension::SetSpawnMaxTotalCount(int32 InCount)
{
	InCount = FMath::Max(InCount, 0);
	if (SpawnMaxTotalCount == InCount)
	{
		return;
	}

	SpawnMaxTotalCount = InCount;
	MarkExtensionDirty();
}

void UCEClonerEmitterSpawnExtension::OnExtensionParametersChanged(UCEClonerComponent* InComponent)
{
	Super::OnExtensionParametersChanged(InComponent);

	FNiagaraUserRedirectionParameterStore& ExposedParameters = InComponent->GetOverrideParameters();

	const FNiagaraVariable SpawnLoopModeVar(FNiagaraTypeDefinition(StaticEnum<ECEClonerSpawnLoopMode>()), TEXT("SpawnLoopMode"));
	ExposedParameters.SetParameterValue<int32>(static_cast<int32>(SpawnLoopMode), SpawnLoopModeVar);

	InComponent->SetIntParameter(TEXT("SpawnLoopIterations"), SpawnLoopIterations);
	
	const ECEClonerSpawnBehaviorMode BehaviorMode = SpawnLoopMode == ECEClonerSpawnLoopMode::Once ? ECEClonerSpawnBehaviorMode::Instant : SpawnBehaviorMode;
	const FNiagaraVariable SpawnBehaviorModeVar(FNiagaraTypeDefinition(StaticEnum<ECEClonerSpawnBehaviorMode>()), TEXT("SpawnBehaviorMode"));
	ExposedParameters.SetParameterValue<int32>(static_cast<int32>(BehaviorMode), SpawnBehaviorModeVar);
	
	InComponent->SetFloatParameter(TEXT("SpawnLoopInterval"), BehaviorMode == ECEClonerSpawnBehaviorMode::Instant ? SpawnLoopInterval : 1.f);

	InComponent->SetFloatParameter(TEXT("SpawnRate"), SpawnRate);

	InComponent->SetIntParameter(TEXT("SpawnMaxFrameCount"), FMath::Clamp(SpawnMaxFrameCount, 0, SpawnMaxTotalCount));

	InComponent->SetIntParameter(TEXT("SpawnMaxTotalCount"), FMath::Max(SpawnMaxTotalCount, 0));

#if WITH_EDITOR
	// Do not allow world space when spawning once
	if (SpawnLoopMode == ECEClonerSpawnLoopMode::Once && !bUseLocalSpace)
	{
		bUseLocalSpace = true;
	}
	
	OnLocalSpaceChanged();
#endif
}

#if WITH_EDITOR
void UCEClonerEmitterSpawnExtension::SetUseLocalSpace(bool bInLocalSpace)
{
	if (bUseLocalSpace == bInLocalSpace)
	{
		return;
	}

	bUseLocalSpace = bInLocalSpace;
	MarkExtensionDirty();
}

const TCEPropertyChangeDispatcher<UCEClonerEmitterSpawnExtension> UCEClonerEmitterSpawnExtension::PropertyChangeDispatcher =
{
	/** Spawn */
	{ GET_MEMBER_NAME_CHECKED(UCEClonerEmitterSpawnExtension, SpawnLoopMode), &UCEClonerEmitterSpawnExtension::OnExtensionPropertyChanged },
	{ GET_MEMBER_NAME_CHECKED(UCEClonerEmitterSpawnExtension, SpawnLoopInterval), &UCEClonerEmitterSpawnExtension::OnExtensionPropertyChanged },
	{ GET_MEMBER_NAME_CHECKED(UCEClonerEmitterSpawnExtension, SpawnLoopIterations), &UCEClonerEmitterSpawnExtension::OnExtensionPropertyChanged },
	{ GET_MEMBER_NAME_CHECKED(UCEClonerEmitterSpawnExtension, SpawnBehaviorMode), &UCEClonerEmitterSpawnExtension::OnExtensionPropertyChanged },
	{ GET_MEMBER_NAME_CHECKED(UCEClonerEmitterSpawnExtension, SpawnRate), &UCEClonerEmitterSpawnExtension::OnExtensionPropertyChanged },
	{ GET_MEMBER_NAME_CHECKED(UCEClonerEmitterSpawnExtension, bUseLocalSpace), &UCEClonerEmitterSpawnExtension::OnExtensionPropertyChanged },
	{ GET_MEMBER_NAME_CHECKED(UCEClonerEmitterSpawnExtension, SpawnMaxFrameCount), &UCEClonerEmitterSpawnExtension::OnExtensionPropertyChanged },
	{ GET_MEMBER_NAME_CHECKED(UCEClonerEmitterSpawnExtension, SpawnMaxTotalCount), &UCEClonerEmitterSpawnExtension::OnExtensionPropertyChanged },
};

void UCEClonerEmitterSpawnExtension::PostEditChangeProperty(FPropertyChangedEvent& InPropertyChangedEvent)
{
	Super::PostEditChangeProperty(InPropertyChangedEvent);

	PropertyChangeDispatcher.OnPropertyChanged(this, InPropertyChangedEvent);
}

void UCEClonerEmitterSpawnExtension::OnLocalSpaceChanged()
{
	UCEClonerComponent* ClonerComponent = GetClonerComponent();

	if (!ClonerComponent)
	{
		return;
	}

	UNiagaraSystem* ClonerSystem = ClonerComponent->GetAsset();

	if (!ClonerSystem || ClonerSystem->GetNumEmitters() == 0)
	{
		return;
	}
	
	const FNiagaraEmitterHandle& EmitterHandle = ClonerSystem->GetEmitterHandle(0);

	if (FVersionedNiagaraEmitterData* EmitterData = EmitterHandle.GetEmitterData())
	{
		if (EmitterData->bLocalSpace != bUseLocalSpace)
		{
			EmitterData->bLocalSpace = bUseLocalSpace;
			ClonerSystem->RequestCompile(/** Force */false);
		}
	}
}
#endif
