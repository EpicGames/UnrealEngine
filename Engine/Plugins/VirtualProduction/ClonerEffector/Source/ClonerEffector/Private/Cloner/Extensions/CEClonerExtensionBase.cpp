// Copyright Epic Games, Inc. All Rights Reserved.

#include "Cloner/Extensions/CEClonerExtensionBase.h"

#include "Cloner/CEClonerComponent.h"
#include "Cloner/Logs/CEClonerLogs.h"

#if WITH_EDITOR
FCEExtensionSection UCEClonerExtensionBase::GetExtensionSection() const
{
	return UE::ClonerEffector::EditorSection::GetExtensionSectionFromClass(GetClass());
}
#endif

UCEClonerExtensionBase::UCEClonerExtensionBase()
	: UCEClonerExtensionBase(
		NAME_None
		, 0
	)
{}

UCEClonerExtensionBase::UCEClonerExtensionBase(FName InExtensionName, int32 InExtensionPriority)
	: ExtensionName(InExtensionName)
	, ExtensionPriority(InExtensionPriority)
{}

UCEClonerComponent* UCEClonerExtensionBase::GetClonerComponent() const
{
	return GetTypedOuter<UCEClonerComponent>();
}

UCEClonerComponent* UCEClonerExtensionBase::GetClonerComponentChecked() const
{
	UCEClonerComponent* ClonerComponent = GetClonerComponent();

	if (IsValid(ClonerComponent))
	{
		return ClonerComponent;
	}

	checkf(false, TEXT("Cloner component is invalid"))

	return nullptr;
}

UCEClonerLayoutBase* UCEClonerExtensionBase::GetClonerLayout() const
{
	if (const UCEClonerComponent* ClonerComponent = GetClonerComponent())
	{
		return ClonerComponent->GetActiveLayout();
	}

	return nullptr;
}

void UCEClonerExtensionBase::ActivateExtension()
{
	if (!bExtensionActive)
	{
		bExtensionActive = true;
		const UCEClonerComponent* ClonerComponent = GetClonerComponentChecked();
		UE_LOGF(LogCECloner, Verbose, "%ls : Cloner extension activated %ls", *ClonerComponent->GetOwner()->GetActorNameOrLabel(), *GetExtensionName().ToString());
		OnExtensionActivated();
	}
}

void UCEClonerExtensionBase::DeactivateExtension()
{
	if (bExtensionActive)
	{
		bExtensionActive = false;
		const UCEClonerComponent* ClonerComponent = GetClonerComponentChecked();
		UE_LOGF(LogCECloner, Verbose, "%ls : Cloner extension deactivated %ls", *ClonerComponent->GetOwner()->GetActorNameOrLabel(), *GetExtensionName().ToString());
		OnExtensionDeactivated();
	}
}

void UCEClonerExtensionBase::UpdateExtensionParameters()
{
	if (!IsExtensionActive())
	{
		return;
	}

	if (UCEClonerComponent* ClonerComponent = GetClonerComponent())
	{
		if (!ClonerComponent->GetEnabled())
		{
			return;
		}

		OnExtensionParametersChanged(ClonerComponent);

		if (EnumHasAnyFlags(ExtensionStatus, ECEClonerSystemStatus::SimulationDirty))
		{
			ClonerComponent->RequestClonerUpdate(/*Immediate*/false);
		}
	}

	ExtensionStatus = ECEClonerSystemStatus::UpToDate;
}

void UCEClonerExtensionBase::MarkExtensionDirty(bool bInUpdateCloner)
{
	if (!IsExtensionDirty())
	{
		// Notify other extensions once
		if (const UCEClonerComponent* ClonerComponent = GetClonerComponent())
		{
			for (UCEClonerExtensionBase* ActiveExtension : ClonerComponent->GetActiveExtensions())
			{
				if (ActiveExtension)
				{
					ActiveExtension->OnExtensionDirtied(this);
				}
			}	
		}
	}

	EnumAddFlags(ExtensionStatus, ECEClonerSystemStatus::ParametersDirty);

	if (bInUpdateCloner)
	{
		EnumAddFlags(ExtensionStatus, ECEClonerSystemStatus::SimulationDirty);
	}
}

bool UCEClonerExtensionBase::IsExtensionDirty() const
{
	return EnumHasAnyFlags(ExtensionStatus, ECEClonerSystemStatus::ParametersDirty);
}

void UCEClonerExtensionBase::PostEditImport()
{
	Super::PostEditImport();

	MarkExtensionDirty();
}

#if WITH_EDITOR
void UCEClonerExtensionBase::PostEditUndo()
{
	Super::PostEditUndo();

	MarkExtensionDirty();
}
#endif

void UCEClonerExtensionBase::OnExtensionPropertyChanged()
{
	MarkExtensionDirty();
}
