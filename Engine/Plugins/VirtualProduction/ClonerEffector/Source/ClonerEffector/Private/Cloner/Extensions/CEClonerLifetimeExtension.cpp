// Copyright Epic Games, Inc. All Rights Reserved.

#include "Cloner/Extensions/CEClonerLifetimeExtension.h"

#include "Cloner/CEClonerComponent.h"
#include "Cloner/Layouts/CEClonerLayoutBase.h"
#include "Curves/CurveFloat.h"
#include "NiagaraDataInterfaceCurve.h"
#include "NiagaraSystem.h"
#include "NiagaraUserRedirectionParameterStore.h"

UCEClonerLifetimeExtension::UCEClonerLifetimeExtension()
	: UCEClonerExtensionBase(
		TEXT("Lifetime")
		, 0)
{
	// Default Scale Curve
	LifetimeScaleCurve.AddKey(0, 1.f);
	LifetimeScaleCurve.AddKey(1, 0.f);
}

void UCEClonerLifetimeExtension::SetLifetimeEnabled(bool bInEnabled)
{
	if (bLifetimeEnabled == bInEnabled)
	{
		return;
	}

	bLifetimeEnabled = bInEnabled;
	MarkExtensionDirty();
}

void UCEClonerLifetimeExtension::SetLifetimeMin(float InMin)
{
	if (LifetimeMin == InMin)
	{
		return;
	}

	if (InMin < 0)
	{
		return;
	}

	LifetimeMin = InMin;
	MarkExtensionDirty();
}

void UCEClonerLifetimeExtension::SetLifetimeMax(float InMax)
{
	if (LifetimeMax == InMax)
	{
		return;
	}

	if (InMax < 0)
	{
		return;
	}

	LifetimeMax = InMax;
	MarkExtensionDirty();
}

void UCEClonerLifetimeExtension::SetLifetimeScaleEnabled(bool bInEnabled)
{
	if (bLifetimeScaleEnabled == bInEnabled)
	{
		return;
	}

	bLifetimeScaleEnabled = bInEnabled;
	MarkExtensionDirty();
}

void UCEClonerLifetimeExtension::SetLifetimeScaleCurve(UCurveFloat* InCurve)
{
	if (!IsValid(InCurve))
	{
		return;
	}

	SetLifetimeScaleCurve(InCurve->FloatCurve);
}

void UCEClonerLifetimeExtension::SetLifetimeScaleCurve(const FRichCurve& InCurve)
{
	LifetimeScaleCurve = InCurve;
	MarkExtensionDirty();
}

void UCEClonerLifetimeExtension::OnExtensionParametersChanged(UCEClonerComponent* InComponent)
{
	Super::OnExtensionParametersChanged(InComponent);

	LifetimeMin = FMath::Max(0, LifetimeMin);
	LifetimeMax = FMath::Max(LifetimeMin, LifetimeMax);

	InComponent->SetBoolParameter(TEXT("LifetimeEnabled"), bLifetimeEnabled);

	InComponent->SetFloatParameter(TEXT("LifetimeMin"), LifetimeMin);

	InComponent->SetFloatParameter(TEXT("LifetimeMax"), LifetimeMax);

	InComponent->SetBoolParameter(TEXT("LifetimeScaleEnabled"), bLifetimeEnabled && bLifetimeScaleEnabled);

	FNiagaraUserRedirectionParameterStore& ExposedParameters = InComponent->GetOverrideParameters();

	static const FNiagaraVariable LifetimeScaleCurveVar(FNiagaraTypeDefinition(UNiagaraDataInterfaceCurve::StaticClass()), TEXT("LifetimeScaleCurve"));

#if WITH_EDITOR
	if (UNiagaraDataInterfaceCurve* LifetimeCurve = LifetimeScaleCurveDIWeak.Get())
	{
		LifetimeCurve->OnChanged().RemoveAll(this);
	}
#endif

	if (UNiagaraDataInterfaceCurve* LifetimeScaleCurveDI = Cast<UNiagaraDataInterfaceCurve>(ExposedParameters.GetDataInterface(LifetimeScaleCurveVar)))
	{
		LifetimeScaleCurveDIWeak = LifetimeScaleCurveDI;
		LifetimeScaleCurveDI->Curve = LifetimeScaleCurve;

#if WITH_EDITOR
		LifetimeScaleCurveDI->UpdateLUT();
		LifetimeScaleCurveDI->OnChanged().AddUObject(this, &UCEClonerLifetimeExtension::OnLifetimeScaleCurveChanged);
#endif
	}
}

void UCEClonerLifetimeExtension::OnLifetimeScaleCurveChanged()
{
	if (const UNiagaraDataInterfaceCurve* LifetimeCurve = LifetimeScaleCurveDIWeak.Get())
	{
		LifetimeScaleCurve = LifetimeCurve->Curve;
		MarkExtensionDirty();
	}
}

#if WITH_EDITOR
const TCEPropertyChangeDispatcher<UCEClonerLifetimeExtension> UCEClonerLifetimeExtension::PropertyChangeDispatcher =
{
	/** Lifetime */
	{ GET_MEMBER_NAME_CHECKED(UCEClonerLifetimeExtension, bLifetimeEnabled), &UCEClonerLifetimeExtension::OnExtensionPropertyChanged },
	{ GET_MEMBER_NAME_CHECKED(UCEClonerLifetimeExtension, LifetimeMin), &UCEClonerLifetimeExtension::OnExtensionPropertyChanged },
	{ GET_MEMBER_NAME_CHECKED(UCEClonerLifetimeExtension, LifetimeMax), &UCEClonerLifetimeExtension::OnExtensionPropertyChanged },
	{ GET_MEMBER_NAME_CHECKED(UCEClonerLifetimeExtension, bLifetimeScaleEnabled), &UCEClonerLifetimeExtension::OnExtensionPropertyChanged },
};

void UCEClonerLifetimeExtension::PostEditChangeProperty(FPropertyChangedEvent& InPropertyChangedEvent)
{
	Super::PostEditChangeProperty(InPropertyChangedEvent);

	PropertyChangeDispatcher.OnPropertyChanged(this, InPropertyChangedEvent);
}
#endif
