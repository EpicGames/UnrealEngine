// Copyright Epic Games, Inc. All Rights Reserved.

#include "Cloner/Extensions/CEClonerMeshRendererExtension.h"

#include "Cloner/CEClonerComponent.h"
#include "Cloner/Logs/CEClonerLogs.h"
#include "DataInterface/NiagaraDataInterfaceArrayMesh.h"
#include "Materials/Material.h"
#include "Materials/MaterialInterface.h"
#include "NiagaraFunctionLibrary.h"
#include "NiagaraSystem.h"
#include "Settings/CEClonerEffectorSettings.h"
#include "UObject/ConstructorHelpers.h"
#include "Utilities/CEClonerEffectorUtilities.h"

#if WITH_EDITOR
#include "Containers/Ticker.h"
#include "Misc/TransactionObjectEvent.h"
#endif

UCEClonerMeshRendererExtension::UCEClonerMeshRendererExtension()
	: UCEClonerExtensionBase(
		TEXT("MeshRenderer")
		, 1)
{
	// Default override material
	static ConstructorHelpers::FObjectFinder<UMaterialInterface> DefaultMaterialFinder(UCEClonerEffectorSettings::DefaultMaterialPath);
	OverrideMaterial = DefaultMaterialFinder.Object;
}

void UCEClonerMeshRendererExtension::SetMeshRenderMode(ECEClonerMeshRenderMode InMode)
{
	if (InMode == MeshRenderMode)
	{
		return;
	}

	MeshRenderMode = InMode;
	MarkExtensionDirty();
}

void UCEClonerMeshRendererExtension::SetMeshFacingMode(ENiagaraMeshFacingMode InMode)
{
	if (MeshFacingMode == InMode)
	{
		return;
	}

	MeshFacingMode = InMode;
	OnOverrideMaterialOptionsChanged();
}

void UCEClonerMeshRendererExtension::SetMeshCastShadows(bool InbCastShadows)
{
	if (bMeshCastShadows == InbCastShadows)
	{
		return;
	}

	bMeshCastShadows = InbCastShadows;
	OnOverrideMaterialOptionsChanged();
}

void UCEClonerMeshRendererExtension::SetDefaultMeshes(const TArray<TObjectPtr<UStaticMesh>>& InMeshes)
{
	DefaultMeshes = InMeshes;
	OnOverrideMaterialOptionsChanged();
}

void UCEClonerMeshRendererExtension::SetDefaultMeshes(const TArray<UStaticMesh*>& InMeshes)
{
	DefaultMeshes.Empty(InMeshes.Num());

	Algo::Transform(InMeshes, DefaultMeshes, [](UStaticMesh* InMesh)->TObjectPtr<UStaticMesh>
	{
		return InMesh;
	});

	OnOverrideMaterialOptionsChanged();
}

void UCEClonerMeshRendererExtension::GetDefaultMeshes(TArray<UStaticMesh*>& OutMeshes) const
{
	OutMeshes.Empty(DefaultMeshes.Num());
	Algo::Transform(DefaultMeshes, OutMeshes, [](const TObjectPtr<UStaticMesh>& InMesh)->UStaticMesh*
	{
		return InMesh;
	});
}

void UCEClonerMeshRendererExtension::SetVisualizeEffectors(bool bInVisualize)
{
	if (bVisualizeEffectors == bInVisualize)
	{
		return;
	}

	bVisualizeEffectors = bInVisualize;
	OnOverrideMaterialOptionsChanged();
}

void UCEClonerMeshRendererExtension::SetUseOverrideMaterial(bool bInOverride)
{
	if (bUseOverrideMaterial == bInOverride)
	{
		return;
	}

	bUseOverrideMaterial = bInOverride;
	OnOverrideMaterialOptionsChanged();
}

void UCEClonerMeshRendererExtension::SetOverrideMaterial(UMaterialInterface* InMaterial)
{
	if (OverrideMaterial == InMaterial)
	{
		return;
	}

	OverrideMaterial = InMaterial;
	OnOverrideMaterialOptionsChanged();
}

void UCEClonerMeshRendererExtension::SetSortTranslucentParticles(bool bInSort)
{
	if (bSortTranslucentParticles == bInSort)
	{
		return;
	}

	bSortTranslucentParticles = bInSort;
	OnOverrideMaterialOptionsChanged();	
}

void UCEClonerMeshRendererExtension::OnExtensionParametersChanged(UCEClonerComponent* InComponent)
{
	Super::OnExtensionParametersChanged(InComponent);

	FNiagaraUserRedirectionParameterStore& ExposedParameters = InComponent->GetOverrideParameters();
	static const FNiagaraVariable MeshModeVar(FNiagaraTypeDefinition(StaticEnum<ECEClonerMeshRenderMode>()), TEXT("MeshRenderMode"));
	ExposedParameters.SetParameterValue<int32>(static_cast<int32>(MeshRenderMode), MeshModeVar);
}

int32 UCEClonerMeshRendererExtension::GetClonerMeshesMaterialCount() const
{
	int32 MaterialCount = 0;

	UCEClonerComponent* const ClonerComponent = GetClonerComponent();
	if (!ClonerComponent)
	{
		return MaterialCount;
	}

	const UNiagaraDataInterfaceArrayMesh* const MeshArrayDI = UE::ClonerEffector::FindMeshArrayDataInterface(ClonerComponent);
	if (!MeshArrayDI)
	{
		return MaterialCount;
	}

	for (const FNiagaraMeshRendererMeshPropertiesBase& MeshProperties : MeshArrayDI->MeshData)
	{
		if (!MeshProperties.Mesh)
		{
			continue;
		}

		MaterialCount += MeshProperties.Mesh->GetNumSections(/** LOD */0);
	}

	return MaterialCount;
}

TArray<FNiagaraMeshMaterialOverride> UCEClonerMeshRendererExtension::GetOverrideMeshesMaterials() const
{
	TArray<FNiagaraMeshMaterialOverride> MaterialOverrides;

	const bool bOverrideMaterial = bUseOverrideMaterial || bVisualizeEffectors;

	if (bOverrideMaterial)
	{
		// Set same material for all available slots
		const int32 MaterialCount = GetClonerMeshesMaterialCount();
		MaterialOverrides.Reserve(MaterialCount);

		UMaterialInterface* OverrideMeshesMaterial = bVisualizeEffectors
			? LoadObject<UMaterialInterface>(nullptr, UCEClonerEffectorSettings::DefaultMaterialPath)
			: OverrideMaterial.Get();

		for (int32 Index = 0; Index < MaterialCount; Index++)
		{
			FNiagaraMeshMaterialOverride MaterialOverride;
			MaterialOverride.ExplicitMat = OverrideMeshesMaterial;
			MaterialOverrides.Add(MaterialOverride);
		}
	}

	return MaterialOverrides;
}

void UCEClonerMeshRendererExtension::OnOverrideMaterialOptionsChanged()
{
	using namespace UE::ClonerEffector::Utilities;

	if (IsValid(OverrideMaterial) && !IsMaterialUsageFlagSet(OverrideMaterial))
	{
		UE_LOGF(LogCECloner, Warning, "%ls : The override material (%ls) you wish to use does not have the required usage flag (bUsedWithNiagaraMeshParticles) to work with the cloner, enable the flag on the material and save the asset", *GetClonerComponent()->GetOwner()->GetActorNameOrLabel(), *OverrideMaterial->GetMaterial()->GetPathName());

#if WITH_EDITOR
		ShowWarning(FText::Format(GetMaterialWarningText(), 1));
#endif

		OverrideMaterial = nullptr;
	}

	if (UCEClonerComponent* ClonerComponent = GetClonerComponent())
	{
		ClonerComponent->RefreshClonerMeshes();
	}
}

bool UCEClonerMeshRendererExtension::OnUpdateClonerMeshes(TArray<FNiagaraMeshRendererMeshPropertiesBase>& InOutMeshes)
{
	Super::OnUpdateClonerMeshes(InOutMeshes);

	UCEClonerComponent* const ClonerComponent = GetClonerComponent();
	if (!ClonerComponent)
	{
		return false;
	}

	UCEClonerLayoutBase* const Layout = GetClonerLayout();
	if (!Layout)
	{
		return false;
	}

	UNiagaraMeshRendererProperties* const MeshRenderer = Layout->GetMeshRenderer();
	if (!MeshRenderer)
	{
		return false;
	}

	// Clear the meshes as the mesh array DI is going to be used. This should be always empty for new systems, but old systems could have lingering meshes here.
	MeshRenderer->Meshes.Empty();
	MeshRenderer->FacingMode = MeshFacingMode;
	MeshRenderer->bCastShadows = bMeshCastShadows;
	MeshRenderer->SortMode = bSortTranslucentParticles ? ENiagaraSortMode::ViewDepth : ENiagaraSortMode::None;

	bool bMeshDataChanged = false;

	// Use default meshes if nothing is attached
	if (ClonerComponent->GetAttachmentCount() == 0)
	{
		const TArray<TObjectPtr<UStaticMesh>>& NewDefaultMeshes = GetDefaultMeshes();

		bMeshDataChanged = InOutMeshes.Num() != NewDefaultMeshes.Num();
		InOutMeshes.SetNum(NewDefaultMeshes.Num());

		for (int32 MeshIndex = 0; MeshIndex < NewDefaultMeshes.Num(); MeshIndex++)
		{
			UStaticMesh* DefaultMesh = NewDefaultMeshes[MeshIndex].Get();
			if (DefaultMesh && DefaultMesh->GetNumTriangles(0) <= 0)
			{
				DefaultMesh = nullptr;
			}

			FNiagaraMeshRendererMeshPropertiesBase& MeshProperties = InOutMeshes[MeshIndex];

			bMeshDataChanged = bMeshDataChanged
				|| MeshProperties.Mesh != DefaultMesh 
				|| !MeshProperties.Scale.Equals(ClonerComponent->GetGlobalScale()) 
				|| !MeshProperties.Rotation.Equals(ClonerComponent->GetGlobalRotation());

			MeshProperties.Mesh = DefaultMesh;
			MeshProperties.Scale = ClonerComponent->GetGlobalScale();
			MeshProperties.Rotation = ClonerComponent->GetGlobalRotation();
		}
	}

	// Set material override
	TArray<FNiagaraMeshMaterialOverride> OverrideMaterials = GetOverrideMeshesMaterials();
	const bool bOverrideMaterials = bUseOverrideMaterial || bVisualizeEffectors;

	bMeshDataChanged = bMeshDataChanged 
		|| MeshRenderer->bOverrideMaterials != bOverrideMaterials
		|| MeshRenderer->OverrideMaterials != OverrideMaterials;

	MeshRenderer->bOverrideMaterials = bOverrideMaterials;
	MeshRenderer->OverrideMaterials = MoveTemp(OverrideMaterials);

	return bMeshDataChanged;
}

#if WITH_EDITOR
const TCEPropertyChangeDispatcher<UCEClonerMeshRendererExtension> UCEClonerMeshRendererExtension::PropertyChangeDispatcher =
{
	/** Renderer */
	{ GET_MEMBER_NAME_CHECKED(UCEClonerMeshRendererExtension, MeshRenderMode), &UCEClonerMeshRendererExtension::OnExtensionPropertyChanged },
	{ GET_MEMBER_NAME_CHECKED(UCEClonerMeshRendererExtension, MeshFacingMode), &UCEClonerMeshRendererExtension::OnOverrideMaterialOptionsChanged },
	{ GET_MEMBER_NAME_CHECKED(UCEClonerMeshRendererExtension, bMeshCastShadows), &UCEClonerMeshRendererExtension::OnOverrideMaterialOptionsChanged },
	{ GET_MEMBER_NAME_CHECKED(UCEClonerMeshRendererExtension, DefaultMeshes), &UCEClonerMeshRendererExtension::OnOverrideMaterialOptionsChanged },
	{ GET_MEMBER_NAME_CHECKED(UCEClonerMeshRendererExtension, bUseOverrideMaterial), &UCEClonerMeshRendererExtension::OnOverrideMaterialOptionsChanged },
	{ GET_MEMBER_NAME_CHECKED(UCEClonerMeshRendererExtension, OverrideMaterial), &UCEClonerMeshRendererExtension::OnOverrideMaterialOptionsChanged },
	{ GET_MEMBER_NAME_CHECKED(UCEClonerMeshRendererExtension, bVisualizeEffectors), &UCEClonerMeshRendererExtension::OnOverrideMaterialOptionsChanged },
	{ GET_MEMBER_NAME_CHECKED(UCEClonerMeshRendererExtension, bSortTranslucentParticles), &UCEClonerMeshRendererExtension::OnOverrideMaterialOptionsChanged },
};

void UCEClonerMeshRendererExtension::PostTransacted(const FTransactionObjectEvent& InTransactionEvent)
{
	Super::PostTransacted(InTransactionEvent);

	if (InTransactionEvent.GetEventType() == ETransactionObjectEventType::UndoRedo)
	{
		if (InTransactionEvent.GetChangedProperties().Contains(GET_MEMBER_NAME_CHECKED(UCEClonerMeshRendererExtension, DefaultMeshes)))
		{
			if (UCEClonerComponent* ClonerComponent = GetClonerComponent())
			{
				// Redo : reactivate system and refresh
				if (!ClonerComponent->IsActive())
				{
					ClonerComponent->SetActiveFlag(true);
					FTSTicker::GetCoreTicker().AddTicker(FTickerDelegate::CreateWeakLambda(this, [this](float)
					{
						if (UCEClonerComponent* ClonerComponent = GetClonerComponent())
						{
							ClonerComponent->RefreshClonerMeshes();
						}

						return false;
					}));
				}
				// Undo : deactivate system and destroy instance
				else
				{
					ClonerComponent->DestroyInstanceNotComponent();
				}
			}
		}
	}
}

void UCEClonerMeshRendererExtension::PostEditChangeProperty(FPropertyChangedEvent& InPropertyChangedEvent)
{
	Super::PostEditChangeProperty(InPropertyChangedEvent);

	PropertyChangeDispatcher.OnPropertyChanged(this, InPropertyChangedEvent);
}
#endif
