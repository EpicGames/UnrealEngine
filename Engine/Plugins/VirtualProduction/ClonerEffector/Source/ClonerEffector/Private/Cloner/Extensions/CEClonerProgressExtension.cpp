// Copyright Epic Games, Inc. All Rights Reserved.

#include "Cloner/Extensions/CEClonerProgressExtension.h"

#include "Cloner/CEClonerComponent.h"

UCEClonerProgressExtension::UCEClonerProgressExtension()
	: UCEClonerExtensionBase(
		TEXT("Progress")
		, 0
	)
{}

void UCEClonerProgressExtension::SetInvertProgress(bool bInInvertProgress)
{
	if (bInvertProgress == bInInvertProgress)
	{
		return;
	}

	bInvertProgress = bInInvertProgress;
	MarkExtensionDirty();
}

void UCEClonerProgressExtension::SetProgress(float InProgress)
{
	InProgress = FMath::Clamp(InProgress, 0, 1);

	if (FMath::IsNearlyEqual(Progress, InProgress))
	{
		return;
	}

	Progress = InProgress;
	MarkExtensionDirty();
}

void UCEClonerProgressExtension::OnExtensionParametersChanged(UCEClonerComponent* InComponent)
{
	Super::OnExtensionParametersChanged(InComponent);

	InComponent->SetFloatParameter(TEXT("ParticleProgress"), Progress * (bInvertProgress ? -1 : 1));
}

#if WITH_EDITOR
const TCEPropertyChangeDispatcher<UCEClonerProgressExtension> UCEClonerProgressExtension::PropertyChangeDispatcher =
{
	/** Progress */
	{ GET_MEMBER_NAME_CHECKED(UCEClonerProgressExtension, bInvertProgress), &UCEClonerProgressExtension::OnExtensionPropertyChanged },
	{ GET_MEMBER_NAME_CHECKED(UCEClonerProgressExtension, Progress), &UCEClonerProgressExtension::OnExtensionPropertyChanged }
};

void UCEClonerProgressExtension::PostEditChangeProperty(FPropertyChangedEvent& InPropertyChangedEvent)
{
	Super::PostEditChangeProperty(InPropertyChangedEvent);

	PropertyChangeDispatcher.OnPropertyChanged(this, InPropertyChangedEvent);
}
#endif
