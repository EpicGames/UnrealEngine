// Copyright Epic Games, Inc. All Rights Reserved.

#include "Cloner/Extensions/CEClonerRangeExtension.h"

#include "Cloner/CEClonerComponent.h"

UCEClonerRangeExtension::UCEClonerRangeExtension()
	: UCEClonerExtensionBase(
		TEXT("Range")
		, 0
	)
{}

void UCEClonerRangeExtension::SetRangeEnabled(bool bInRangeEnabled)
{
	if (bRangeEnabled == bInRangeEnabled)
	{
		return;
	}

	bRangeEnabled = bInRangeEnabled;
	MarkExtensionDirty();
}

void UCEClonerRangeExtension::SetRangeMirrored(bool bInMirrored)
{
	if (bRangeMirrored == bInMirrored)
	{
		return;
	}

	bRangeMirrored = bInMirrored;
	MarkExtensionDirty();
}

void UCEClonerRangeExtension::SetRangeOffsetMin(const FVector& InRangeOffsetMin)
{
	if (RangeOffsetMin == InRangeOffsetMin)
	{
		return;
	}

	RangeOffsetMin = InRangeOffsetMin;
	MarkExtensionDirty();
}

void UCEClonerRangeExtension::SetRangeOffsetMax(const FVector& InRangeOffsetMax)
{
	if (RangeOffsetMax == InRangeOffsetMax)
	{
		return;
	}

	RangeOffsetMax = InRangeOffsetMax;
	MarkExtensionDirty();
}

void UCEClonerRangeExtension::SetRangeRotationMin(const FRotator& InRangeRotationMin)
{
	if (RangeRotationMin == InRangeRotationMin)
	{
		return;
	}

	RangeRotationMin = InRangeRotationMin;
	MarkExtensionDirty();
}

void UCEClonerRangeExtension::SetRangeRotationMax(const FRotator& InRangeRotationMax)
{
	if (RangeRotationMax == InRangeRotationMax)
	{
		return;
	}

	RangeRotationMax = InRangeRotationMax;
	MarkExtensionDirty();
}

void UCEClonerRangeExtension::SetRangeScaleUniform(bool bInRangeScaleUniform)
{
	if (bRangeScaleUniform == bInRangeScaleUniform)
	{
		return;
	}

	bRangeScaleUniform = bInRangeScaleUniform;
	MarkExtensionDirty();
}

void UCEClonerRangeExtension::SetRangeScaleMin(const FVector& InRangeScaleMin)
{
	if (RangeScaleMin == InRangeScaleMin)
	{
		return;
	}

	RangeScaleMin = InRangeScaleMin;
	MarkExtensionDirty();
}

void UCEClonerRangeExtension::SetRangeScaleMax(const FVector& InRangeScaleMax)
{
	if (RangeScaleMax == InRangeScaleMax)
	{
		return;
	}

	RangeScaleMax = InRangeScaleMax;
	MarkExtensionDirty();
}

void UCEClonerRangeExtension::SetRangeScaleUniformMin(float InRangeScaleUniformMin)
{
	if (RangeScaleUniformMin == InRangeScaleUniformMin)
	{
		return;
	}

	RangeScaleUniformMin = InRangeScaleUniformMin;
	MarkExtensionDirty();
}

void UCEClonerRangeExtension::SetRangeScaleUniformMax(float InRangeScaleUniformMax)
{
	if (RangeScaleUniformMax == InRangeScaleUniformMax)
	{
		return;
	}

	RangeScaleUniformMax = InRangeScaleUniformMax;
	MarkExtensionDirty();
}

void UCEClonerRangeExtension::OnExtensionParametersChanged(UCEClonerComponent* InComponent)
{
	Super::OnExtensionParametersChanged(InComponent);

	if (bRangeMirrored)
	{
		RangeOffsetMin = -RangeOffsetMax;
		RangeRotationMin = -1 * RangeRotationMax;
	}

	RangeScaleUniformMin = FMath::Clamp(RangeScaleUniformMin, UE_KINDA_SMALL_NUMBER, RangeScaleUniformMax);
	RangeScaleUniformMax = FMath::Max3(RangeScaleUniformMin, RangeScaleUniformMax, UE_KINDA_SMALL_NUMBER);

	RangeScaleMin.X = FMath::Clamp(RangeScaleMin.X, UE_KINDA_SMALL_NUMBER, RangeScaleMax.X);
	RangeScaleMin.Y = FMath::Clamp(RangeScaleMin.Y, UE_KINDA_SMALL_NUMBER, RangeScaleMax.Y);
	RangeScaleMin.Z = FMath::Clamp(RangeScaleMin.Z, UE_KINDA_SMALL_NUMBER, RangeScaleMax.Z);

	RangeScaleMax.X = FMath::Max3<double>(RangeScaleMin.X, RangeScaleMax.X, UE_KINDA_SMALL_NUMBER);
	RangeScaleMax.Y = FMath::Max3<double>(RangeScaleMin.Y, RangeScaleMax.Y, UE_KINDA_SMALL_NUMBER);
	RangeScaleMax.Z = FMath::Max3<double>(RangeScaleMin.Z, RangeScaleMax.Z, UE_KINDA_SMALL_NUMBER);

	InComponent->SetBoolParameter(TEXT("RangeEnabled"), bRangeEnabled);

	InComponent->SetVectorParameter(TEXT("RangeOffsetMin"), RangeOffsetMin);

	InComponent->SetVectorParameter(TEXT("RangeOffsetMax"), RangeOffsetMax);

	InComponent->SetVariableQuat(TEXT("RangeRotationMin"), RangeRotationMin.Quaternion());

	InComponent->SetVariableQuat(TEXT("RangeRotationMax"), RangeRotationMax.Quaternion());

	InComponent->SetBoolParameter(TEXT("RangeScaleUniform"), bRangeScaleUniform);

	InComponent->SetVectorParameter(TEXT("RangeScaleMin"), RangeScaleMin);

	InComponent->SetVectorParameter(TEXT("RangeScaleMax"), RangeScaleMax);

	InComponent->SetFloatParameter(TEXT("RangeScaleUniformMin"), RangeScaleUniformMin);

	InComponent->SetFloatParameter(TEXT("RangeScaleUniformMax"), RangeScaleUniformMax);
}

#if WITH_EDITOR
const TCEPropertyChangeDispatcher<UCEClonerRangeExtension> UCEClonerRangeExtension::PropertyChangeDispatcher =
{
	/** Range */
	{ GET_MEMBER_NAME_CHECKED(UCEClonerRangeExtension, bRangeEnabled), &UCEClonerRangeExtension::OnExtensionPropertyChanged },
	{ GET_MEMBER_NAME_CHECKED(UCEClonerRangeExtension, bRangeMirrored), &UCEClonerRangeExtension::OnExtensionPropertyChanged },
	{ GET_MEMBER_NAME_CHECKED(UCEClonerRangeExtension, RangeOffsetMin), &UCEClonerRangeExtension::OnExtensionPropertyChanged },
	{ GET_MEMBER_NAME_CHECKED(UCEClonerRangeExtension, RangeOffsetMax), &UCEClonerRangeExtension::OnExtensionPropertyChanged },
	{ GET_MEMBER_NAME_CHECKED(UCEClonerRangeExtension, RangeRotationMin), &UCEClonerRangeExtension::OnExtensionPropertyChanged },
	{ GET_MEMBER_NAME_CHECKED(UCEClonerRangeExtension, RangeRotationMax), &UCEClonerRangeExtension::OnExtensionPropertyChanged },
	{ GET_MEMBER_NAME_CHECKED(UCEClonerRangeExtension, RangeScaleMin), &UCEClonerRangeExtension::OnExtensionPropertyChanged },
	{ GET_MEMBER_NAME_CHECKED(UCEClonerRangeExtension, RangeScaleMax), &UCEClonerRangeExtension::OnExtensionPropertyChanged },
	{ GET_MEMBER_NAME_CHECKED(UCEClonerRangeExtension, bRangeScaleUniform), &UCEClonerRangeExtension::OnExtensionPropertyChanged },
	{ GET_MEMBER_NAME_CHECKED(UCEClonerRangeExtension, RangeScaleUniformMin), &UCEClonerRangeExtension::OnExtensionPropertyChanged },
	{ GET_MEMBER_NAME_CHECKED(UCEClonerRangeExtension, RangeScaleUniformMax), &UCEClonerRangeExtension::OnExtensionPropertyChanged },
};

void UCEClonerRangeExtension::PostEditChangeProperty(FPropertyChangedEvent& InPropertyChangedEvent)
{
	Super::PostEditChangeProperty(InPropertyChangedEvent);

	PropertyChangeDispatcher.OnPropertyChanged(this, InPropertyChangedEvent);
}
#endif
