// Copyright Epic Games, Inc. All Rights Reserved.

#include "Cloner/Extensions/CEClonerStepExtension.h"

#include "Cloner/CEClonerComponent.h"

UCEClonerStepExtension::UCEClonerStepExtension()
	: UCEClonerExtensionBase(
		TEXT("Step")
		, 0
	)
{}

void UCEClonerStepExtension::SetDeltaStepEnabled(bool bInEnabled)
{
	if (bDeltaStepEnabled == bInEnabled)
	{
		return;
	}

	bDeltaStepEnabled = bInEnabled;
	MarkExtensionDirty();
}

void UCEClonerStepExtension::SetDeltaStepPosition(const FVector& InPosition)
{
	if (InPosition.Equals(DeltaStepPosition))
	{
		return;
	}

	DeltaStepPosition = InPosition;
	MarkExtensionDirty();
}

void UCEClonerStepExtension::SetDeltaStepRotation(const FRotator& InRotation)
{
	if (InRotation.Equals(DeltaStepRotation))
	{
		return;
	}

	DeltaStepRotation = InRotation;
	MarkExtensionDirty();
}

void UCEClonerStepExtension::SetDeltaStepScale(const FVector& InScale)
{
	if (InScale.Equals(DeltaStepScale))
	{
		return;
	}

	DeltaStepScale = InScale;
	MarkExtensionDirty();
}

void UCEClonerStepExtension::OnExtensionParametersChanged(UCEClonerComponent* InComponent)
{
	Super::OnExtensionParametersChanged(InComponent);

	InComponent->SetBoolParameter(TEXT("DeltaStepEnabled"), bDeltaStepEnabled);

	InComponent->SetVectorParameter(TEXT("DeltaStepPosition"), DeltaStepPosition);

	InComponent->SetVectorParameter(TEXT("DeltaStepRotation"), FVector(DeltaStepRotation.Roll, DeltaStepRotation.Pitch, DeltaStepRotation.Yaw));

	InComponent->SetVectorParameter(TEXT("DeltaStepScale"), DeltaStepScale);
}

#if WITH_EDITOR
const TCEPropertyChangeDispatcher<UCEClonerStepExtension> UCEClonerStepExtension::PropertyChangeDispatcher =
{
	/** Step */
	{ GET_MEMBER_NAME_CHECKED(UCEClonerStepExtension, bDeltaStepEnabled), &UCEClonerStepExtension::OnExtensionPropertyChanged },
	{ GET_MEMBER_NAME_CHECKED(UCEClonerStepExtension, DeltaStepPosition), &UCEClonerStepExtension::OnExtensionPropertyChanged },
	{ GET_MEMBER_NAME_CHECKED(UCEClonerStepExtension, DeltaStepRotation), &UCEClonerStepExtension::OnExtensionPropertyChanged },
	{ GET_MEMBER_NAME_CHECKED(UCEClonerStepExtension, DeltaStepScale), &UCEClonerStepExtension::OnExtensionPropertyChanged },
};

void UCEClonerStepExtension::PostEditChangeProperty(FPropertyChangedEvent& InPropertyChangedEvent)
{
	Super::PostEditChangeProperty(InPropertyChangedEvent);

	PropertyChangeDispatcher.OnPropertyChanged(this, InPropertyChangedEvent);
}
#endif
