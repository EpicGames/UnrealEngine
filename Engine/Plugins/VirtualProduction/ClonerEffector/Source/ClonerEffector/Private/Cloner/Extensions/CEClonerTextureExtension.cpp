// Copyright Epic Games, Inc. All Rights Reserved.

#include "Cloner/Extensions/CEClonerTextureExtension.h"

#include "Cloner/CEClonerComponent.h"
#include "Cloner/Extensions/CEClonerConstraintExtension.h"
#include "Cloner/Extensions/CEClonerDisplacementExtension.h"
#include "Cloner/Layouts/CEClonerGridLayout.h"
#include "Cloner/Layouts/CEClonerLayoutBase.h"
#include "NiagaraMeshRendererProperties.h"

#if WITH_EDITOR
const TCEPropertyChangeDispatcher<UCEClonerTextureExtension> UCEClonerTextureExtension::PropertyChangeDispatcher =
{
	{ GET_MEMBER_NAME_CHECKED(UCEClonerTextureExtension, bTextureEnabled), &UCEClonerTextureExtension::OnExtensionPropertyChanged },
	{ GET_MEMBER_NAME_CHECKED(UCEClonerTextureExtension, TextureProvider), &UCEClonerTextureExtension::OnExtensionPropertyChanged },
	{ GET_MEMBER_NAME_CHECKED(UCEClonerTextureExtension, CustomTextureAsset), &UCEClonerTextureExtension::OnExtensionPropertyChanged },
	{ GET_MEMBER_NAME_CHECKED(UCEClonerTextureExtension, TextureUVProvider), &UCEClonerTextureExtension::OnExtensionPropertyChanged },
	{ GET_MEMBER_NAME_CHECKED(UCEClonerTextureExtension, CustomTextureUVPlane), &UCEClonerTextureExtension::OnExtensionPropertyChanged },
	{ GET_MEMBER_NAME_CHECKED(UCEClonerTextureExtension, CustomTextureUVOffset), &UCEClonerTextureExtension::OnExtensionPropertyChanged },
	{ GET_MEMBER_NAME_CHECKED(UCEClonerTextureExtension, CustomTextureUVRotation), &UCEClonerTextureExtension::OnExtensionPropertyChanged },
	{ GET_MEMBER_NAME_CHECKED(UCEClonerTextureExtension, CustomTextureUVScale), &UCEClonerTextureExtension::OnExtensionPropertyChanged },
	{ GET_MEMBER_NAME_CHECKED(UCEClonerTextureExtension, bCustomTextureUVClamp), &UCEClonerTextureExtension::OnExtensionPropertyChanged },
};

void UCEClonerTextureExtension::PostEditChangeProperty(FPropertyChangedEvent& InPropertyChangedEvent)
{
	Super::PostEditChangeProperty(InPropertyChangedEvent);

	PropertyChangeDispatcher.OnPropertyChanged(this, InPropertyChangedEvent);
}
#endif

UCEClonerTextureExtension::UCEClonerTextureExtension()
	: UCEClonerExtensionBase(
		TEXT("Texture")
		, 0
	)
{}

void UCEClonerTextureExtension::SetTextureEnabled(bool bInEnabled)
{
	if (bTextureEnabled == bInEnabled)
	{
		return;
	}

	bTextureEnabled = bInEnabled;
	MarkExtensionDirty();
}

void UCEClonerTextureExtension::SetTextureProvider(ECEClonerTextureProvider InProvider)
{
	if (TextureProvider == InProvider)
	{
		return;
	}

	TextureProvider = InProvider;
	MarkExtensionDirty();
}

void UCEClonerTextureExtension::SetCustomTextureAsset(UTexture* InTexture)
{
	if (CustomTextureAsset == InTexture)
	{
		return;
	}

	CustomTextureAsset = InTexture;
	MarkExtensionDirty();
}

void UCEClonerTextureExtension::SetTextureUVProvider(ECEClonerTextureProvider InProvider)
{
	if (TextureUVProvider == InProvider)
	{
		return;
	}

	TextureUVProvider = InProvider;
	MarkExtensionDirty();
}

void UCEClonerTextureExtension::SetCustomTextureUVPlane(ECEClonerPlane InPlane)
{
	if (CustomTextureUVPlane == InPlane)
	{
		return;
	}

	CustomTextureUVPlane = InPlane;
	MarkExtensionDirty();
}

void UCEClonerTextureExtension::SetCustomTextureUVOffset(const FVector2D& InOffset)
{
	if (CustomTextureUVOffset.Equals(InOffset))
	{
		return;
	}

	CustomTextureUVOffset = InOffset;
	MarkExtensionDirty();
}

void UCEClonerTextureExtension::SetCustomTextureUVRotation(float InRotation)
{
	if (FMath::IsNearlyEqual(CustomTextureUVRotation, InRotation))
	{
		return;
	}

	CustomTextureUVRotation = InRotation;
	MarkExtensionDirty();
}

void UCEClonerTextureExtension::SetCustomTextureUVScale(const FVector2D& InScale)
{
	if (CustomTextureUVScale.Equals(InScale))
	{
		return;
	}

	CustomTextureUVScale = InScale;
	MarkExtensionDirty();
}

void UCEClonerTextureExtension::SetCustomTextureUVClamp(bool bInClamp)
{
	if (bCustomTextureUVClamp == bInClamp)
	{
		return;
	}

	bCustomTextureUVClamp = bInClamp;
	MarkExtensionDirty();
}

void UCEClonerTextureExtension::OnExtensionParametersChanged(UCEClonerComponent* InComponent)
{
	Super::OnExtensionParametersChanged(InComponent);

	const UCEClonerLayoutBase* ActiveLayout = InComponent->GetActiveLayout();

	if (!ActiveLayout)
	{
		return;
	}

	UNiagaraMeshRendererProperties* MeshRenderer = ActiveLayout->GetMeshRenderer();

	if (!MeshRenderer)
	{
		return;
	}

	FNiagaraRendererMaterialScalarParameter* TextureEnabledParameter = MeshRenderer->MaterialParameters.ScalarParameters.FindByPredicate(
		[](const FNiagaraRendererMaterialScalarParameter& InMaterialParameter)
		{
			return InMaterialParameter.MaterialParameterName == TEXT("ClonerTextureEnabled");
		}
	);

	if (!TextureEnabledParameter)
	{
		TextureEnabledParameter = &MeshRenderer->MaterialParameters.ScalarParameters.AddDefaulted_GetRef();
		TextureEnabledParameter->MaterialParameterName = TEXT("ClonerTextureEnabled");
	}

	TextureEnabledParameter->Value = bTextureEnabled ? 1.0 : 0.0;

	FNiagaraRendererMaterialTextureParameter* MaterialTextureParameter = MeshRenderer->MaterialParameters.TextureParameters.FindByPredicate(
		[](const FNiagaraRendererMaterialTextureParameter& InMaterialParameter)
		{
			return InMaterialParameter.MaterialParameterName == TEXT("ClonerTexture");
		}
	);

	if (!MaterialTextureParameter)
	{
		MaterialTextureParameter = &MeshRenderer->MaterialParameters.TextureParameters.AddDefaulted_GetRef();
		MaterialTextureParameter->MaterialParameterName = TEXT("ClonerTexture");
	}

	if (TextureProvider == ECEClonerTextureProvider::Constraint)
	{
		if (const UCEClonerConstraintExtension* ConstraintExtension = InComponent->GetExtension<UCEClonerConstraintExtension>())
		{
			MaterialTextureParameter->Texture = ConstraintExtension->GetTextureAsset();
		}
	}
	else if (TextureProvider == ECEClonerTextureProvider::Displacement)
	{
		if (const UCEClonerDisplacementExtension* DisplacementExtension = InComponent->GetExtension<UCEClonerDisplacementExtension>())
		{
			MaterialTextureParameter->Texture = DisplacementExtension->GetDisplacementTextureAsset();
		}
	}
	else
	{
		MaterialTextureParameter->Texture = CustomTextureAsset;
	}

	FNiagaraUserRedirectionParameterStore& ExposedParameters = InComponent->GetOverrideParameters();

	InComponent->SetVariableInt(TEXT("TextureUVIndex"), static_cast<int32>(TextureUVProvider));

	static const FNiagaraVariable TexturePlaneVar(FNiagaraTypeDefinition(StaticEnum<ECEClonerPlane>()), TEXT("CustomTextureUVPlane"));
	ExposedParameters.SetParameterValue<int32>(static_cast<int32>(CustomTextureUVPlane), TexturePlaneVar);
	
	InComponent->SetVariableVec2(TEXT("CustomTextureUVOffset"), CustomTextureUVOffset);

	InComponent->SetFloatParameter(TEXT("CustomTextureUVRotation"), CustomTextureUVRotation);

	InComponent->SetVariableVec2(TEXT("CustomTextureUVScale"), CustomTextureUVScale);

	InComponent->SetVariableBool(TEXT("CustomTextureUVClamp"), bCustomTextureUVClamp);
}

bool UCEClonerTextureExtension::IsLayoutSupported(const UCEClonerLayoutBase* InLayout) const
{
	return InLayout->IsA<UCEClonerGridLayout>();
}

void UCEClonerTextureExtension::OnExtensionDirtied(const UCEClonerExtensionBase* InExtension)
{
	if (TextureProvider == ECEClonerTextureProvider::Displacement
		|| TextureProvider == ECEClonerTextureProvider::Constraint
		|| TextureUVProvider == ECEClonerTextureProvider::Displacement
		|| TextureUVProvider == ECEClonerTextureProvider::Constraint)
	{
		if (InExtension->IsA<UCEClonerConstraintExtension>()
			|| InExtension->IsA<UCEClonerDisplacementExtension>())
		{
			MarkExtensionDirty();
		}
	}
}
