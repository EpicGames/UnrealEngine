// Copyright Epic Games, Inc. All Rights Reserved.

#include "Cloner/Layouts/CEClonerCircleLayout.h"

#include "Cloner/CEClonerComponent.h"

void UCEClonerCircleLayout::SetCount(int32 InCount)
{
	if (Count == InCount)
	{
		return;
	}

	Count = InCount;
	MarkLayoutDirty();
}

void UCEClonerCircleLayout::SetRingCount(int32 InCount)
{
	InCount = FMath::Max(1, InCount);
	if (RingCount == InCount)
	{
		return;
	}

	RingCount = InCount;
	MarkLayoutDirty();
}

void UCEClonerCircleLayout::SetRadius(float InRadius)
{
	if (Radius == InRadius)
	{
		return;
	}

	Radius = InRadius;
	MarkLayoutDirty();
}

void UCEClonerCircleLayout::SetAngleStart(float InAngleStart)
{
	if (AngleStart == InAngleStart)
	{
		return;
	}

	AngleStart = InAngleStart;
	MarkLayoutDirty();
}

void UCEClonerCircleLayout::SetAngleRatio(float InAngleRatio)
{
	if (AngleRatio == InAngleRatio)
	{
		return;
	}

	AngleRatio = InAngleRatio;
	MarkLayoutDirty();
}

void UCEClonerCircleLayout::SetOrientMesh(bool bInOrientMesh)
{
	if (bOrientMesh == bInOrientMesh)
	{
		return;
	}

	bOrientMesh = bInOrientMesh;
	MarkLayoutDirty();
}

void UCEClonerCircleLayout::SetPlane(ECEClonerPlane InPlane)
{
	if (Plane == InPlane)
	{
		return;
	}

	Plane = InPlane;
	MarkLayoutDirty();
}

void UCEClonerCircleLayout::SetRotation(const FRotator& InRotation)
{
	if (Rotation == InRotation)
	{
		return;
	}

	Rotation = InRotation;
	MarkLayoutDirty();
}

void UCEClonerCircleLayout::SetScale(const FVector& InScale)
{
	if (Scale == InScale)
	{
		return;
	}

	Scale = InScale;
	MarkLayoutDirty();
}

#if WITH_EDITOR
const TCEPropertyChangeDispatcher<UCEClonerCircleLayout> UCEClonerCircleLayout::PropertyChangeDispatcher =
{
	{ GET_MEMBER_NAME_CHECKED(UCEClonerCircleLayout, Count), &UCEClonerCircleLayout::OnLayoutPropertyChanged },
	{ GET_MEMBER_NAME_CHECKED(UCEClonerCircleLayout, RingCount), &UCEClonerCircleLayout::OnLayoutPropertyChanged },
	{ GET_MEMBER_NAME_CHECKED(UCEClonerCircleLayout, Radius), &UCEClonerCircleLayout::OnLayoutPropertyChanged },
	{ GET_MEMBER_NAME_CHECKED(UCEClonerCircleLayout, AngleStart), &UCEClonerCircleLayout::OnLayoutPropertyChanged },
	{ GET_MEMBER_NAME_CHECKED(UCEClonerCircleLayout, AngleRatio), &UCEClonerCircleLayout::OnLayoutPropertyChanged },
	{ GET_MEMBER_NAME_CHECKED(UCEClonerCircleLayout, bOrientMesh), &UCEClonerCircleLayout::OnLayoutPropertyChanged },
	{ GET_MEMBER_NAME_CHECKED(UCEClonerCircleLayout, Plane), &UCEClonerCircleLayout::OnLayoutPropertyChanged },
	{ GET_MEMBER_NAME_CHECKED(UCEClonerCircleLayout, Rotation), &UCEClonerCircleLayout::OnLayoutPropertyChanged },
	{ GET_MEMBER_NAME_CHECKED(UCEClonerCircleLayout, Scale), &UCEClonerCircleLayout::OnLayoutPropertyChanged },
};

void UCEClonerCircleLayout::PostEditChangeProperty(FPropertyChangedEvent& InPropertyChangedEvent)
{
	Super::PostEditChangeProperty(InPropertyChangedEvent);

	PropertyChangeDispatcher.OnPropertyChanged(this, InPropertyChangedEvent);
}
#endif

void UCEClonerCircleLayout::OnLayoutParametersChanged(UCEClonerComponent* InComponent)
{
	Super::OnLayoutParametersChanged(InComponent);

	InComponent->SetIntParameter(TEXT("CircleCount"), Count);

	InComponent->SetIntParameter(TEXT("CircleRingCount"), RingCount);

	InComponent->SetFloatParameter(TEXT("CircleRadius"), Radius);

	InComponent->SetFloatParameter(TEXT("CircleStart"), AngleStart);

	InComponent->SetFloatParameter(TEXT("CircleRatio"), AngleRatio);

	InComponent->SetBoolParameter(TEXT("MeshOrientAxisEnable"), bOrientMesh);

	FVector CircleRotation(Rotation.Yaw, Rotation.Pitch, Rotation.Roll);

	if (Plane == ECEClonerPlane::XY)
	{
		CircleRotation = FVector(0);
	}
	else if (Plane == ECEClonerPlane::YZ)
	{
		CircleRotation = FVector(0, 90, 0);
	}
	else if (Plane == ECEClonerPlane::XZ)
	{
		CircleRotation = FVector(0, 0, 90);
	}

	InComponent->SetVectorParameter(TEXT("CircleRotation"), CircleRotation);

	InComponent->SetVectorParameter(TEXT("CircleScale"), Scale);
}
