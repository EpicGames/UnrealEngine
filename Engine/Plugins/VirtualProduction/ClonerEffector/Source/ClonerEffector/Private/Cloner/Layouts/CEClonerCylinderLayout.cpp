// Copyright Epic Games, Inc. All Rights Reserved.

#include "Cloner/Layouts/CEClonerCylinderLayout.h"

#include "Cloner/CEClonerComponent.h"

void UCEClonerCylinderLayout::SetBaseCount(int32 InBaseCount)
{
	if (BaseCount == InBaseCount)
	{
		return;
	}

	BaseCount = InBaseCount;
	MarkLayoutDirty();
}

void UCEClonerCylinderLayout::SetHeightCount(int32 InHeightCount)
{
	if (HeightCount == InHeightCount)
	{
		return;
	}

	HeightCount = InHeightCount;
	MarkLayoutDirty();
}

void UCEClonerCylinderLayout::SetHeight(float InHeight)
{
	if (Height == InHeight)
	{
		return;
	}

	Height = InHeight;
	MarkLayoutDirty();
}

void UCEClonerCylinderLayout::SetRadius(float InRadius)
{
	if (Radius == InRadius)
	{
		return;
	}

	Radius = InRadius;
	MarkLayoutDirty();
}

void UCEClonerCylinderLayout::SetAngleStart(float InAngleStart)
{
	if (AngleStart == InAngleStart)
	{
		return;
	}

	AngleStart = InAngleStart;
	MarkLayoutDirty();
}

void UCEClonerCylinderLayout::SetAngleRatio(float InAngleRatio)
{
	if (AngleRatio == InAngleRatio)
	{
		return;
	}

	AngleRatio = InAngleRatio;
	MarkLayoutDirty();
}

void UCEClonerCylinderLayout::SetOrientMesh(bool bInOrientMesh)
{
	if (bOrientMesh == bInOrientMesh)
	{
		return;
	}

	bOrientMesh = bInOrientMesh;
	MarkLayoutDirty();
}

void UCEClonerCylinderLayout::SetPlane(ECEClonerPlane InPlane)
{
	if (Plane == InPlane)
	{
		return;
	}

	Plane = InPlane;
	MarkLayoutDirty();
}

void UCEClonerCylinderLayout::SetRotation(const FRotator& InRotation)
{
	if (Rotation == InRotation)
	{
		return;
	}

	Rotation = InRotation;
	MarkLayoutDirty();
}

void UCEClonerCylinderLayout::SetScale(const FVector& InScale)
{
	if (Scale == InScale)
	{
		return;
	}

	Scale = InScale;
	MarkLayoutDirty();
}

#if WITH_EDITOR
const TCEPropertyChangeDispatcher<UCEClonerCylinderLayout> UCEClonerCylinderLayout::PropertyChangeDispatcher =
{
	{ GET_MEMBER_NAME_CHECKED(UCEClonerCylinderLayout, BaseCount), &UCEClonerCylinderLayout::OnLayoutPropertyChanged },
	{ GET_MEMBER_NAME_CHECKED(UCEClonerCylinderLayout, HeightCount), &UCEClonerCylinderLayout::OnLayoutPropertyChanged },
	{ GET_MEMBER_NAME_CHECKED(UCEClonerCylinderLayout, Height), &UCEClonerCylinderLayout::OnLayoutPropertyChanged },
	{ GET_MEMBER_NAME_CHECKED(UCEClonerCylinderLayout, Radius), &UCEClonerCylinderLayout::OnLayoutPropertyChanged },
	{ GET_MEMBER_NAME_CHECKED(UCEClonerCylinderLayout, AngleStart), &UCEClonerCylinderLayout::OnLayoutPropertyChanged },
	{ GET_MEMBER_NAME_CHECKED(UCEClonerCylinderLayout, AngleRatio), &UCEClonerCylinderLayout::OnLayoutPropertyChanged },
	{ GET_MEMBER_NAME_CHECKED(UCEClonerCylinderLayout, bOrientMesh), &UCEClonerCylinderLayout::OnLayoutPropertyChanged },
	{ GET_MEMBER_NAME_CHECKED(UCEClonerCylinderLayout, Plane), &UCEClonerCylinderLayout::OnLayoutPropertyChanged },
	{ GET_MEMBER_NAME_CHECKED(UCEClonerCylinderLayout, Rotation), &UCEClonerCylinderLayout::OnLayoutPropertyChanged },
	{ GET_MEMBER_NAME_CHECKED(UCEClonerCylinderLayout, Scale), &UCEClonerCylinderLayout::OnLayoutPropertyChanged },
};

void UCEClonerCylinderLayout::PostEditChangeProperty(FPropertyChangedEvent& InPropertyChangedEvent)
{
	Super::PostEditChangeProperty(InPropertyChangedEvent);

	PropertyChangeDispatcher.OnPropertyChanged(this, InPropertyChangedEvent);
}
#endif

void UCEClonerCylinderLayout::OnLayoutParametersChanged(UCEClonerComponent* InComponent)
{
	Super::OnLayoutParametersChanged(InComponent);

	InComponent->SetIntParameter(TEXT("CylinderBaseCount"), BaseCount);

	InComponent->SetIntParameter(TEXT("CylinderHeightCount"), HeightCount);

	InComponent->SetFloatParameter(TEXT("CylinderHeight"), Height);

	InComponent->SetFloatParameter(TEXT("CylinderRadius"), Radius);

	InComponent->SetFloatParameter(TEXT("CylinderRatio"), AngleRatio);

	InComponent->SetFloatParameter(TEXT("CylinderStart"), AngleStart);

	InComponent->SetBoolParameter(TEXT("MeshOrientAxisEnable"), bOrientMesh);

	FVector CylinderRotation(Rotation.Yaw, Rotation.Pitch, Rotation.Roll);

	if (Plane == ECEClonerPlane::XY)
	{
		CylinderRotation = FVector(0);
	}
	else if (Plane == ECEClonerPlane::YZ)
	{
		CylinderRotation = FVector(0, 90, 0);
	}
	else if (Plane == ECEClonerPlane::XZ)
	{
		CylinderRotation = FVector(0, 0, 90);
	}

	InComponent->SetVectorParameter(TEXT("CylinderRotation"), CylinderRotation);

	InComponent->SetVectorParameter(TEXT("CylinderScale"), Scale);
}
