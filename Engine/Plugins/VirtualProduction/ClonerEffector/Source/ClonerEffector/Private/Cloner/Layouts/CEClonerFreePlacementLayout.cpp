// Copyright Epic Games, Inc. All Rights Reserved.

#include "Cloner/Layouts/CEClonerFreePlacementLayout.h"

#include "Cloner/CEClonerComponent.h"
#include "NiagaraDataInterfaceArrayFloat.h"

#if WITH_EDITOR
const TCEPropertyChangeDispatcher<UCEClonerFreePlacementLayout> UCEClonerFreePlacementLayout::PropertyChangeDispatcher =
{
};

void UCEClonerFreePlacementLayout::PostEditChangeProperty(FPropertyChangedEvent& InPropertyChangedEvent)
{
	Super::PostEditChangeProperty(InPropertyChangedEvent);

	PropertyChangeDispatcher.OnPropertyChanged(this, InPropertyChangedEvent);
}
#endif

void UCEClonerFreePlacementLayout::OnLayoutActive()
{
	Super::OnLayoutActive();

	if (UCEClonerComponent* ClonerComponent = GetClonerComponent())
	{
		UCEClonerComponent::OnClonerMeshUpdated().AddUObject(this, &UCEClonerFreePlacementLayout::OnClonerMeshUpdated);

#if WITH_EDITOR
		UCEClonerComponent::OnClonerActorAttached().AddUObject(this, &UCEClonerFreePlacementLayout::OnClonerActorAttached);
		UCEClonerComponent::OnClonerActorDetached().AddUObject(this, &UCEClonerFreePlacementLayout::OnClonerActorDetached);
		
		// Disable selection in order to select the underlying cloned actors to move them easily
		ClonerComponent->bSelectable = false;

		// Render in custom pass to select actor without rendering its geometry
		ApplyComponentsSettings();
#endif
	}
}

void UCEClonerFreePlacementLayout::OnLayoutInactive()
{
	Super::OnLayoutInactive();
	
	if (UCEClonerComponent* ClonerComponent = GetClonerComponent())
	{
		UCEClonerComponent::OnClonerMeshUpdated().RemoveAll(this);
		
#if WITH_EDITOR
		UCEClonerComponent::OnClonerActorAttached().RemoveAll(this);
		UCEClonerComponent::OnClonerActorDetached().RemoveAll(this);
		
		ClonerComponent->bSelectable = true;

		RestoreComponentsSettings();
#endif
	}
}

void UCEClonerFreePlacementLayout::OnLayoutParametersChanged(UCEClonerComponent* InComponent)
{
	Super::OnLayoutParametersChanged(InComponent);

	const AActor* ClonerActor = GetClonerActor();

	if (!ClonerActor)
	{
		return;
	}

	const FNiagaraUserRedirectionParameterStore& ExposedParameters = InComponent->GetOverrideParameters();
	
	static const FNiagaraVariable SpawnPositionsVar(FNiagaraTypeDefinition(UNiagaraDataInterfaceArrayFloat3::StaticClass()), TEXT("SpawnPositions"));
	UNiagaraDataInterfaceArrayFloat3* PositionsDI = Cast<UNiagaraDataInterfaceArrayFloat3>(ExposedParameters.GetDataInterface(SpawnPositionsVar));

	if (!PositionsDI)
	{
		return;
	}
	
	const TArray<AActor*> RootActors = InComponent->GetClonerRootActors();

	TArray<FVector3f>& Positions = PositionsDI->GetArrayReference();
	Positions.Empty(RootActors.Num());

	// Position relative to cloner actor
	const FTransform ClonerTransform = ClonerActor->GetActorTransform();
	for (const AActor* RootActor : RootActors)
	{
		FVector3f Position = FVector3f::ZeroVector;

		if (RootActor)
		{
			Position = FVector3f(RootActor->GetActorTransform().GetRelativeTransform(ClonerTransform).GetLocation());
		}
		
		Positions.Add(Position);
	}
}

void UCEClonerFreePlacementLayout::OnClonerMeshUpdated(UCEClonerComponent* InClonerComponent)
{
	if (InClonerComponent->GetActiveLayout() != this)
	{
		return;
	}
	
	MarkLayoutDirty();
}

#if WITH_EDITOR
void UCEClonerFreePlacementLayout::OnClonerActorAttached(UCEClonerComponent* InClonerComponent, AActor* InActor)
{
	if (InClonerComponent->GetActiveLayout() != this)
	{
		return;
	}

	ApplyComponentsSettings(InActor);
}

void UCEClonerFreePlacementLayout::OnClonerActorDetached(UCEClonerComponent* InClonerComponent, AActor* InActor)
{
	if (InClonerComponent->GetActiveLayout() != this)
	{
		return;
	}

	RestoreComponentsSettings(InActor);
}

void UCEClonerFreePlacementLayout::ApplyComponentsSettings()
{
	const AActor* ClonerActor = GetClonerActor();

	if (!IsValid(ClonerActor))
	{
		return;
	}

	TArray<AActor*> AttachedActors;
	ClonerActor->GetAttachedActors(AttachedActors, /** Reset */true, /** IncludeChildren */true);

	for (AActor* AttachedActor : AttachedActors)
	{
		ApplyComponentsSettings(AttachedActor);
	}
}

void UCEClonerFreePlacementLayout::RestoreComponentsSettings()
{
	const AActor* ClonerActor = GetClonerActor();

	if (!IsValid(ClonerActor))
	{
		return;
	}

	TArray<AActor*> AttachedActors;
	ClonerActor->GetAttachedActors(AttachedActors, /** Reset */true, /** IncludeChildren */true);

	for (AActor* AttachedActor : AttachedActors)
	{
		if (AttachedActor)
		{
			AttachedActor->SetIsTemporarilyHiddenInEditor(true);
			RestoreComponentsSettings(AttachedActor);
		}
	}
}

void UCEClonerFreePlacementLayout::ApplyComponentsSettings(AActor* InActor)
{
	if (!IsValid(InActor))
	{
		return;
	}
	
	InActor->SetIsTemporarilyHiddenInEditor(false);

	TArray<UPrimitiveComponent*> PrimitiveComponents;
	InActor->GetComponents(PrimitiveComponents);

	for (UPrimitiveComponent* PrimitiveComponent : PrimitiveComponents)
	{
		if (PrimitiveComponent)
		{
			PrimitiveComponent->SetRenderInMainPass(false);
			PrimitiveComponent->SetRenderInDepthPass(false);
			PrimitiveComponent->SetRenderCustomDepth(true);
		}
	}
}

void UCEClonerFreePlacementLayout::RestoreComponentsSettings(AActor* InActor)
{
	if (!IsValid(InActor))
	{
		return;
	}
	
	TArray<UPrimitiveComponent*> PrimitiveComponents;
	InActor->GetComponents(PrimitiveComponents);

	for (UPrimitiveComponent* PrimitiveComponent : PrimitiveComponents)
	{
		if (PrimitiveComponent)
		{
			PrimitiveComponent->SetRenderInMainPass(true);
			PrimitiveComponent->SetRenderInDepthPass(true);
			PrimitiveComponent->SetRenderCustomDepth(false);
		}
	}
}
#endif