// Copyright Epic Games, Inc. All Rights Reserved.

#include "Cloner/Layouts/CEClonerGridLayout.h"

#include "Cloner/CEClonerComponent.h"
#include "NiagaraSystem.h"

void UCEClonerGridLayout::SetCountX(int32 InCountX)
{
	if (CountX == InCountX)
	{
		return;
	}

	CountX = InCountX;
	MarkLayoutDirty();
}

void UCEClonerGridLayout::SetCountY(int32 InCountY)
{
	if (CountY == InCountY)
	{
		return;
	}

	CountY = InCountY;
	MarkLayoutDirty();
}

void UCEClonerGridLayout::SetCountZ(int32 InCountZ)
{
	if (CountZ == InCountZ)
	{
		return;
	}

	CountZ = InCountZ;
	MarkLayoutDirty();
}

void UCEClonerGridLayout::SetSpacingX(float InSpacingX)
{
	if (SpacingX == InSpacingX)
	{
		return;
	}

	SpacingX = InSpacingX;
	MarkLayoutDirty();
}

void UCEClonerGridLayout::SetSpacingY(float InSpacingY)
{
	if (SpacingY == InSpacingY)
	{
		return;
	}

	SpacingY = InSpacingY;
	MarkLayoutDirty();
}

void UCEClonerGridLayout::SetSpacingZ(float InSpacingZ)
{
	if (SpacingZ == InSpacingZ)
	{
		return;
	}

	SpacingZ = InSpacingZ;
	MarkLayoutDirty();
}

void UCEClonerGridLayout::SetTwistFactor(float InFactor)
{
	if (TwistFactor == InFactor)
	{
		return;
	}

	TwistFactor = InFactor;
	MarkLayoutDirty();
}

void UCEClonerGridLayout::SetTwistAxis(ECEClonerAxis InAxis)
{
	if (TwistAxis == InAxis || InAxis == ECEClonerAxis::Custom)
	{
		return;
	}

	TwistAxis = InAxis;
	MarkLayoutDirty();
}

#if WITH_EDITOR
const TCEPropertyChangeDispatcher<UCEClonerGridLayout> UCEClonerGridLayout::PropertyChangeDispatcher =
{
	{ GET_MEMBER_NAME_CHECKED(UCEClonerGridLayout, CountX), &UCEClonerGridLayout::OnLayoutPropertyChanged },
	{ GET_MEMBER_NAME_CHECKED(UCEClonerGridLayout, CountY), &UCEClonerGridLayout::OnLayoutPropertyChanged },
	{ GET_MEMBER_NAME_CHECKED(UCEClonerGridLayout, CountZ), &UCEClonerGridLayout::OnLayoutPropertyChanged },
	{ GET_MEMBER_NAME_CHECKED(UCEClonerGridLayout, SpacingX), &UCEClonerGridLayout::OnLayoutPropertyChanged },
	{ GET_MEMBER_NAME_CHECKED(UCEClonerGridLayout, SpacingY), &UCEClonerGridLayout::OnLayoutPropertyChanged },
	{ GET_MEMBER_NAME_CHECKED(UCEClonerGridLayout, SpacingZ), &UCEClonerGridLayout::OnLayoutPropertyChanged },
	{ GET_MEMBER_NAME_CHECKED(UCEClonerGridLayout, TwistFactor), &UCEClonerGridLayout::OnLayoutPropertyChanged },
	{ GET_MEMBER_NAME_CHECKED(UCEClonerGridLayout, TwistAxis), &UCEClonerGridLayout::OnLayoutPropertyChanged },
};

void UCEClonerGridLayout::PostEditChangeProperty(FPropertyChangedEvent& InPropertyChangedEvent)
{
	Super::PostEditChangeProperty(InPropertyChangedEvent);

	PropertyChangeDispatcher.OnPropertyChanged(this, InPropertyChangedEvent);
}
#endif

void UCEClonerGridLayout::OnLayoutParametersChanged(UCEClonerComponent* InComponent)
{
	Super::OnLayoutParametersChanged(InComponent);

	InComponent->SetIntParameter(TEXT("GridCountX"), CountX);

	InComponent->SetIntParameter(TEXT("GridCountY"), CountY);

	InComponent->SetIntParameter(TEXT("GridCountZ"), CountZ);

	InComponent->SetVectorParameter(TEXT("GridSpacing"), FVector(SpacingX, SpacingY, SpacingZ));

	InComponent->SetFloatParameter(TEXT("TwistFactor"), TwistFactor / 100.f);

	FNiagaraUserRedirectionParameterStore& ExposedParameters = InComponent->GetOverrideParameters();
	static const FNiagaraVariable TwistAxisVar(FNiagaraTypeDefinition(StaticEnum<ENiagaraOrientationAxis>()), TEXT("TwistAxis"));
	ExposedParameters.SetParameterValue<int32>(static_cast<int32>(TwistAxis), TwistAxisVar);
}
