// Copyright Epic Games, Inc. All Rights Reserved.

#include "Cloner/Layouts/CEClonerHoneycombLayout.h"

#include "Cloner/CEClonerComponent.h"
#include "NiagaraSystem.h"

void UCEClonerHoneycombLayout::SetPlane(ECEClonerPlane InPlane)
{
	if (Plane == ECEClonerPlane::Custom)
	{
		return;
	}

	if (Plane == InPlane)
	{
		return;
	}

	Plane = InPlane;
	OnTwistAxisChanged();
}

void UCEClonerHoneycombLayout::SetWidthCount(int32 InWidthCount)
{
	if (WidthCount == InWidthCount)
	{
		return;
	}

	WidthCount = InWidthCount;
	MarkLayoutDirty();
}

void UCEClonerHoneycombLayout::SetHeightCount(int32 InHeightCount)
{
	if (HeightCount == InHeightCount)
	{
		return;
	}

	HeightCount = InHeightCount;
	MarkLayoutDirty();
}

void UCEClonerHoneycombLayout::SetWidthOffset(float InWidthOffset)
{
	if (WidthOffset == InWidthOffset)
	{
		return;
	}

	WidthOffset = InWidthOffset;
	MarkLayoutDirty();
}

void UCEClonerHoneycombLayout::SetHeightOffset(float InHeightOffset)
{
	if (HeightOffset == InHeightOffset)
	{
		return;
	}

	HeightOffset = InHeightOffset;
	MarkLayoutDirty();
}

void UCEClonerHoneycombLayout::SetHeightSpacing(float InHeightSpacing)
{
	if (HeightSpacing == InHeightSpacing)
	{
		return;
	}

	HeightSpacing = InHeightSpacing;
	MarkLayoutDirty();
}

void UCEClonerHoneycombLayout::SetWidthSpacing(float InWidthSpacing)
{
	if (WidthSpacing == InWidthSpacing)
	{
		return;
	}

	WidthSpacing = InWidthSpacing;
	MarkLayoutDirty();
}

void UCEClonerHoneycombLayout::SetTwistFactor(float InFactor)
{
	if (TwistFactor == InFactor)
	{
		return;
	}

	TwistFactor = InFactor;
	MarkLayoutDirty();
}

void UCEClonerHoneycombLayout::SetTwistAxis(ECEClonerAxis InAxis)
{
	if (TwistAxis == InAxis || InAxis == ECEClonerAxis::Custom)
	{
		return;
	}

	TwistAxis = InAxis;
	OnTwistAxisChanged();
}

#if WITH_EDITOR
const TCEPropertyChangeDispatcher<UCEClonerHoneycombLayout> UCEClonerHoneycombLayout::PropertyChangeDispatcher =
{
	{ GET_MEMBER_NAME_CHECKED(UCEClonerHoneycombLayout, Plane), &UCEClonerHoneycombLayout::OnTwistAxisChanged },
	{ GET_MEMBER_NAME_CHECKED(UCEClonerHoneycombLayout, WidthCount), &UCEClonerHoneycombLayout::OnLayoutPropertyChanged },
	{ GET_MEMBER_NAME_CHECKED(UCEClonerHoneycombLayout, HeightCount), &UCEClonerHoneycombLayout::OnLayoutPropertyChanged },
	{ GET_MEMBER_NAME_CHECKED(UCEClonerHoneycombLayout, WidthOffset), &UCEClonerHoneycombLayout::OnLayoutPropertyChanged },
	{ GET_MEMBER_NAME_CHECKED(UCEClonerHoneycombLayout, HeightOffset), &UCEClonerHoneycombLayout::OnLayoutPropertyChanged },
	{ GET_MEMBER_NAME_CHECKED(UCEClonerHoneycombLayout, WidthSpacing), &UCEClonerHoneycombLayout::OnLayoutPropertyChanged },
	{ GET_MEMBER_NAME_CHECKED(UCEClonerHoneycombLayout, HeightSpacing), &UCEClonerHoneycombLayout::OnLayoutPropertyChanged },
	{ GET_MEMBER_NAME_CHECKED(UCEClonerHoneycombLayout, TwistFactor), &UCEClonerHoneycombLayout::OnLayoutPropertyChanged },
	{ GET_MEMBER_NAME_CHECKED(UCEClonerHoneycombLayout, TwistAxis), &UCEClonerHoneycombLayout::OnTwistAxisChanged },
};

void UCEClonerHoneycombLayout::PostEditChangeProperty(FPropertyChangedEvent& InPropertyChangedEvent)
{
	Super::PostEditChangeProperty(InPropertyChangedEvent);

	PropertyChangeDispatcher.OnPropertyChanged(this, InPropertyChangedEvent);
}
#endif

void UCEClonerHoneycombLayout::OnTwistAxisChanged()
{
	// Restrict twist axis to plane axis
	if (Plane == ECEClonerPlane::XY)
	{
		if (TwistAxis == ECEClonerAxis::Z)
		{
			TwistAxis = ECEClonerAxis::X;
		}
	}
	else if (Plane == ECEClonerPlane::XZ)
	{
		if (TwistAxis == ECEClonerAxis::Y)
		{
			TwistAxis = ECEClonerAxis::X;
		}
	}
	else if (Plane == ECEClonerPlane::YZ)
	{
		if (TwistAxis == ECEClonerAxis::X)
		{
			TwistAxis = ECEClonerAxis::Y;
		}
	}

	MarkLayoutDirty();
}

void UCEClonerHoneycombLayout::OnLayoutParametersChanged(UCEClonerComponent* InComponent)
{
	Super::OnLayoutParametersChanged(InComponent);

	InComponent->SetIntParameter(TEXT("HoneycombWidthCount"), WidthCount);

	InComponent->SetIntParameter(TEXT("HoneycombHeightCount"), HeightCount);

	InComponent->SetFloatParameter(TEXT("HoneycombWidthOffset"), WidthOffset);

	InComponent->SetFloatParameter(TEXT("HoneycombHeightOffset"), HeightOffset);

	InComponent->SetFloatParameter(TEXT("HoneycombWidthSpacing"), WidthSpacing);

	InComponent->SetFloatParameter(TEXT("HoneycombHeightSpacing"), HeightSpacing);

	FNiagaraUserRedirectionParameterStore& ExposedParameters = InComponent->GetOverrideParameters();
	static const FNiagaraVariable HoneycombPlaneVar(FNiagaraTypeDefinition(StaticEnum<ECEClonerPlane>()), TEXT("HoneycombPlane"));
	ExposedParameters.SetParameterValue<int32>(static_cast<int32>(Plane), HoneycombPlaneVar);

	InComponent->SetFloatParameter(TEXT("TwistFactor"), TwistFactor / 100.f);

	static const FNiagaraVariable TwistAxisVar(FNiagaraTypeDefinition(StaticEnum<ENiagaraOrientationAxis>()), TEXT("TwistAxis"));
	ExposedParameters.SetParameterValue<int32>(static_cast<int32>(TwistAxis), TwistAxisVar);
}
