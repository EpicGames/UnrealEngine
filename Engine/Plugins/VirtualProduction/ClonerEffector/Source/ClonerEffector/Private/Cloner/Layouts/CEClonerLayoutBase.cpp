// Copyright Epic Games, Inc. All Rights Reserved.

#include "Cloner/Layouts/CEClonerLayoutBase.h"

#include "Async/Async.h"
#include "Cloner/CEClonerActor.h"
#include "Cloner/CEClonerComponent.h"
#include "Cloner/Extensions/CEClonerExtensionBase.h"
#include "Cloner/Logs/CEClonerLogs.h"
#include "DataInterface/NiagaraDataInterfaceArrayMesh.h"
#include "Engine/Level.h"
#include "Engine/World.h"
#include "Misc/Guid.h"
#include "Misc/PackageName.h"
#include "NiagaraEmitter.h"
#include "NiagaraFunctionLibrary.h"
#include "NiagaraMeshRendererProperties.h"
#include "NiagaraSystem.h"
#include "Subsystems/CEClonerSubsystem.h"
#include "UObject/Package.h"
#include "UObject/UObjectGlobals.h"

namespace UE::ClonerEffector::Private
{
#if WITH_EDITOR
	TAutoConsoleVariable<bool> CVarUpdateSystemAfterLoad(
		TEXT("ClonerEffector.UpdateSystemAfterLoad"),
		false,
		TEXT("Calls UpdateSystemAfterLoad() on a Cloner Layout's PostLoad, so that this is done in loading time instead of activation time.\n")
		TEXT("Only relevant for -game instances (in editor builds).\n")
		TEXT("Non-editor builds do not require this as they already call this in the system PostLoad."));
#endif
}

bool UCEClonerLayoutBase::IsLayoutValid() const
{
	if (LayoutName.IsNone() || LayoutAssetPath.IsEmpty())
	{
		return false;
	}

	UCEClonerSubsystem* const ClonerSubsystem = UCEClonerSubsystem::Get();
	if (!ClonerSubsystem)
	{
		return false;
	}

	const UNiagaraSystem* const TemplateNiagaraSystem = ClonerSubsystem->FindOrLoadTemplateNiagaraSystem(this);
	const UNiagaraSystem* const BaseNiagaraSystem = ClonerSubsystem->FindOrLoadBaseNiagaraSystem();

	if (!TemplateNiagaraSystem || !BaseNiagaraSystem)
	{
		UE_LOGF(LogCECloner, Warning, "Cloner layout %ls : Template system (%ls) or base system (%ls) is invalid", *LayoutName.ToString(), *LayoutAssetPath, LayoutBaseAssetPath);
		return false;
	}

	// Compare parameters : template should have base parameters
	bool bIsSystemBasedOnBaseAsset = true;

	{
		TArray<FNiagaraVariable> TemplateSystemParameters;
		TemplateNiagaraSystem->GetExposedParameters().GetParameters(TemplateSystemParameters);

		TArray<FNiagaraVariable> BaseSystemParameters;
		BaseNiagaraSystem->GetExposedParameters().GetParameters(BaseSystemParameters);

		for (const FNiagaraVariable& SystemParameter : BaseSystemParameters)
		{
			if (!TemplateSystemParameters.Contains(SystemParameter))
			{
				bIsSystemBasedOnBaseAsset = false;
				UE_LOGF(LogCECloner, Warning, "Cloner layout %ls : Template system (%ls) missing parameter (%ls) from base system (%ls)", *LayoutName.ToString(), *LayoutAssetPath, *SystemParameter.ToString(), LayoutBaseAssetPath);
				break;
			}
		}
	}

	if (!bIsSystemBasedOnBaseAsset)
	{
		UE_LOGF(LogCECloner, Warning, "Cloner layout %ls : Template system (%ls) is not based off base system (%ls)", *LayoutName.ToString(), *LayoutAssetPath, LayoutBaseAssetPath);
	}
#if WITH_EDITOR
	else
	{
		const FString LayoutSystemHash = GetLayoutHash();

		if (LayoutSystemHash.IsEmpty())
		{
			UE_LOGF(LogCECloner, Warning, "Cloner layout %ls : Template system (%ls) hash could not be calculated", *LayoutName.ToString(), *LayoutAssetPath);
			return false;
		}

		UE_LOGF(LogCECloner, Verbose, "Cloner layout %ls : Template system (%ls) hash is %ls", *LayoutName.ToString(), *LayoutAssetPath, *LayoutSystemHash);
	}
#endif

	return bIsSystemBasedOnBaseAsset;
}

bool UCEClonerLayoutBase::IsLayoutLoaded() const
{
	return !IsTemplate() && NiagaraSystem && MeshRenderer;
}

void UCEClonerLayoutBase::LoadLayout()
{
	if (IsLayoutLoaded())
	{
		return;
	}

	// Already being loaded
	if (LoadRequestIdentifier != INDEX_NONE)
	{
		return;
	}

	// System already cached and available with matching version
	if (NiagaraSystem)
	{
		if (IsSystemHashMatching())
		{
			UE_LOGF(LogCECloner, Verbose, "%ls : Cloner layout %ls using cached system %ls", *GetClonerActor()->GetActorNameOrLabel(), *LayoutName.ToString(), *CachedSystemHash)

			CacheMeshRenderer();
			OnSystemLoaded();
			return;
		}
		
		FString LayoutHash;
#if WITH_EDITOR
		LayoutHash = GetLayoutHash();
#endif

		UE_LOGF(LogCECloner, Warning, "%ls : Cloner layout %ls skipping cached system %ls due to hash mismatch %ls", *GetClonerActor()->GetActorNameOrLabel(), *LayoutName.ToString(), *CachedSystemHash, *LayoutHash)

		NiagaraSystem->MarkAsGarbage();
		NiagaraSystem = nullptr;
	}
	else
	{
		CleanOwnedSystem();
	}

	const UCEClonerComponent* ClonerComponent = GetClonerComponent();

	if (!IsValid(ClonerComponent))
	{
		return;
	}

	if (LayoutAssetPath.IsEmpty())
	{
		return;
	}

	// Extract package path
	FString MountedPath = LayoutAssetPath;
	{
		int32 FirstQuoteIndex;
		MountedPath.FindChar('\'', FirstQuoteIndex);

		int32 LastQuoteIndex;
		MountedPath.FindLastChar('\'', LastQuoteIndex);

		if (FirstQuoteIndex != LastQuoteIndex)
		{
			MountedPath = MountedPath.Mid(FirstQuoteIndex + 1, LastQuoteIndex - FirstQuoteIndex - 1);
		}

		MountedPath = FPackageName::ObjectPathToPackageName(MountedPath);
	}

	FPackagePath LayoutPackagePath;
	FPackagePath::TryFromMountedName(MountedPath, LayoutPackagePath);

	FPackagePath CustomPackagePath;
	FPackagePath::TryFromPackageName(TEXT("/Game/Temp/") + GetLayoutName().ToString() + TEXT("_") + FGuid::NewGuid().ToString(), CustomPackagePath);
	CustomPackagePath.SetHeaderExtension(EPackageExtension::Asset);

	UE_LOGF(LogCECloner, Verbose, "%ls : Cloner layout load requested %ls - Template system %ls - Package %ls", *GetClonerActor()->GetActorNameOrLabel(), *LayoutName.ToString(), *LayoutAssetPath, *CustomPackagePath.GetPackageFName().ToString())

	FLoadPackageAsyncOptionalParams Params;
	Params.PackagePriority = INT32_MAX;
	Params.LoadFlags = LOAD_Async | LOAD_MemoryReader | LOAD_DisableCompileOnLoad;
	Params.CustomPackageName = CustomPackagePath.GetPackageFName();
	Params.CompletionDelegate = MakeUnique<FLoadPackageAsyncDelegate>(FLoadPackageAsyncDelegate::CreateUObject(this, &UCEClonerLayoutBase::OnSystemPackageLoaded));

	LoadRequestIdentifier = LoadPackageAsync(LayoutPackagePath, MoveTemp(Params));

	BindCleanupDelegates();
}

bool UCEClonerLayoutBase::UnloadLayout()
{
	if (!IsLayoutLoaded())
	{
		return false;
	}

	// Cannot unload while active
	if (IsLayoutActive())
	{
		return false;
	}

	if (UCEClonerComponent* const ClonerComponent = GetClonerComponent())
	{
		const UNiagaraDataInterfaceArrayMesh* const MeshArrayDI = UE::ClonerEffector::FindMeshArrayDataInterface(ClonerComponent);
		// Avoid redundant operation if MeshData is already empty.
		if (MeshArrayDI && !MeshArrayDI->MeshData.IsEmpty())
		{
			UNiagaraDataInterfaceArrayMesh::SetNiagaraArrayMesh(ClonerComponent, UE::ClonerEffector::MeshArrayName, {});
		}
	}

	// Clear the meshes as the mesh array DI is going to be used. This should be always empty for new systems, but old systems could have lingering meshes here.
	MeshRenderer->Meshes.Empty();
#if WITH_EDITOR
	NiagaraSystem->KillAllActiveCompilations();
#endif
	NiagaraSystem->RemoveFromRoot();

	MeshRenderer = nullptr;

	UE_LOGF(LogCECloner, Verbose, "%ls : Cloner layout unloaded %ls", *GetClonerActor()->GetActorNameOrLabel(), *LayoutName.ToString())

	OnLayoutUnloaded();

	return true;
}

bool UCEClonerLayoutBase::IsLayoutActive() const
{
	const UCEClonerComponent* Component = GetClonerComponent();

	if (!Component)
	{
		return false;
	}

	return IsLayoutLoaded() && Component->GetAsset() == NiagaraSystem;
}

bool UCEClonerLayoutBase::ActivateLayout()
{
	if (IsLayoutActive())
	{
		return false;
	}

	// Load layout first
	if (!IsLayoutLoaded())
	{
		return false;
	}

	UCEClonerComponent* ClonerComponent = GetClonerComponent();

	if (!ClonerComponent)
	{
		return false;
	}

	ClonerComponent->SetAsset(NiagaraSystem);

	UE_LOGF(LogCECloner, Verbose, "%ls : Cloner layout activated %ls", *GetClonerActor()->GetActorNameOrLabel(), *LayoutName.ToString())

	OnLayoutActive();

	return true;
}

bool UCEClonerLayoutBase::DeactivateLayout()
{
	if (!IsLayoutActive())
	{
		return false;
	}

	UCEClonerComponent* ClonerComponent = GetClonerComponent();

	if (!ClonerComponent)
	{
		return false;
	}

	ClonerComponent->GetOverrideParameters().Empty(/** ClearBindings */true);
	ClonerComponent->SetAsset(nullptr);

#if WITH_EDITOR
	NiagaraSystem->KillAllActiveCompilations();
#endif

	UE_LOGF(LogCECloner, Verbose, "%ls : Cloner layout deactivated %ls", *GetClonerActor()->GetActorNameOrLabel(), *LayoutName.ToString())

	OnLayoutInactive();

	return true;
}

TSet<TSubclassOf<UCEClonerExtensionBase>> UCEClonerLayoutBase::GetSupportedExtensions() const
{
	TSet<TSubclassOf<UCEClonerExtensionBase>> ExtensionSupported;

	if (const UCEClonerSubsystem* ClonerSubsystem = UCEClonerSubsystem::Get())
	{
		for (const TSubclassOf<UCEClonerExtensionBase>& ExtensionClass : ClonerSubsystem->GetExtensionClasses())
		{
			const UCEClonerExtensionBase* Extension = ExtensionClass.GetDefaultObject();

			if (!Extension)
			{
				continue;
			}

			// Does the layout supports this extension
			if (!IsExtensionSupported(Extension))
			{
				continue;
			}

			// Does the extension supports this layout
			if (!Extension->IsLayoutSupported(this))
			{
				continue;
			}

			ExtensionSupported.Add(ExtensionClass);
		}
	}

	return ExtensionSupported;
}

bool UCEClonerLayoutBase::IsLayoutDirty() const
{
	return EnumHasAnyFlags(LayoutStatus, ECEClonerSystemStatus::ParametersDirty);
}

void UCEClonerLayoutBase::PostEditImport()
{
	Super::PostEditImport();

	// After cloner duplication in editor, niagara system should not be duplicated but still is,
	// so look for it in outer chain otherwise it will trigger a world GC leak when switching level
	CleanOwnedSystem();

	MarkLayoutDirty();
}

void UCEClonerLayoutBase::PostLoad()
{
	Super::PostLoad();

	if (CachedSystemHash.IsEmpty())
	{
		// After cloner layout load, niagara system should not be loaded since property was transient pre versioning,
		// so look for it in outer chain otherwise it will trigger a world GC leak when switching level
		CleanOwnedSystem();
	}

#if WITH_EDITOR
	if (NiagaraSystem && !GIsEditor && UE::ClonerEffector::Private::CVarUpdateSystemAfterLoad.GetValueOnAnyThread())
	{
		// This is done in loading time to not pay this cost in activation time
		// This is already called in UNiagaraSystem::PostLoad in non-editor builds.
		NiagaraSystem->UpdateSystemAfterLoad();
	}
#endif
}

#if WITH_EDITOR
void UCEClonerLayoutBase::PostEditUndo()
{
	Super::PostEditUndo();

	MarkLayoutDirty();
}
#endif

void UCEClonerLayoutBase::OnLayoutPropertyChanged()
{
	MarkLayoutDirty();
}

void UCEClonerLayoutBase::OnSystemPackageLoaded(const FName& InName, UPackage* InPackage, EAsyncLoadingResult::Type InResult)
{
	NiagaraSystem = InPackage ? Cast<UNiagaraSystem>(InPackage->FindAssetInPackage()) : nullptr;
	LoadRequestIdentifier = INDEX_NONE;

	if (NiagaraSystem)
	{
		InPackage->SetFlags(RF_Transient);
		NiagaraSystem->RemoveFromRoot();
		NiagaraSystem->ClearFlags(RF_Standalone | RF_Public | RF_Transient | RF_Transactional);

		constexpr ERenameFlags RenameFlags = REN_NonTransactional | REN_DontCreateRedirectors | REN_AllowPackageLinkerMismatch;
		if (NiagaraSystem->Rename(nullptr, this, RenameFlags))
		{
#if WITH_EDITOR
			CachedSystemHash = GetLayoutHash();
#endif
			CacheMeshRenderer();
		}

		InPackage->MarkAsGarbage();
	}

	OnSystemLoaded();
}

void UCEClonerLayoutBase::OnSystemLoaded()
{
	const bool bLayoutLoaded = IsLayoutLoaded();

	if (bLayoutLoaded)
	{
		UE_LOGF(LogCECloner, Verbose, "%ls : Cloner layout loaded %ls - Template system %ls", *GetClonerActor()->GetActorNameOrLabel(), *LayoutName.ToString(), *LayoutAssetPath)

		OnLayoutLoaded();
	}
	else
	{
		UE_LOGF(LogCECloner, Warning, "%ls : Cloner layout load failed %ls - Template system %ls", *GetClonerActor()->GetActorNameOrLabel(), *LayoutName.ToString(), *LayoutAssetPath)
	}

	OnClonerLayoutLoadedDelegate.Broadcast(this, bLayoutLoaded);
	OnClonerLayoutLoadedDelegate.Clear();
}

void UCEClonerLayoutBase::CacheMeshRenderer()
{
	if (!NiagaraSystem)
	{
		return;
	}

	for (FNiagaraEmitterHandle& SystemEmitterHandle : NiagaraSystem->GetEmitterHandles())
	{
		if (const FVersionedNiagaraEmitterData* EmitterData = SystemEmitterHandle.GetEmitterData())
		{
			for (UNiagaraRendererProperties* EmitterRenderer : EmitterData->GetRenderers())
			{
				if (UNiagaraMeshRendererProperties* EmitterMeshRenderer = Cast<UNiagaraMeshRendererProperties>(EmitterRenderer))
				{
					MeshRenderer = EmitterMeshRenderer;
					return;
				}
			}
		}
	}
}

void UCEClonerLayoutBase::BindCleanupDelegates()
{
	UnbindCleanupDelegates();

	if (const UCEClonerComponent* ClonerComponent = GetClonerComponent())
	{
		if (ULevel* ClonerLevel = ClonerComponent->GetComponentLevel())
		{
			ClonerLevel->OnCleanupLevel.AddUObject(this, &UCEClonerLayoutBase::OnLevelCleanup);
		}

		FWorldDelegates::OnWorldCleanup.AddUObject(this, &UCEClonerLayoutBase::OnWorldCleanup);
	}
}

void UCEClonerLayoutBase::UnbindCleanupDelegates() const
{
	if (const UCEClonerComponent* ClonerComponent = GetClonerComponent())
	{
		if (ULevel* ClonerLevel = ClonerComponent->GetComponentLevel())
		{
			ClonerLevel->OnCleanupLevel.RemoveAll(this);
		}

		FWorldDelegates::OnWorldCleanup.RemoveAll(this);
	}
}

void UCEClonerLayoutBase::OnWorldCleanup(UWorld* InWorld, bool bInSessionEnded, bool bInCleanupResources)
{
	const AActor* Actor = GetClonerActor();
	if (bInCleanupResources && Actor && Actor->GetWorld() == InWorld)
	{
		OnLevelCleanup();
	}
}

void UCEClonerLayoutBase::OnLevelCleanup()
{
	if (IsLayoutLoaded())
	{
		UE_LOGF(LogCECloner, Verbose, "%ls : Cloner layout cleanup %ls", *GetClonerActor()->GetActorNameOrLabel(), *LayoutName.ToString())

		DeactivateLayout();

		UnloadLayout();
	}

	UnbindCleanupDelegates();
}

void UCEClonerLayoutBase::CleanOwnedSystem() const
{
	TArray<UObject*> OwnedObjects;
	GetObjectsWithOuter(this, OwnedObjects, EGetObjectsFlags::None);

	for (UObject* OwnedObject : OwnedObjects)
	{
		if (OwnedObject && OwnedObject->IsA<UNiagaraSystem>())
		{
			UE_LOGF(LogCECloner, Warning, "%ls : Cloner layout %ls cleaning owned system %ls", *GetClonerActor()->GetActorNameOrLabel(), *LayoutName.ToString(), *OwnedObject->GetName())
			OwnedObject->MarkAsGarbage();
		}
	}
}

#if WITH_EDITOR
FString UCEClonerLayoutBase::GetLayoutHash() const
{
	FString LayoutHash;

	if (UCEClonerSubsystem* const ClonerSubsystem = UCEClonerSubsystem::Get())
	{
		if (const UNiagaraSystem* const TemplateNiagaraSystem = ClonerSubsystem->FindOrLoadTemplateNiagaraSystem(this))
		{
			if (UPackage* Package = TemplateNiagaraSystem->GetPackage())
			{ 
				BytesToHex(Package->GetSavedHash().GetBytes(), sizeof(FIoHash::ByteArray), LayoutHash);
			}
		}
	}

	return LayoutHash;
}
#endif

bool UCEClonerLayoutBase::IsSystemHashMatching() const
{
	return !CachedSystemHash.IsEmpty()
#if WITH_EDITOR
		&& GetLayoutHash().Equals(CachedSystemHash)
#endif
	;
}

UCEClonerComponent* UCEClonerLayoutBase::GetClonerComponent() const
{
	return GetTypedOuter<UCEClonerComponent>();
}

AActor* UCEClonerLayoutBase::GetClonerActor() const
{
	if (const UCEClonerComponent* ClonerComponent = GetClonerComponent())
	{
		return ClonerComponent->GetOwner();
	}

	return nullptr;
}

void UCEClonerLayoutBase::UpdateLayoutParameters()
{
	if (!IsLayoutActive())
	{
		return;
	}

	if (UCEClonerComponent* ClonerComponent = GetClonerComponent())
	{
		if (!ClonerComponent->GetEnabled())
		{
			return;
		}

		OnLayoutParametersChanged(ClonerComponent);

		if (EnumHasAnyFlags(LayoutStatus, ECEClonerSystemStatus::SimulationDirty))
		{
			ClonerComponent->RequestClonerUpdate(/*Immediate*/false);
		}
	}

	LayoutStatus = ECEClonerSystemStatus::UpToDate;
}

void UCEClonerLayoutBase::MarkLayoutDirty(bool bInUpdateCloner)
{
	EnumAddFlags(LayoutStatus, ECEClonerSystemStatus::ParametersDirty);

	if (bInUpdateCloner)
	{
		EnumAddFlags(LayoutStatus, ECEClonerSystemStatus::SimulationDirty);
	}
}
