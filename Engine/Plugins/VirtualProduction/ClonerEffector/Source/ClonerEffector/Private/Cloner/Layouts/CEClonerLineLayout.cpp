// Copyright Epic Games, Inc. All Rights Reserved.

#include "Cloner/Layouts/CEClonerLineLayout.h"

#include "Cloner/CEClonerComponent.h"

void UCEClonerLineLayout::SetCount(int32 InCount)
{
	if (Count == InCount)
	{
		return;
	}

	Count = InCount;
	MarkLayoutDirty();
}

void UCEClonerLineLayout::SetSpacing(float InSpacing)
{
	if (Spacing == InSpacing)
	{
		return;
	}

	Spacing = InSpacing;
	MarkLayoutDirty();
}

void UCEClonerLineLayout::SetAxis(ECEClonerAxis InAxis)
{
	if (Axis == InAxis)
	{
		return;
	}

	Axis = InAxis;
	MarkLayoutDirty();
}

void UCEClonerLineLayout::SetDirection(const FVector& InDirection)
{
	if (Direction == InDirection)
	{
		return;
	}

	Direction = InDirection;
	MarkLayoutDirty();
}

void UCEClonerLineLayout::SetRotation(const FRotator& InRotation)
{
	if (Rotation == InRotation)
	{
		return;
	}

	Rotation = InRotation;
	MarkLayoutDirty();
}

#if WITH_EDITOR
const TCEPropertyChangeDispatcher<UCEClonerLineLayout> UCEClonerLineLayout::PropertyChangeDispatcher =
{
	{ GET_MEMBER_NAME_CHECKED(UCEClonerLineLayout, Count), &UCEClonerLineLayout::OnLayoutPropertyChanged },
	{ GET_MEMBER_NAME_CHECKED(UCEClonerLineLayout, Spacing), &UCEClonerLineLayout::OnLayoutPropertyChanged },
	{ GET_MEMBER_NAME_CHECKED(UCEClonerLineLayout, Axis), &UCEClonerLineLayout::OnLayoutPropertyChanged },
	{ GET_MEMBER_NAME_CHECKED(UCEClonerLineLayout, Direction), &UCEClonerLineLayout::OnLayoutPropertyChanged },
	{ GET_MEMBER_NAME_CHECKED(UCEClonerLineLayout, Rotation), &UCEClonerLineLayout::OnLayoutPropertyChanged },
};

void UCEClonerLineLayout::PostEditChangeProperty(FPropertyChangedEvent& InPropertyChangedEvent)
{
	Super::PostEditChangeProperty(InPropertyChangedEvent);

	PropertyChangeDispatcher.OnPropertyChanged(this, InPropertyChangedEvent);
}
#endif

void UCEClonerLineLayout::OnLayoutParametersChanged(UCEClonerComponent* InComponent)
{
	Super::OnLayoutParametersChanged(InComponent);

	InComponent->SetIntParameter(TEXT("LineCount"), Count);

	InComponent->SetFloatParameter(TEXT("LineSpacing"), Spacing);

	FVector LineAxis;
	if (Axis == ECEClonerAxis::X)
	{
		LineAxis = FVector::XAxisVector;
	}
	else if (Axis == ECEClonerAxis::Y)
	{
		LineAxis = FVector::YAxisVector;
	}
	else if (Axis == ECEClonerAxis::Z)
	{
		LineAxis = FVector::ZAxisVector;
	}
	else
	{
		LineAxis = Direction.GetSafeNormal();
	}

	InComponent->SetVectorParameter(TEXT("LineAxis"), LineAxis);

	InComponent->SetVectorParameter(TEXT("LineRotation"), FVector(Rotation.Roll, Rotation.Pitch, Rotation.Yaw));
}
