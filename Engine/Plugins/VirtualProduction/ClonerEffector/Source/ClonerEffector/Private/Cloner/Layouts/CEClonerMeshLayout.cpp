// Copyright Epic Games, Inc. All Rights Reserved.

#include "Cloner/Layouts/CEClonerMeshLayout.h"

#include "Cloner/CEClonerActor.h"
#include "Cloner/CEClonerComponent.h"
#include "DataInterface/NiagaraDataInterfaceActorComponent.h"
#include "DataInterface/NiagaraDataInterfaceStaticMesh.h"
#include "NiagaraDataInterfaceSkeletalMesh.h"
#include "NiagaraSystem.h"

#if WITH_EDITOR
FName UCEClonerMeshLayout::GetSampleActorWeakName()
{
	return GET_MEMBER_NAME_CHECKED(UCEClonerMeshLayout, SampleActorWeak);
}

FName UCEClonerMeshLayout::GetAssetName()
{
	return GET_MEMBER_NAME_CHECKED(UCEClonerMeshLayout, Asset);
}
#endif

void UCEClonerMeshLayout::SetCount(int32 InCount)
{
	if (Count == InCount)
	{
		return;
	}

	Count = InCount;
	MarkLayoutDirty();
}

void UCEClonerMeshLayout::SetAsset(ECEClonerMeshAsset InAsset)
{
	if (Asset == InAsset)
	{
		return;
	}

	Asset = InAsset;
	MarkLayoutDirty();
}

void UCEClonerMeshLayout::SetSampleData(ECEClonerMeshSampleData InSampleData)
{
	if (SampleData == InSampleData)
	{
		return;
	}

	SampleData = InSampleData;
	MarkLayoutDirty();
}

void UCEClonerMeshLayout::SetSampleActorWeak(const TWeakObjectPtr<AActor>& InSampleActor)
{
	if (SampleActorWeak == InSampleActor)
	{
		return;
	}

	SampleActorWeak = InSampleActor;
	MarkLayoutDirty();
}

void UCEClonerMeshLayout::SetSampleActor(AActor* InActor)
{
	SetSampleActorWeak(InActor);
}

#if WITH_EDITOR
const TCEPropertyChangeDispatcher<UCEClonerMeshLayout> UCEClonerMeshLayout::PropertyChangeDispatcher =
{
	{ GET_MEMBER_NAME_CHECKED(UCEClonerMeshLayout, Count), &UCEClonerMeshLayout::OnLayoutPropertyChanged },
	{ GET_MEMBER_NAME_CHECKED(UCEClonerMeshLayout, Asset), &UCEClonerMeshLayout::OnLayoutPropertyChanged },
	{ GET_MEMBER_NAME_CHECKED(UCEClonerMeshLayout, SampleData), &UCEClonerMeshLayout::OnLayoutPropertyChanged },
	{ GET_MEMBER_NAME_CHECKED(UCEClonerMeshLayout, SampleActorWeak), &UCEClonerMeshLayout::OnLayoutPropertyChanged },
};

void UCEClonerMeshLayout::PostEditChangeProperty(FPropertyChangedEvent& InPropertyChangedEvent)
{
	Super::PostEditChangeProperty(InPropertyChangedEvent);

	PropertyChangeDispatcher.OnPropertyChanged(this, InPropertyChangedEvent);
}
#endif

void UCEClonerMeshLayout::OnLayoutParametersChanged(UCEClonerComponent* InComponent)
{
	Super::OnLayoutParametersChanged(InComponent);

	InComponent->SetIntParameter(TEXT("SampleMeshCount"), Count);

	FNiagaraUserRedirectionParameterStore& ExposedParameters = InComponent->GetOverrideParameters();

	static const FNiagaraVariable SampleMeshAssetVar(FNiagaraTypeDefinition(StaticEnum<ECEClonerMeshAsset>()), TEXT("SampleMeshAsset"));
	ExposedParameters.SetParameterValue<int32>(static_cast<int32>(Asset), SampleMeshAssetVar);

	static const FNiagaraVariable SampleMeshDataVar(FNiagaraTypeDefinition(StaticEnum<ECEClonerMeshSampleData>()), TEXT("SampleMeshData"));
	ExposedParameters.SetParameterValue<int32>(static_cast<int32>(SampleData), SampleMeshDataVar);

	static const FNiagaraVariable SampleMeshActorVar(FNiagaraTypeDefinition(UNiagaraDataInterfaceActorComponent::StaticClass()), TEXT("SampleMeshActor"));
	UNiagaraDataInterfaceActorComponent* ActorMeshDI = Cast<UNiagaraDataInterfaceActorComponent>(ExposedParameters.GetDataInterface(SampleMeshActorVar));

	// unbind
	if (const USceneComponent* SceneComponent = SceneComponentWeak.Get())
	{
		if (AActor* Actor = SceneComponent->GetOwner())
		{
			Actor->OnDestroyed.RemoveAll(this);
		}
	}
	SceneComponentWeak = nullptr;

	// bind
	AActor* SampleActor = SampleActorWeak.Get();
	if (SampleActor && SampleActor->GetRootComponent())
	{
		ActorMeshDI->SourceActor = SampleActor;
		SampleActor->OnDestroyed.AddUniqueDynamic(this, &UCEClonerMeshLayout::OnSampleActorDestroyed);
		SceneComponentWeak = SampleActor->GetRootComponent();
	}
	else
	{
		SampleActorWeak.Reset();
		SceneComponentWeak.Reset();
	}

	static const FNiagaraVariable SampleMeshStaticVar(FNiagaraTypeDefinition(UNiagaraDataInterfaceStaticMesh::StaticClass()), TEXT("SampleMeshStatic"));
	UNiagaraDataInterfaceStaticMesh* StaticMeshDI = Cast<UNiagaraDataInterfaceStaticMesh>(ExposedParameters.GetDataInterface(SampleMeshStaticVar));
	UStaticMeshComponent* StaticMeshComponent = SampleActor ? SampleActor->FindComponentByClass<UStaticMeshComponent>() : nullptr;

	if (Asset == ECEClonerMeshAsset::StaticMesh && StaticMeshComponent)
	{
		StaticMeshDI->SetSourceComponentFromBlueprints(StaticMeshComponent);
	}
	else
	{
		StaticMeshDI->SetSourceComponentFromBlueprints(nullptr);
	}

	static const FNiagaraVariable SampleMeshSkeletalVar(FNiagaraTypeDefinition(UNiagaraDataInterfaceSkeletalMesh::StaticClass()), TEXT("SampleMeshSkeletal"));
	UNiagaraDataInterfaceSkeletalMesh* SkeletalMeshDI = Cast<UNiagaraDataInterfaceSkeletalMesh>(ExposedParameters.GetDataInterface(SampleMeshSkeletalVar));
	USkeletalMeshComponent* SkeletalMeshComponent = SampleActor ? SampleActor->FindComponentByClass<USkeletalMeshComponent>() : nullptr;

	if (Asset == ECEClonerMeshAsset::SkeletalMesh && SkeletalMeshComponent)
	{
		SkeletalMeshDI->SetSourceComponentFromBlueprints(SkeletalMeshComponent);
	}
	else
	{
		SkeletalMeshDI->SetSourceComponentFromBlueprints(nullptr);
	}
}

void UCEClonerMeshLayout::OnSampleActorDestroyed(AActor* InDestroyedActor)
{
	UCEClonerComponent* ClonerComponent = GetClonerComponent();

	if (!ClonerComponent)
	{
		return;
	}

	const FNiagaraUserRedirectionParameterStore& ExposedParameters = ClonerComponent->GetOverrideParameters();

	static const FNiagaraVariable SampleMeshStaticVar(FNiagaraTypeDefinition(UNiagaraDataInterfaceStaticMesh::StaticClass()), TEXT("SampleMeshStatic"));
	if (UNiagaraDataInterfaceStaticMesh* StaticMeshDI = Cast<UNiagaraDataInterfaceStaticMesh>(ExposedParameters.GetDataInterface(SampleMeshStaticVar)))
	{
		StaticMeshDI->Modify();

		StaticMeshDI->SetSourceComponentFromBlueprints(nullptr);
	}

	static const FNiagaraVariable SampleMeshSkeletalVar(FNiagaraTypeDefinition(UNiagaraDataInterfaceSkeletalMesh::StaticClass()), TEXT("SampleMeshSkeletal"));
	if (UNiagaraDataInterfaceSkeletalMesh* SkeletalMeshDI = Cast<UNiagaraDataInterfaceSkeletalMesh>(ExposedParameters.GetDataInterface(SampleMeshSkeletalVar)))
	{
		SkeletalMeshDI->Modify();

		SkeletalMeshDI->SetSourceComponentFromBlueprints(nullptr);
	}

	MarkLayoutDirty();
}
