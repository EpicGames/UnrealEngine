// Copyright Epic Games, Inc. All Rights Reserved.

#include "Cloner/Layouts/CEClonerSphereRandomLayout.h"

#include "Cloner/CEClonerComponent.h"

void UCEClonerSphereRandomLayout::SetCount(int32 InCount)
{
	if (Count == InCount)
	{
		return;
	}

	Count = InCount;
	MarkLayoutDirty();
}

void UCEClonerSphereRandomLayout::SetRadius(float InRadius)
{
	if (Radius == InRadius)
	{
		return;
	}

	Radius = InRadius;
	MarkLayoutDirty();
}

void UCEClonerSphereRandomLayout::SetDistribution(float InDistribution)
{
	if (Distribution == InDistribution)
	{
		return;
	}

	if (InDistribution < 0.f || InDistribution > 1.f)
	{
		return;
	}

	Distribution = InDistribution;
	MarkLayoutDirty();
}

void UCEClonerSphereRandomLayout::SetLongitude(float InLongitude)
{
	if (Longitude == InLongitude)
	{
		return;
	}

	if (InLongitude < 0.f || InLongitude > 1.f)
	{
		return;
	}

	Longitude = InLongitude;
	MarkLayoutDirty();
}

void UCEClonerSphereRandomLayout::SetLatitude(float InLatitude)
{
	if (Latitude == InLatitude)
	{
		return;
	}

	if (InLatitude < 0.f || InLatitude > 1.f)
	{
		return;
	}

	Latitude = InLatitude;
	MarkLayoutDirty();
}

void UCEClonerSphereRandomLayout::SetOrientMesh(bool bInOrientMesh)
{
	if (bOrientMesh == bInOrientMesh)
	{
		return;
	}

	bOrientMesh = bInOrientMesh;
	MarkLayoutDirty();
}

void UCEClonerSphereRandomLayout::SetRotation(const FRotator& InRotation)
{
	if (Rotation == InRotation)
	{
		return;
	}

	Rotation = InRotation;
	MarkLayoutDirty();
}

void UCEClonerSphereRandomLayout::SetScale(const FVector& InScale)
{
	if (Scale == InScale)
	{
		return;
	}

	Scale = InScale;
	MarkLayoutDirty();
}

#if WITH_EDITOR
const TCEPropertyChangeDispatcher<UCEClonerSphereRandomLayout> UCEClonerSphereRandomLayout::PropertyChangeDispatcher =
{
	{ GET_MEMBER_NAME_CHECKED(UCEClonerSphereRandomLayout, Count), &UCEClonerSphereRandomLayout::OnLayoutPropertyChanged },
	{ GET_MEMBER_NAME_CHECKED(UCEClonerSphereRandomLayout, Radius), &UCEClonerSphereRandomLayout::OnLayoutPropertyChanged },
	{ GET_MEMBER_NAME_CHECKED(UCEClonerSphereRandomLayout, Distribution), &UCEClonerSphereRandomLayout::OnLayoutPropertyChanged },
	{ GET_MEMBER_NAME_CHECKED(UCEClonerSphereRandomLayout, Longitude), &UCEClonerSphereRandomLayout::OnLayoutPropertyChanged },
	{ GET_MEMBER_NAME_CHECKED(UCEClonerSphereRandomLayout, Latitude), &UCEClonerSphereRandomLayout::OnLayoutPropertyChanged },
	{ GET_MEMBER_NAME_CHECKED(UCEClonerSphereRandomLayout, bOrientMesh), &UCEClonerSphereRandomLayout::OnLayoutPropertyChanged },
	{ GET_MEMBER_NAME_CHECKED(UCEClonerSphereRandomLayout, Rotation), &UCEClonerSphereRandomLayout::OnLayoutPropertyChanged },
	{ GET_MEMBER_NAME_CHECKED(UCEClonerSphereRandomLayout, Scale), &UCEClonerSphereRandomLayout::OnLayoutPropertyChanged },
};

void UCEClonerSphereRandomLayout::PostEditChangeProperty(FPropertyChangedEvent& InPropertyChangedEvent)
{
	Super::PostEditChangeProperty(InPropertyChangedEvent);

	PropertyChangeDispatcher.OnPropertyChanged(this, InPropertyChangedEvent);
}
#endif

void UCEClonerSphereRandomLayout::OnLayoutParametersChanged(UCEClonerComponent* InComponent)
{
	Super::OnLayoutParametersChanged(InComponent);

	InComponent->SetIntParameter(TEXT("SphereCount"), Count);

	InComponent->SetFloatParameter(TEXT("SphereRadius"), Radius);

	InComponent->SetFloatParameter(TEXT("SphereDistribution"), Distribution);

	InComponent->SetFloatParameter(TEXT("SphereLongitude"), Longitude);

	InComponent->SetFloatParameter(TEXT("SphereLatitude"), Latitude);

	InComponent->SetBoolParameter(TEXT("MeshOrientAxisEnable"), bOrientMesh);

	InComponent->SetVectorParameter(TEXT("SphereRotation"), FVector(Rotation.Yaw, Rotation.Pitch, Rotation.Roll));

	InComponent->SetVectorParameter(TEXT("SphereScale"), Scale);
}
