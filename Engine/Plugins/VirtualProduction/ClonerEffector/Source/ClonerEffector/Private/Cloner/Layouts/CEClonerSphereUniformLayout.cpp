// Copyright Epic Games, Inc. All Rights Reserved.

#include "Cloner/Layouts/CEClonerSphereUniformLayout.h"

#include "Cloner/CEClonerComponent.h"

void UCEClonerSphereUniformLayout::SetCount(int32 InCount)
{
	if (Count == InCount)
	{
		return;
	}

	Count = InCount;
	MarkLayoutDirty();
}

void UCEClonerSphereUniformLayout::SetRadius(float InRadius)
{
	if (Radius == InRadius)
	{
		return;
	}

	Radius = InRadius;
	MarkLayoutDirty();
}

void UCEClonerSphereUniformLayout::SetRatio(float InRatio)
{
	if (Ratio == InRatio)
	{
		return;
	}

	Ratio = InRatio;
	MarkLayoutDirty();
}

void UCEClonerSphereUniformLayout::SetOrientMesh(bool bInOrientMesh)
{
	if (bOrientMesh == bInOrientMesh)
	{
		return;
	}

	bOrientMesh = bInOrientMesh;
	MarkLayoutDirty();
}

void UCEClonerSphereUniformLayout::SetRotation(const FRotator& InRotation)
{
	if (Rotation == InRotation)
	{
		return;
	}

	Rotation = InRotation;
	MarkLayoutDirty();
}

void UCEClonerSphereUniformLayout::SetScale(const FVector& InScale)
{
	if (Scale == InScale)
	{
		return;
	}

	Scale = InScale;
	MarkLayoutDirty();
}

#if WITH_EDITOR
const TCEPropertyChangeDispatcher<UCEClonerSphereUniformLayout> UCEClonerSphereUniformLayout::PropertyChangeDispatcher =
{
	{ GET_MEMBER_NAME_CHECKED(UCEClonerSphereUniformLayout, Count), &UCEClonerSphereUniformLayout::OnLayoutPropertyChanged },
	{ GET_MEMBER_NAME_CHECKED(UCEClonerSphereUniformLayout, Radius), &UCEClonerSphereUniformLayout::OnLayoutPropertyChanged },
	{ GET_MEMBER_NAME_CHECKED(UCEClonerSphereUniformLayout, Ratio), &UCEClonerSphereUniformLayout::OnLayoutPropertyChanged },
	{ GET_MEMBER_NAME_CHECKED(UCEClonerSphereUniformLayout, bOrientMesh), &UCEClonerSphereUniformLayout::OnLayoutPropertyChanged },
	{ GET_MEMBER_NAME_CHECKED(UCEClonerSphereUniformLayout, Rotation), &UCEClonerSphereUniformLayout::OnLayoutPropertyChanged },
	{ GET_MEMBER_NAME_CHECKED(UCEClonerSphereUniformLayout, Scale), &UCEClonerSphereUniformLayout::OnLayoutPropertyChanged },
};

void UCEClonerSphereUniformLayout::PostEditChangeProperty(FPropertyChangedEvent& InPropertyChangedEvent)
{
	Super::PostEditChangeProperty(InPropertyChangedEvent);

	PropertyChangeDispatcher.OnPropertyChanged(this, InPropertyChangedEvent);
}
#endif

void UCEClonerSphereUniformLayout::OnLayoutParametersChanged(UCEClonerComponent* InComponent)
{
	Super::OnLayoutParametersChanged(InComponent);

	InComponent->SetIntParameter(TEXT("SphereCount"), Count);

	InComponent->SetFloatParameter(TEXT("SphereRadius"), Radius);

	InComponent->SetFloatParameter(TEXT("SphereRatio"), Ratio);

	InComponent->SetBoolParameter(TEXT("MeshOrientAxisEnable"), bOrientMesh);

	InComponent->SetVectorParameter(TEXT("SphereRotation"), FVector(Rotation.Yaw, Rotation.Pitch, Rotation.Roll));

	InComponent->SetVectorParameter(TEXT("SphereScale"), Scale);
}
