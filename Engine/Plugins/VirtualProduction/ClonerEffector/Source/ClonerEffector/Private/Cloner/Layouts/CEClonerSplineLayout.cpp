// Copyright Epic Games, Inc. All Rights Reserved.

#include "Cloner/Layouts/CEClonerSplineLayout.h"

#include "Cloner/CEClonerComponent.h"
#include "Engine/World.h"
#include "GameFramework/Actor.h"
#include "NiagaraDataInterfaceSpline.h"
#include "NiagaraSystem.h"

#if WITH_EDITOR
#include "Editor/EditorEngine.h"
#endif

#if WITH_EDITOR
FName UCEClonerSplineLayout::GetSplineActorWeakName()
{
	return GET_MEMBER_NAME_CHECKED(UCEClonerSplineLayout, SplineActorWeak);
}
#endif

void UCEClonerSplineLayout::SetCount(int32 InCount)
{
	if (Count == InCount)
	{
		return;
	}

	Count = InCount;
	MarkLayoutDirty();
}

void UCEClonerSplineLayout::SetSplineActorWeak(const TWeakObjectPtr<AActor>& InSplineActor)
{
	if (SplineActorWeak == InSplineActor)
	{
		return;
	}

	SplineActorWeak = InSplineActor;
	MarkLayoutDirty();
}

void UCEClonerSplineLayout::SetSplineActor(AActor* InSplineActor)
{
	SetSplineActorWeak(InSplineActor);
}

void UCEClonerSplineLayout::SetOrientMesh(bool bInOrientMesh)
{
	if (bOrientMesh == bInOrientMesh)
	{
		return;
	}

	bOrientMesh = bInOrientMesh;
	MarkLayoutDirty();
}

#if WITH_EDITOR
void UCEClonerSplineLayout::SpawnLinkedSplineActor()
{
	const UCEClonerComponent* ClonerComponent = GetClonerComponent();

	if (!IsValid(ClonerComponent))
	{
		return;
	}

	UWorld* ClonerWorld = ClonerComponent->GetWorld();

	if (!IsValid(ClonerWorld))
	{
		return;
	}

	FActorSpawnParameters Params;
	Params.bTemporaryEditorActor = false;

	AActor* SpawnedSplineActor = ClonerWorld->SpawnActor<AActor>(FVector::ZeroVector, FRotator::ZeroRotator, Params);

	if (!SpawnedSplineActor)
	{
		return;
	}

	// Construct the new component and attach as needed
	USplineComponent* const NewComponent = NewObject<USplineComponent>(SpawnedSplineActor
		, USplineComponent::StaticClass()
		, MakeUniqueObjectName(SpawnedSplineActor, USplineComponent::StaticClass(), TEXT("SplineComponent"))
		, RF_Transactional);

	SpawnedSplineActor->SetRootComponent(NewComponent);

	// Add to SerializedComponents array so it gets saved
	SpawnedSplineActor->AddInstanceComponent(NewComponent);
	NewComponent->OnComponentCreated();
	NewComponent->RegisterComponent();

	// Rerun construction scripts
	SpawnedSplineActor->RerunConstructionScripts();

	SpawnedSplineActor->SetActorLocation(ClonerComponent->GetComponentLocation());
	SpawnedSplineActor->SetActorRotation(ClonerComponent->GetComponentRotation());

	SetSplineActorWeak(SpawnedSplineActor);
	FActorLabelUtilities::RenameExistingActor(SpawnedSplineActor, TEXT("SplineActor"), true);
}

const TCEPropertyChangeDispatcher<UCEClonerSplineLayout> UCEClonerSplineLayout::PropertyChangeDispatcher =
{
	{ GET_MEMBER_NAME_CHECKED(UCEClonerSplineLayout, Count), &UCEClonerSplineLayout::OnLayoutPropertyChanged },
	{ GET_MEMBER_NAME_CHECKED(UCEClonerSplineLayout, SplineActorWeak), &UCEClonerSplineLayout::OnLayoutPropertyChanged },
	{ GET_MEMBER_NAME_CHECKED(UCEClonerSplineLayout, bOrientMesh), &UCEClonerSplineLayout::OnLayoutPropertyChanged },
};

void UCEClonerSplineLayout::PostEditChangeProperty(FPropertyChangedEvent& InPropertyChangedEvent)
{
	Super::PostEditChangeProperty(InPropertyChangedEvent);

	PropertyChangeDispatcher.OnPropertyChanged(this, InPropertyChangedEvent);
}
#endif

void UCEClonerSplineLayout::OnLayoutActive()
{
	Super::OnLayoutActive();

	USceneComponent::MarkRenderStateDirtyEvent.AddUObject(this, &UCEClonerSplineLayout::OnSampleSplineRenderStateUpdated);
}

void UCEClonerSplineLayout::OnLayoutInactive()
{
	Super::OnLayoutInactive();

	USceneComponent::MarkRenderStateDirtyEvent.RemoveAll(this);
}

void UCEClonerSplineLayout::OnLayoutParametersChanged(UCEClonerComponent* InComponent)
{
	Super::OnLayoutParametersChanged(InComponent);

	InComponent->SetIntParameter(TEXT("SampleSplineCount"), Count);

	const FNiagaraUserRedirectionParameterStore& ExposedParameters = InComponent->GetOverrideParameters();
	static const FNiagaraVariable SampleSplineVar(FNiagaraTypeDefinition(UNiagaraDataInterfaceSpline::StaticClass()), TEXT("SampleSpline"));
	UNiagaraDataInterfaceSpline* SplineDI = Cast<UNiagaraDataInterfaceSpline>(ExposedParameters.GetDataInterface(SampleSplineVar));

	InComponent->SetBoolParameter(TEXT("MeshOrientAxisEnable"), bOrientMesh);

	// unbind
	if (AActor* PrevSplineActor = SplineDI->SoftSourceActor.Get())
	{
		PrevSplineActor->OnDestroyed.RemoveAll(this);
	}
	
	SplineDI->SoftSourceActor = nullptr;

	// bind
	if (AActor* SplineActor = SplineActorWeak.Get())
	{
		if (USplineComponent* SplineComponent = SplineActor->FindComponentByClass<USplineComponent>())
		{
			SplineDI->SoftSourceActor = SplineActor;
			SplineActor->OnDestroyed.AddUniqueDynamic(this, &UCEClonerSplineLayout::OnSampleSplineDestroyed);
		}
	}
}

void UCEClonerSplineLayout::OnSampleSplineRenderStateUpdated(UActorComponent& InComponent)
{
	if (SplineActorWeak.IsValid() && InComponent.GetOwner() == SplineActorWeak.Get())
	{
		MarkLayoutDirty();
	}
}

void UCEClonerSplineLayout::OnSampleSplineDestroyed(AActor* InDestroyedActor)
{
	MarkLayoutDirty();
}
