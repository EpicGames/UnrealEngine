// Copyright Epic Games, Inc. All Rights Reserved.

#include "Effector/CEEffectorActor.h"

#include "Components/DynamicMeshComponent.h"
#include "Effector/CEEffectorComponent.h"
#include "Effector/Effects/CEEffectorForceEffect.h"
#include "Effector/Modes/CEEffectorOffsetMode.h"
#include "Effector/Modes/CEEffectorProceduralMode.h"
#include "Effector/Modes/CEEffectorPushMode.h"
#include "Effector/Modes/CEEffectorTargetMode.h"
#include "Effector/Types/CEEffectorBoxType.h"
#include "Effector/Types/CEEffectorPlaneType.h"
#include "Effector/Types/CEEffectorRadialType.h"
#include "Effector/Types/CEEffectorSphereType.h"
#include "Effector/Types/CEEffectorTorusType.h"
#include "Logs/CEEffectorLogs.h"

#if WITH_EDITOR
#include "Materials/MaterialInstanceDynamic.h"
#include "UObject/ConstructorHelpers.h"
#endif

struct FCEEffectorActorVersion
{
	enum Type : int32
	{
		PreVersioning = 0,

		/** Migrating all logic and properties to component */
		ComponentMigration,

		VersionPlusOne,
		LatestVersion = VersionPlusOne - 1
	};

	static constexpr FGuid GUID = FGuid(0x9271D8A6, 0xBF4146B2, 0xA20FC0A4, 0x9D8295B3);
};

FCustomVersionRegistration GRegisterCEEffectorActorVersion(FCEEffectorActorVersion::GUID, static_cast<int32>(FCEEffectorActorVersion::LatestVersion), TEXT("CEEffectorActorVersion"));

ACEEffectorActor::ACEEffectorActor()
{
	SetCanBeDamaged(false);
	PrimaryActorTick.bCanEverTick = false;

	EffectorComponent = CreateDefaultSubobject<UCEEffectorComponent>(TEXT("AvaEffectorComponent"));
	SetRootComponent(EffectorComponent);

#if WITH_EDITOR
	if (GIsEditor && !HasAnyFlags(RF_ClassDefaultObject))
	{
		UDynamicMeshComponent* InnerVisualizerComponent = CreateEditorOnlyDefaultSubobject<UDynamicMeshComponent>(TEXT("InnerVisualizerComponent"));
		InnerVisualizerComponent->SetupAttachment(EffectorComponent);
		InnerVisualizerComponent->bIsEditorOnly = true;
		EffectorComponent->AddVisualizerComponent(InnerVisualizerComponent);	
	}

	if (GIsEditor && !HasAnyFlags(RF_ClassDefaultObject))
	{
		UDynamicMeshComponent* OuterVisualizerComponent = CreateEditorOnlyDefaultSubobject<UDynamicMeshComponent>(TEXT("OuterVisualizerComponent"));
		OuterVisualizerComponent->SetupAttachment(EffectorComponent);
		OuterVisualizerComponent->bIsEditorOnly = true;
		EffectorComponent->AddVisualizerComponent(OuterVisualizerComponent);
	}
#endif
}

#if WITH_EDITOR
FString ACEEffectorActor::GetDefaultActorLabel() const
{
	return DefaultLabel;
}
#endif

void ACEEffectorActor::Serialize(FArchive& InArchive)
{
	InArchive.UsingCustomVersion(FCEEffectorActorVersion::GUID);

	Super::Serialize(InArchive);

	const int32 Version = InArchive.CustomVer(FCEEffectorActorVersion::GUID);

	if (Version < FCEEffectorActorVersion::ComponentMigration)
	{
		MigrateToVersion = FCEEffectorActorVersion::ComponentMigration;
	}
}

void ACEEffectorActor::PostLoad()
{
	Super::PostLoad();

	MigrateDeprecatedProperties();
}

void ACEEffectorActor::MigrateDeprecatedProperties()
{
	if (MigrateToVersion == INDEX_NONE)
	{
		return;
	}

	UE_LOGF(LogCEEffector, Warning, "%ls : Effector migrating from version %i to latest %i, please re-save this asset", *GetActorNameOrLabel(), MigrateToVersion, FCEEffectorActorVersion::LatestVersion);

	PRAGMA_DISABLE_DEPRECATION_WARNINGS

	if (MigrateToVersion <= FCEEffectorActorVersion::ComponentMigration)
	{
		EffectorComponent->SetEnabled(bEnabled);
		EffectorComponent->SetMagnitude(Magnitude);
		EffectorComponent->SetColor(Color);

		{
			const TArray<FName> TypeNames = EffectorComponent->GetEffectorTypeNames();
			const int32 TypeIndex = static_cast<int32>(Type);
			if (TypeNames.IsValidIndex(TypeIndex))
			{
				EffectorComponent->SetTypeName(TypeNames[TypeIndex]);
			}
		}

		{
			const TArray<FName> ModeNames = EffectorComponent->GetEffectorModeNames();
			const int32 ModeIndex = static_cast<int32>(Mode);
			if (ModeNames.IsValidIndex(ModeIndex))
			{
				EffectorComponent->SetModeName(ModeNames[ModeIndex]);
			}
		}

#if WITH_EDITOR
		EffectorComponent->SetVisualizerComponentVisible(bVisualizerComponentVisible);
		EffectorComponent->SetVisualizerSpriteVisible(bVisualizerSpriteVisible);
#endif

		if (UCEEffectorSphereType* SphereExtension = EffectorComponent->FindOrAddExtension<UCEEffectorSphereType>())
		{
			SphereExtension->SetInnerRadius(InnerRadius);
			SphereExtension->SetOuterRadius(OuterRadius);
			SphereExtension->SetInvertType(bInvertType);
			SphereExtension->SetEasing(Easing);
		}

		if (UCEEffectorBoxType* BoxExtension = EffectorComponent->FindOrAddExtension<UCEEffectorBoxType>())
		{
			BoxExtension->SetInnerExtent(InnerExtent);
			BoxExtension->SetOuterExtent(OuterExtent);
			BoxExtension->SetInvertType(bInvertType);
			BoxExtension->SetEasing(Easing);
		}

		if (UCEEffectorPlaneType* PlaneExtension = EffectorComponent->FindOrAddExtension<UCEEffectorPlaneType>())
		{
			PlaneExtension->SetPlaneSpacing(PlaneSpacing);
			PlaneExtension->SetInvertType(bInvertType);
			PlaneExtension->SetEasing(Easing);
		}

		if (UCEEffectorRadialType* RadialExtension = EffectorComponent->FindOrAddExtension<UCEEffectorRadialType>())
		{
			RadialExtension->SetRadialAngle(RadialAngle);
			RadialExtension->SetRadialMinRadius(RadialMinRadius);
			RadialExtension->SetRadialMaxRadius(RadialMaxRadius);
			RadialExtension->SetInvertType(bInvertType);
			RadialExtension->SetEasing(Easing);
		}

		if (UCEEffectorTorusType* TorusExtension = EffectorComponent->FindOrAddExtension<UCEEffectorTorusType>())
		{
			TorusExtension->SetTorusRadius(TorusRadius);
			TorusExtension->SetTorusInnerRadius(TorusInnerRadius);
			TorusExtension->SetTorusOuterRadius(TorusOuterRadius);
			TorusExtension->SetInvertType(bInvertType);
			TorusExtension->SetEasing(Easing);
		}

		if (UCEEffectorOffsetMode* OffsetExtension = EffectorComponent->FindOrAddExtension<UCEEffectorOffsetMode>())
		{
			OffsetExtension->SetOffset(Offset);
			OffsetExtension->SetScale(Scale);
			OffsetExtension->SetRotation(Rotation);
		}

		if (UCEEffectorTargetMode* TargetExtension = EffectorComponent->FindOrAddExtension<UCEEffectorTargetMode>())
		{
			TargetExtension->SetTargetActorWeak(TargetActorWeak);
		}

		if (UCEEffectorProceduralMode* NoiseExtension = EffectorComponent->FindOrAddExtension<UCEEffectorProceduralMode>())
		{
			NoiseExtension->SetLocationStrength(LocationStrength);
			NoiseExtension->SetRotationStrength(RotationStrength);
			NoiseExtension->SetScaleStrength(ScaleStrength);
			NoiseExtension->SetFrequency(Frequency);
			NoiseExtension->SetPan(Pan);
		}

		if (UCEEffectorPushMode* PushExtension = EffectorComponent->FindOrAddExtension<UCEEffectorPushMode>())
		{
			PushExtension->SetPushDirection(PushDirection);
			PushExtension->SetPushStrength(PushStrength);
		}

		if (UCEEffectorForceEffect* ForceExtension = EffectorComponent->FindOrAddExtension<UCEEffectorForceEffect>())
		{
			ForceExtension->SetForcesEnabled(bAttractionForceEnabled || bGravityForceEnabled || bOrientationForceEnabled || bVortexForceEnabled || bCurlNoiseForceEnabled);

			ForceExtension->SetAttractionForceEnabled(bAttractionForceEnabled);
			ForceExtension->SetAttractionForceFalloff(AttractionForceFalloff);
			ForceExtension->SetAttractionForceStrength(AttractionForceStrength);

			ForceExtension->SetGravityForceEnabled(bGravityForceEnabled);
			ForceExtension->SetGravityForceAcceleration(GravityForceAcceleration);

			ForceExtension->SetOrientationForceEnabled(bOrientationForceEnabled);
			ForceExtension->SetOrientationForceMin(OrientationForceMin);
			ForceExtension->SetOrientationForceMax(OrientationForceMax);
			ForceExtension->SetOrientationForceRate(OrientationForceRate);

			ForceExtension->SetVortexForceEnabled(bVortexForceEnabled);
			ForceExtension->SetVortexForceAmount(VortexForceAmount);
			ForceExtension->SetVortexForceAxis(VortexForceAxis);

			ForceExtension->SetCurlNoiseForceEnabled(bCurlNoiseForceEnabled);
			ForceExtension->SetCurlNoiseForceFrequency(CurlNoiseForceFrequency);
			ForceExtension->SetCurlNoiseForceStrength(CurlNoiseForceStrength);
		}
	}

	PRAGMA_ENABLE_DEPRECATION_WARNINGS

	MigrateToVersion = INDEX_NONE;
}

void ACEEffectorActor::RegisterToChannel() const
{
	if (EffectorComponent)
	{
		EffectorComponent->RegisterToChannel();
	}
}
