// Copyright Epic Games, Inc. All Rights Reserved.

#include "Effector/CEEffectorComponent.h"

#include "Cloner/CEClonerComponent.h"
#include "Cloner/Extensions/CEClonerEffectorExtension.h"
#include "Components/BillboardComponent.h"
#include "Components/DynamicMeshComponent.h"
#include "Containers/Ticker.h"
#include "Effector/CEEffectorExtensionBase.h"
#include "Effector/Effects/CEEffectorEffectBase.h"
#include "Effector/Modes/CEEffectorModeBase.h"
#include "Effector/Types/CEEffectorTypeBase.h"
#include "Engine/Texture2D.h"
#include "GameFramework/Actor.h"
#include "Logs/CEEffectorLogs.h"
#include "Materials/MaterialInstanceDynamic.h"
#include "Materials/MaterialInterface.h"
#include "Subsystems/CEEffectorSubsystem.h"

#if WITH_EDITOR
FName UCEEffectorComponent::GetModeNamePropertyName()
{
	return GET_MEMBER_NAME_CHECKED(UCEEffectorComponent, ModeName);
}

FName UCEEffectorComponent::GetTypeNamePropertyName()
{
	return GET_MEMBER_NAME_CHECKED(UCEEffectorComponent, TypeName);
}
#endif

UCEEffectorComponent::UCEEffectorComponent()
{
	PrimaryComponentTick.bCanEverTick = false;

	// Show sprite for this component to visualize it when empty
#if WITH_EDITORONLY_DATA
	bVisualizeComponent = true;
#endif

	if (!IsTemplate())
	{
		// Apply default type
		const TArray<FName> TypeNames = GetEffectorTypeNames();
		TypeName = !TypeNames.IsEmpty() ? FName(TypeNames[0]) : NAME_None;

		// Apply default mode
		const TArray<FName> ModeNames = GetEffectorModeNames();
		ModeName = !ModeNames.IsEmpty() ? FName(ModeNames[0]) : NAME_None;

		UCEEffectorSubsystem::OnEffectorSetEnabled().AddUObject(this, &UCEEffectorComponent::OnEffectorSetEnabled);
		TransformUpdated.AddUObject(this, &UCEEffectorComponent::OnTransformUpdated);
	}
}

void UCEEffectorComponent::SetEnabled(bool bInEnable)
{
	if (bInEnable == bEnabled)
	{
		return;
	}

	bEnabled = bInEnable;
	OnEnabledChanged();
}

void UCEEffectorComponent::SetMagnitude(float InMagnitude)
{
	InMagnitude = FMath::Clamp(InMagnitude, 0, 1);

	if (FMath::IsNearlyEqual(InMagnitude, Magnitude))
	{
		return;
	}

	Magnitude = InMagnitude;
	OnEffectorOptionsChanged();
}

void UCEEffectorComponent::SetTypeName(FName InTypeName)
{
	if (TypeName.IsEqual(InTypeName))
	{
		return;
	}

	if (!GetEffectorTypeNames().Contains(InTypeName))
	{
		return;
	}

	TypeName = InTypeName;
	OnTypeNameChanged();
}

void UCEEffectorComponent::SetTypeClass(TSubclassOf<UCEEffectorTypeBase> InTypeClass)
{
	if (!InTypeClass.Get())
	{
		return;
	}

	if (const UCEEffectorSubsystem* EffectorSubsystem = UCEEffectorSubsystem::Get())
	{
		const FName ExtensionName = EffectorSubsystem->FindExtensionName(InTypeClass);

		if (!ExtensionName.IsNone())
		{
			SetTypeName(ExtensionName);
		}
	}
}

TSubclassOf<UCEEffectorTypeBase> UCEEffectorComponent::GetTypeClass() const
{
	return ActiveType ? ActiveType->GetClass() : nullptr;
}

void UCEEffectorComponent::SetModeName(FName InModeName)
{
	if (ModeName.IsEqual(InModeName))
	{
		return;
	}

	if (!GetEffectorModeNames().Contains(InModeName))
	{
		return;
	}

	ModeName = InModeName;
	OnModeNameChanged();
}

void UCEEffectorComponent::SetModeClass(TSubclassOf<UCEEffectorModeBase> InModeClass)
{
	if (!InModeClass.Get())
	{
		return;
	}

	if (const UCEEffectorSubsystem* EffectorSubsystem = UCEEffectorSubsystem::Get())
	{
		const FName ExtensionName = EffectorSubsystem->FindExtensionName(InModeClass);

		if (!ExtensionName.IsNone())
		{
			SetModeName(ExtensionName);
		}
	}
}

TSubclassOf<UCEEffectorModeBase> UCEEffectorComponent::GetModeClass() const
{
	return ActiveMode ? ActiveMode->GetClass() : nullptr;
}

TConstArrayView<TWeakObjectPtr<UCEClonerEffectorExtension>> UCEEffectorComponent::GetClonerExtensionsWeak() const
{
	return ClonerExtensionsWeak;
}

FCEClonerEffectorChannelData& UCEEffectorComponent::GetChannelData()
{
	return ChannelData;
}

void UCEEffectorComponent::RegisterToChannel()
{
	if (IsValid(this) && ChannelData.GetIdentifier() == INDEX_NONE)
	{
		// Register this effector to the effector channel
		if (UCEEffectorSubsystem* EffectorSubsystem = UCEEffectorSubsystem::Get())
		{
			EffectorSubsystem->RegisterChannelEffector(this);
		}
	}
}

void UCEEffectorComponent::UnregisterFromChannel()
{
	// Remove this effector from the effector channel
	if (UCEEffectorSubsystem* EffectorSubsystem = UCEEffectorSubsystem::Get())
	{
		EffectorSubsystem->UnregisterChannelEffector(this);
	}
}

int32 UCEEffectorComponent::GetChannelIdentifier() const
{
	return ChannelData.GetIdentifier();
}

void UCEEffectorComponent::OnTransformUpdated(USceneComponent* InComponent, EUpdateTransformFlags InFlags, ETeleportType InTeleportType)
{
	OnEffectorOptionsChanged();
}

void UCEEffectorComponent::OnClonerLinked(UCEClonerEffectorExtension* InCloner)
{
	if (!IsValid(InCloner) || ClonerExtensionsWeak.Contains(InCloner))
	{
		return;
	}

	ClonerExtensionsWeak.Add(InCloner);
	OnEffectorOptionsChanged();
}

void UCEEffectorComponent::OnClonerUnlinked(UCEClonerEffectorExtension* InCloner)
{
	if (!IsValid(InCloner) || !ClonerExtensionsWeak.Contains(InCloner))
	{
		return;
	}

	ClonerExtensionsWeak.Remove(InCloner);
}

void UCEEffectorComponent::GetActiveEffects(TArray<UCEEffectorEffectBase*>& OutEffects) const
{
	OutEffects = ActiveEffects;
}

UCEEffectorExtensionBase* UCEEffectorComponent::GetExtension(TSubclassOf<UCEEffectorExtensionBase> InExtensionClass) const
{
	const UCEEffectorSubsystem* Subsystem = UCEEffectorSubsystem::Get();

	if (!Subsystem)
	{
		return nullptr;
	}

	const FName ExtensionName = Subsystem->FindExtensionName(InExtensionClass.Get());

	if (ExtensionName.IsNone())
	{
		return nullptr;
	}

	return GetExtension(ExtensionName);
}

UCEEffectorExtensionBase* UCEEffectorComponent::GetExtension(FName InExtensionName) const
{
	for (const TObjectPtr<UCEEffectorExtensionBase>& ExtensionInstance : ExtensionInstances)
	{
		if (ExtensionInstance && ExtensionInstance->GetExtensionName() == InExtensionName)
		{
			return ExtensionInstance;
		}
	}

	return nullptr;
}

void UCEEffectorComponent::RequestClonerUpdate(bool bInImmediate)
{
	if (bInImmediate)
	{
		for (const TWeakObjectPtr<UCEClonerEffectorExtension>& ClonerExtensionWeak : GetClonerExtensionsWeak())
		{
			if (UCEClonerEffectorExtension* ClonerExtension = ClonerExtensionWeak.Get())
			{
				ClonerExtension->MarkExtensionDirty(bInImmediate);
			}
		}
	}
	else
	{
		TWeakObjectPtr<UCEEffectorComponent> ThisWeak(this);
		FTSTicker::GetCoreTicker().AddTicker(FTickerDelegate::CreateLambda([ThisWeak](float InDeltaTime)
		{
			if (UCEEffectorComponent* EffectorComponent = ThisWeak.Get())
			{
				EffectorComponent->RequestClonerUpdate(true);
			}

			return false;
		}));
	}
}

void UCEEffectorComponent::OnComponentCreated()
{
	Super::OnComponentCreated();

	OnEffectorOptionsChanged();
}

void UCEEffectorComponent::OnComponentDestroyed(bool bDestroyingHierarchy)
{
	Super::OnComponentDestroyed(bDestroyingHierarchy);

	UnregisterFromChannel();
}

void UCEEffectorComponent::PostEditImport()
{
	Super::PostEditImport();

	if (AActor* Owner = GetOwner())
	{
		for (const TWeakObjectPtr<UCEClonerEffectorExtension>& ClonerExtensionWeak : ClonerExtensionsWeak)
		{
			if (UCEClonerEffectorExtension* ClonerExtension = ClonerExtensionWeak.Get())
			{
				UE_LOGF(LogCEEffector, Log, "Linking effector %ls to cloner %ls after duplication", *Owner->GetActorNameOrLabel(), *ClonerExtension->GetClonerComponent()->GetOwner()->GetActorNameOrLabel())

				ClonerExtension->LinkEffector(Owner);
			}
		}
	}

	OnEffectorOptionsChanged();
}

void UCEEffectorComponent::PostLoad()
{
	Super::PostLoad();

	OnEffectorOptionsChanged();
}

void UCEEffectorComponent::PostDuplicate(EDuplicateMode::Type InDuplicateMode)
{
	Super::PostDuplicate(InDuplicateMode);

	OnEffectorOptionsChanged();
}

#if WITH_EDITOR
void UCEEffectorComponent::ForceRefreshLinkedCloners()
{
	RequestClonerUpdate(true);
}

const TCEPropertyChangeDispatcher<UCEEffectorComponent> UCEEffectorComponent::PropertyChangeDispatcher =
{
	/** Effector */
	{ GET_MEMBER_NAME_CHECKED(UCEEffectorComponent, bEnabled), &UCEEffectorComponent::OnEnabledChanged },
	{ GET_MEMBER_NAME_CHECKED(UCEEffectorComponent, Magnitude), &UCEEffectorComponent::OnEffectorOptionsChanged },
	{ GET_MEMBER_NAME_CHECKED(UCEEffectorComponent, Color), &UCEEffectorComponent::OnEffectorOptionsChanged },
	{ GET_MEMBER_NAME_CHECKED(UCEEffectorComponent, TypeName), &UCEEffectorComponent::OnTypeNameChanged },
	{ GET_MEMBER_NAME_CHECKED(UCEEffectorComponent, ModeName), &UCEEffectorComponent::OnModeNameChanged },
	{ GET_MEMBER_NAME_CHECKED(UCEEffectorComponent, bVisualizerComponentVisible), &UCEEffectorComponent::OnVisualizerOptionsChanged },
	{ GET_MEMBER_NAME_CHECKED(UCEEffectorComponent, bVisualizerSpriteVisible), &UCEEffectorComponent::OnVisualizerOptionsChanged },
};

void UCEEffectorComponent::PostEditChangeProperty(FPropertyChangedEvent& InPropertyChangedEvent)
{
	Super::PostEditChangeProperty(InPropertyChangedEvent);

	PropertyChangeDispatcher.OnPropertyChanged(this, InPropertyChangedEvent);
}

void UCEEffectorComponent::PostEditUndo()
{
	Super::PostEditUndo();

	if (IsValid(this))
	{
		OnEffectorOptionsChanged();
	}
	else
	{
		UnregisterFromChannel();
	}
}

void UCEEffectorComponent::SetVisualizerComponentVisible(bool bInVisible)
{
	if (bVisualizerComponentVisible == bInVisible)
	{
		return;
	}

	bVisualizerComponentVisible = bInVisible;
	OnVisualizerOptionsChanged();
}

void UCEEffectorComponent::SetVisualizerSpriteVisible(bool bInVisible)
{
	if (bVisualizerSpriteVisible == bInVisible)
	{
		return;
	}

	bVisualizerSpriteVisible = bInVisible;
	OnVisualizerOptionsChanged();
}

int32 UCEEffectorComponent::AddVisualizerComponent(UDynamicMeshComponent* InVisualizerComponent)
{
	if (!IsValid(InVisualizerComponent) || InVisualizerComponent->GetOwner() != GetOwner())
	{
		return INDEX_NONE;
	}

	int32 Index = VisualizerComponentsWeak.Find(InVisualizerComponent);

	if (Index != INDEX_NONE)
	{
		return Index;
	}

	Index = VisualizerComponentsWeak.Add(InVisualizerComponent);

	UMaterialInstanceDynamic* DynamicVisualizerMaterial = VisualizerMaterialsWeak.IsValidIndex(Index) ? VisualizerMaterialsWeak[Index].Get() : nullptr;

	if (UMaterialInterface* VisualizerMaterial = LoadObject<UMaterialInterface>(nullptr, VisualizerMaterialPath))
	{
		DynamicVisualizerMaterial = UMaterialInstanceDynamic::Create(
			VisualizerMaterial,
			this
		);
	}

	VisualizerMaterialsWeak.Add(DynamicVisualizerMaterial);

	InVisualizerComponent->SetHiddenInGame(true);
	InVisualizerComponent->SetTranslucentSortPriority(Index);
#if WITH_EDITOR
	InVisualizerComponent->SetIsVisualizationComponent(true);
#endif
	InVisualizerComponent->bIsEditorOnly = true;

	return Index;
}

void UCEEffectorComponent::UpdateVisualizer(int32 InVisualizerIndex, TFunctionRef<void(UDynamicMesh*)> InMeshFunction) const
{
	UDynamicMeshComponent* MeshComponent = VisualizerComponentsWeak.IsValidIndex(InVisualizerIndex) ? VisualizerComponentsWeak[InVisualizerIndex].Get() : nullptr;

	if (!IsValid(MeshComponent))
	{
		return;
	}

	UDynamicMesh* DynamicMesh = MeshComponent->GetDynamicMesh();

	DynamicMesh->EditMesh([](FDynamicMesh3& InMesh)
	{
		InMesh.Clear();
	});

	if (bVisualizerComponentVisible)
	{
		InMeshFunction(DynamicMesh);
	}

	UMaterialInstanceDynamic* VisualizerMaterial = VisualizerMaterialsWeak.IsValidIndex(InVisualizerIndex) ? VisualizerMaterialsWeak[InVisualizerIndex].Get() : nullptr;

	// Apply material
	if (IsValid(VisualizerMaterial))
	{
		MeshComponent->SetMaterial(0, VisualizerMaterial);
	}
}

void UCEEffectorComponent::SetVisualizerColor(int32 InVisualizerIndex, const FLinearColor& InColor)
{
	UMaterialInstanceDynamic* VisualizerMaterial = VisualizerMaterialsWeak.IsValidIndex(InVisualizerIndex) ? VisualizerMaterialsWeak[InVisualizerIndex].Get() : nullptr;

	if (!IsValid(VisualizerMaterial))
	{
		return;
	}

	VisualizerMaterial->SetVectorParameterValue(VisualizerColorName, InColor);
}
#endif

void UCEEffectorComponent::SetColor(const FLinearColor& InColor)
{
	if (Color.Equals(InColor))
	{
		return;
	}

	Color = InColor;
	OnEffectorOptionsChanged();
}

void UCEEffectorComponent::OnEnabledChanged()
{
	if (bEnabled)
	{
		OnEffectorEnabled();
	}
	else
	{
		OnEffectorDisabled();
	}
}

void UCEEffectorComponent::OnEffectorEnabled()
{
	if (ActiveType)
	{
		ActiveType->ActivateExtension();
	}

	if (ActiveMode)
	{
		ActiveMode->ActivateExtension();
	}

	for (const TObjectPtr<UCEEffectorEffectBase>& ActiveEffect : ActiveEffects)
	{
		if (ActiveEffect)
		{
			ActiveEffect->ActivateExtension();
		}
	}

	OnEffectorOptionsChanged();
}

void UCEEffectorComponent::OnEffectorDisabled()
{
	ChannelData.Magnitude = 0.f;

	if (ActiveType)
	{
		ActiveType->DeactivateExtension();
	}

	if (ActiveMode)
	{
		ActiveMode->DeactivateExtension();
	}

	for (const TObjectPtr<UCEEffectorEffectBase>& ActiveEffect : ActiveEffects)
	{
		if (ActiveEffect)
		{
			ActiveEffect->DeactivateExtension();
		}
	}

#if WITH_EDITOR
	OnVisualizerOptionsChanged();
#endif
}

void UCEEffectorComponent::OnEffectorSetEnabled(const UWorld* InWorld, bool bInEnabled, bool bInTransact)
{
	if (GetWorld() == InWorld)
	{
#if WITH_EDITOR
		if (bInTransact)
		{
			Modify();
		}
#endif

		SetEnabled(bInEnabled);
	}
}

void UCEEffectorComponent::OnEffectorOptionsChanged()
{
	RegisterToChannel();

	// General
	ChannelData.Magnitude = bEnabled ? GetMagnitude() : 0.f;
	ChannelData.Color = Color;

	// Effector Transform
	ChannelData.Location = GetComponentLocation();
	ChannelData.Rotation = GetComponentRotation().Quaternion();
	ChannelData.Scale = GetComponentScale();

	// Update Type
	OnTypeNameChanged();

	// Update Mode & Effects
	OnModeNameChanged();

#if WITH_EDITOR
	OnVisualizerOptionsChanged();
#endif
}

void UCEEffectorComponent::OnTypeNameChanged()
{
	const TArray<FName> TypeNames = GetEffectorTypeNames();

	// Set default if value does not exists
	if (!TypeNames.Contains(TypeName) && !TypeNames.IsEmpty())
	{
		TypeName = TypeNames[0];
		
		for (const FName& ExtensionName : TypeNames)
		{
			if (const UCEEffectorExtensionBase* ExtensionType = GetExtension(ExtensionName))
			{
				if (ExtensionType->RedirectExtensionName(TypeName))
				{
					TypeName = ExtensionType->GetExtensionName();
					break;
				}
			}
		}
	}

	if (UCEEffectorTypeBase* Type = Cast<UCEEffectorTypeBase>(FindOrAddExtension(TypeName)))
	{
		if (ActiveType != Type)
		{
			if (ActiveType)
			{
				ActiveType->DeactivateExtension();
			}

			ActiveType = Type;

			if (ActiveType)
			{
				ActiveType->ActivateExtension();
				ActiveType->UpdateExtensionParameters();
			}
		}
		else
		{
			ActiveType->UpdateExtensionParameters();
		}
	}
}

void UCEEffectorComponent::OnModeNameChanged()
{
	const TArray<FName> ModeNames = GetEffectorModeNames();

	// Set default if value does not exists
	if (!ModeNames.Contains(ModeName) && !ModeNames.IsEmpty())
	{
		ModeName = ModeNames[0];
		
		for (const FName& ExtensionName : ModeNames)
		{
			if (const UCEEffectorExtensionBase* ExtensionMode = GetExtension(ExtensionName))
			{
				if (ExtensionMode->RedirectExtensionName(ModeName))
				{
					ModeName = ExtensionMode->GetExtensionName();
					break;
				}
			}
		}
	}

	if (UCEEffectorModeBase* Mode = Cast<UCEEffectorModeBase>(FindOrAddExtension(ModeName)))
	{
		if (ActiveMode != Mode)
		{
			if (ActiveMode)
			{
				ActiveMode->DeactivateExtension();
			}

			ActiveMode = Mode;

			if (ActiveMode)
			{
				ActiveMode->ActivateExtension();
				ActiveMode->UpdateExtensionParameters();
			}
		}
		else
		{
			ActiveMode->UpdateExtensionParameters();
		}
	}

	OnEffectsChanged();
}

void UCEEffectorComponent::OnEffectsChanged()
{
	if (!ActiveMode)
	{
		return;
	}

	TArray<TObjectPtr<UCEEffectorEffectBase>> RemoveEffects = ActiveEffects;
	
	for (const TSubclassOf<UCEEffectorEffectBase>& SupportedEffect : ActiveMode->GetSupportedEffects())
	{
		if (UCEEffectorEffectBase* Effect = Cast<UCEEffectorEffectBase>(FindOrAddExtension(SupportedEffect)))
		{
			if (RemoveEffects.Remove(Effect) == 0)
			{
				Effect->ActivateExtension();

				ActiveEffects.Add(Effect);
			}

			Effect->UpdateExtensionParameters();
		}
	}

	for (const TObjectPtr<UCEEffectorEffectBase>& ActiveEffect : RemoveEffects)
	{
		if (IsValid(ActiveEffect))
		{
			ActiveEffect->DeactivateExtension();

			ActiveEffects.Remove(ActiveEffect);
		}
	}
}

#if WITH_EDITOR
void UCEEffectorComponent::OnVisualizerOptionsChanged()
{
	// Only run this while in editor
	if (!GIsEditor)
	{
		return;
	}

	for (const TWeakObjectPtr<UDynamicMeshComponent>& VisualizerComponentWeak : VisualizerComponentsWeak)
	{
		if (UDynamicMeshComponent* VisualizerComponent = VisualizerComponentWeak.Get())
		{
			VisualizerComponent->SetVisibility(bEnabled && bVisualizerComponentVisible, false);
		}
	}

	if (UTexture2D* SpriteTexture = LoadObject<UTexture2D>(nullptr, SpriteTexturePath))
	{
		if (IsValid(GetWorld()))
		{
			CreateSpriteComponent(SpriteTexture);
		}

		if (IsValid(SpriteComponent))
		{
			if (SpriteComponent->Sprite != SpriteTexture)
			{
				SpriteComponent->SetSprite(SpriteTexture);
			}

			SpriteComponent->SetVisibility(bVisualizerSpriteVisible, /** bPropagateToChildren */false);
		}
	}
}
#endif

TArray<FName> UCEEffectorComponent::GetEffectorTypeNames() const
{
	TArray<FName> TypeNames;

	if (const UCEEffectorSubsystem* EffectorSubsystem = UCEEffectorSubsystem::Get())
	{
		TypeNames = EffectorSubsystem->GetExtensionNames<UCEEffectorTypeBase>().Array();
	}

	return TypeNames;
}

TArray<FName> UCEEffectorComponent::GetEffectorModeNames() const
{
	TArray<FName> ModeNames;

	if (const UCEEffectorSubsystem* EffectorSubsystem = UCEEffectorSubsystem::Get())
	{
		ModeNames = EffectorSubsystem->GetExtensionNames<UCEEffectorModeBase>().Array();
	}

	return ModeNames;
}

UCEEffectorExtensionBase* UCEEffectorComponent::FindOrAddExtension(TSubclassOf<UCEEffectorExtensionBase> InClass)
{
	const UCEEffectorSubsystem* Subsystem = UCEEffectorSubsystem::Get();

	if (!Subsystem)
	{
		return nullptr;
	}

	const FName ExtensionName = Subsystem->FindExtensionName(InClass);

	if (ExtensionName.IsNone())
	{
		return nullptr;
	}

	return FindOrAddExtension(ExtensionName);
}

UCEEffectorExtensionBase* UCEEffectorComponent::FindOrAddExtension(FName InExtensionName)
{
	// Check cached extension instances
	UCEEffectorExtensionBase* NewActiveExtension = nullptr;
	for (TObjectPtr<UCEEffectorExtensionBase>& ExtensionInstance : ExtensionInstances)
	{
		if (IsValid(ExtensionInstance) && ExtensionInstance->GetExtensionName() == InExtensionName)
		{
			NewActiveExtension = ExtensionInstance;
			break;
		}
	}

	// Create new extension instance and cache it
	if (!NewActiveExtension)
	{
		UCEEffectorSubsystem* Subsystem = UCEEffectorSubsystem::Get();
		if (!Subsystem)
		{
			return nullptr;
		}

		NewActiveExtension = Subsystem->CreateNewExtension(InExtensionName, this);
		ExtensionInstances.Add(NewActiveExtension);
	}

	return NewActiveExtension;
}
