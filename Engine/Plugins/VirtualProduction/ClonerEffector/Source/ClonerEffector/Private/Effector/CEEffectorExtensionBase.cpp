// Copyright Epic Games, Inc. All Rights Reserved.

#include "Effector/CEEffectorExtensionBase.h"

#include "Effector/CEEffectorComponent.h"

#if WITH_EDITOR
FCEExtensionSection UCEEffectorExtensionBase::GetExtensionSection() const
{
	return UE::ClonerEffector::EditorSection::GetExtensionSectionFromClass(GetClass());
}
#endif

UCEEffectorComponent* UCEEffectorExtensionBase::GetEffectorComponent() const
{
	return GetTypedOuter<UCEEffectorComponent>();
}

void UCEEffectorExtensionBase::UpdateExtensionParameters(bool bInUpdateLinkedCloners)
{
	if (!IsExtensionActive())
	{
		return;
	}

	if (UCEEffectorComponent* EffectorComponent = GetEffectorComponent())
	{
		if (!EffectorComponent->GetEnabled())
		{
			return;
		}

		OnExtensionParametersChanged(EffectorComponent);

		if (bInUpdateLinkedCloners)
		{
			EffectorComponent->RequestClonerUpdate(/** Immediate */false);
		}
	}
}

void UCEEffectorExtensionBase::ActivateExtension()
{
	if (!bExtensionActive)
	{
		bExtensionActive = true;
		OnExtensionActivated();
		UpdateExtensionParameters();
	}
}

void UCEEffectorExtensionBase::DeactivateExtension()
{
	if (bExtensionActive)
	{
		bExtensionActive = false;
		OnExtensionDeactivated();
	}
}

void UCEEffectorExtensionBase::PostEditImport()
{
	Super::PostEditImport();

	UpdateExtensionParameters();
}

void UCEEffectorExtensionBase::OnExtensionPropertyChanged()
{
	UpdateExtensionParameters();
}
