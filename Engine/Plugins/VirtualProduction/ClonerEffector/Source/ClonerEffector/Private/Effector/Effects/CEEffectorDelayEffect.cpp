// Copyright Epic Games, Inc. All Rights Reserved.

#include "Effector/Effects/CEEffectorDelayEffect.h"

#include "Effector/CEEffectorComponent.h"

void UCEEffectorDelayEffect::SetDelayEnabled(bool bInEnabled)
{
	if (bDelayEnabled == bInEnabled)
	{
		return;
	}

	bDelayEnabled = bInEnabled;
	UpdateExtensionParameters();
}

void UCEEffectorDelayEffect::SetDelayInDuration(float InDuration)
{
	InDuration = FMath::Max(InDuration, 0.f);
	if (FMath::IsNearlyEqual(DelayInDuration, InDuration))
	{
		return;
	}

	DelayInDuration = InDuration;
	UpdateExtensionParameters();
}

void UCEEffectorDelayEffect::SetDelayOutDuration(float InDuration)
{
	InDuration = FMath::Max(InDuration, 0.f);
	if (FMath::IsNearlyEqual(DelayOutDuration, InDuration))
	{
		return;
	}

	DelayOutDuration = InDuration;
	UpdateExtensionParameters();	
}

void UCEEffectorDelayEffect::SetDelaySpringFrequency(float InFrequency)
{
	InFrequency = FMath::Max(InFrequency, 1.f);
	if (FMath::IsNearlyEqual(DelaySpringFrequency, InFrequency))
	{
		return;
	}

	DelaySpringFrequency = InFrequency;
	UpdateExtensionParameters();
}

void UCEEffectorDelayEffect::SetDelaySpringFalloff(float InFalloff)
{
	InFalloff = FMath::Max(InFalloff, 1.f);
	if (FMath::IsNearlyEqual(DelaySpringFalloff, InFalloff))
	{
		return;
	}

	DelaySpringFalloff = InFalloff;
	UpdateExtensionParameters();
}

#if WITH_EDITOR
const TCEPropertyChangeDispatcher<UCEEffectorDelayEffect> UCEEffectorDelayEffect::PropertyChangeDispatcher =
{
	{ GET_MEMBER_NAME_CHECKED(UCEEffectorDelayEffect, bDelayEnabled), &UCEEffectorDelayEffect::OnExtensionPropertyChanged },
	{ GET_MEMBER_NAME_CHECKED(UCEEffectorDelayEffect, DelayInDuration), &UCEEffectorDelayEffect::OnExtensionPropertyChanged },
	{ GET_MEMBER_NAME_CHECKED(UCEEffectorDelayEffect, DelaySpringFrequency), &UCEEffectorDelayEffect::OnExtensionPropertyChanged },
	{ GET_MEMBER_NAME_CHECKED(UCEEffectorDelayEffect, DelaySpringFalloff), &UCEEffectorDelayEffect::OnExtensionPropertyChanged }
};

void UCEEffectorDelayEffect::PostEditChangeProperty(FPropertyChangedEvent& InPropertyChangedEvent)
{
	Super::PostEditChangeProperty(InPropertyChangedEvent);

	PropertyChangeDispatcher.OnPropertyChanged(this, InPropertyChangedEvent);
}
#endif

void UCEEffectorDelayEffect::UpdateEffectChannelData(FCEClonerEffectorChannelData& InChannelData, bool bInEnabled)
{
	Super::UpdateEffectChannelData(InChannelData, bInEnabled);

	bInEnabled &= bDelayEnabled;
	
	if (bInEnabled)
	{
		InChannelData.DelayInDuration = DelayInDuration;
		InChannelData.DelayOutDuration = DelayOutDuration;
		InChannelData.DelaySpringFrequency = DelaySpringFrequency * DelayOutDuration;
		InChannelData.DelaySpringFalloff = DelaySpringFalloff;
	}
	else
	{
		InChannelData.DelayInDuration = 0.f;
		InChannelData.DelayOutDuration = 0.f;
	}
}
