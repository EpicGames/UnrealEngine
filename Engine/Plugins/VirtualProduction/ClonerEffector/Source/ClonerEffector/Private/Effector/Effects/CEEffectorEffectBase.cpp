// Copyright Epic Games, Inc. All Rights Reserved.

#include "Effector/Effects/CEEffectorEffectBase.h"

#include "Effector/CEEffectorComponent.h"

void UCEEffectorEffectBase::OnExtensionParametersChanged(UCEEffectorComponent* InComponent)
{
	Super::OnExtensionParametersChanged(InComponent);
	
	UpdateEffectChannelData(InComponent->GetChannelData(), /** Enabled */true);
}

void UCEEffectorEffectBase::OnExtensionDeactivated()
{
	Super::OnExtensionDeactivated();

	if (UCEEffectorComponent* EffectorComponent = GetEffectorComponent())
	{
		UpdateEffectChannelData(EffectorComponent->GetChannelData(), /** Enabled */false);
	}
}
