// Copyright Epic Games, Inc. All Rights Reserved.

#include "Effector/Effects/CEEffectorForceEffect.h"

#include "CEClonerEffectorShared.h"
#include "Effector/CEEffectorComponent.h"

void UCEEffectorForceEffect::SetForcesEnabled(bool bInForcesEnabled)
{
	if (bForcesEnabled == bInForcesEnabled)
	{
		return;
	}

	bForcesEnabled = bInForcesEnabled;
	UpdateExtensionParameters(/** UpdateCloners */ true);
}

void UCEEffectorForceEffect::SetAttractionForceStrength(float InForceStrength)
{
	if (FMath::IsNearlyEqual(AttractionForceStrength, InForceStrength))
	{
		return;
	}

	AttractionForceStrength = InForceStrength;
	UpdateExtensionParameters();
}

void UCEEffectorForceEffect::SetAttractionForceFalloff(float InForceFalloff)
{
	if (FMath::IsNearlyEqual(AttractionForceFalloff, InForceFalloff))
	{
		return;
	}

	AttractionForceFalloff = InForceFalloff;
	UpdateExtensionParameters();
}

void UCEEffectorForceEffect::SetGravityForceEnabled(bool bInForceEnabled)
{
	if (bGravityForceEnabled == bInForceEnabled)
	{
		return;
	}

	bGravityForceEnabled = bInForceEnabled;
	UpdateExtensionParameters(/** UpdateCloners */ true);
}

void UCEEffectorForceEffect::SetGravityForceAcceleration(const FVector& InAcceleration)
{
	if (GravityForceAcceleration.Equals(InAcceleration))
	{
		return;
	}

	GravityForceAcceleration = InAcceleration;
	UpdateExtensionParameters();
}

void UCEEffectorForceEffect::SetDragForceEnabled(bool bInEnabled)
{
	if (bDragForceEnabled == bInEnabled)
	{
		return;
	}

	bDragForceEnabled = bInEnabled;
	UpdateExtensionParameters(/** UpdateCloners */ true);
}

void UCEEffectorForceEffect::SetDragForceLinear(float InStrength)
{
	if (FMath::IsNearlyEqual(DragForceLinear, InStrength))
	{
		return;
	}

	DragForceLinear = InStrength;
	UpdateExtensionParameters();
}

void UCEEffectorForceEffect::SetDragForceRotational(float InStrength)
{
	if (FMath::IsNearlyEqual(DragForceRotational, InStrength))
	{
		return;
	}

	DragForceRotational = InStrength;
	UpdateExtensionParameters();
}

void UCEEffectorForceEffect::SetVectorNoiseForceEnabled(bool bInEnabled)
{
	if (bVectorNoiseForceEnabled == bInEnabled)
	{
		return;
	}

	bVectorNoiseForceEnabled = bInEnabled;
	UpdateExtensionParameters(/** UpdateCloners */ true);
}

void UCEEffectorForceEffect::SetVectorNoiseForceAmount(float InAmount)
{
	if (FMath::IsNearlyEqual(VectorNoiseForceAmount, InAmount))
	{
		return;
	}

	VectorNoiseForceAmount = InAmount;
	UpdateExtensionParameters();
}

void UCEEffectorForceEffect::SetOrientationForceEnabled(bool bInForceEnabled)
{
	if (bOrientationForceEnabled == bInForceEnabled)
	{
		return;
	}

	bOrientationForceEnabled = bInForceEnabled;
	UpdateExtensionParameters(/** UpdateCloners */ true);
}

void UCEEffectorForceEffect::SetOrientationForceRate(float InForceOrientationRate)
{
	if (FMath::IsNearlyEqual(OrientationForceRate, InForceOrientationRate))
	{
		return;
	}

	OrientationForceRate = InForceOrientationRate;
	UpdateExtensionParameters();
}

void UCEEffectorForceEffect::SetOrientationForceMin(const FVector& InForceOrientationMin)
{
	if (OrientationForceMin.Equals(InForceOrientationMin))
	{
		return;
	}

	OrientationForceMin = InForceOrientationMin;
	UpdateExtensionParameters();
}

void UCEEffectorForceEffect::SetOrientationForceMax(const FVector& InForceOrientationMax)
{
	if (OrientationForceMax.Equals(InForceOrientationMax))
	{
		return;
	}

	OrientationForceMax = InForceOrientationMax;
	UpdateExtensionParameters();
}

void UCEEffectorForceEffect::SetVortexForceEnabled(bool bInForceEnabled)
{
	if (bVortexForceEnabled == bInForceEnabled)
	{
		return;
	}

	bVortexForceEnabled = bInForceEnabled;
	UpdateExtensionParameters(/** UpdateCloners */ true);
}

void UCEEffectorForceEffect::SetVortexForceAmount(float InForceVortexAmount)
{
	if (FMath::IsNearlyEqual(VortexForceAmount, InForceVortexAmount))
	{
		return;
	}

	VortexForceAmount = InForceVortexAmount;
	UpdateExtensionParameters();
}

void UCEEffectorForceEffect::SetVortexForceAxis(const FVector& InForceVortexAxis)
{
	if (VortexForceAxis.Equals(InForceVortexAxis))
	{
		return;
	}

	VortexForceAxis = InForceVortexAxis;
	UpdateExtensionParameters();
}

void UCEEffectorForceEffect::SetCurlNoiseForceEnabled(bool bInForceEnabled)
{
	if (bCurlNoiseForceEnabled == bInForceEnabled)
	{
		return;
	}

	bCurlNoiseForceEnabled = bInForceEnabled;
	UpdateExtensionParameters(/** UpdateCloners */ true);
}

void UCEEffectorForceEffect::SetCurlNoiseForceStrength(float InForceCurlNoiseStrength)
{
	if (FMath::IsNearlyEqual(CurlNoiseForceStrength, InForceCurlNoiseStrength))
	{
		return;
	}

	CurlNoiseForceStrength = InForceCurlNoiseStrength;
	UpdateExtensionParameters();
}

void UCEEffectorForceEffect::SetCurlNoiseForceFrequency(float InForceCurlNoiseFrequency)
{
	if (FMath::IsNearlyEqual(CurlNoiseForceFrequency, InForceCurlNoiseFrequency))
	{
		return;
	}

	CurlNoiseForceFrequency = InForceCurlNoiseFrequency;
	UpdateExtensionParameters();
}

void UCEEffectorForceEffect::SetAttractionForceEnabled(bool bInForceEnabled)
{
	if (bAttractionForceEnabled == bInForceEnabled)
	{
		return;
	}

	bAttractionForceEnabled = bInForceEnabled;
	UpdateExtensionParameters(/** UpdateCloners */ true);
}

void UCEEffectorForceEffect::UpdateEffectChannelData(FCEClonerEffectorChannelData& InChannelData, bool bInEnabled)
{
	Super::UpdateEffectChannelData(InChannelData, bInEnabled);

	bInEnabled &= bForcesEnabled;

	if (bInEnabled && bOrientationForceEnabled)
	{
		InChannelData.OrientationForceRate = OrientationForceRate;
		InChannelData.OrientationForceMin = OrientationForceMin;
		InChannelData.OrientationForceMax = OrientationForceMax;
	}
	else
	{
		InChannelData.OrientationForceRate = 0.f;
		InChannelData.OrientationForceMin = FVector::ZeroVector;
		InChannelData.OrientationForceMax = FVector::ZeroVector;
	}

	if (bInEnabled && bVortexForceEnabled)
	{
		InChannelData.VortexForceAmount = VortexForceAmount;
		InChannelData.VortexForceAxis = VortexForceAxis;
	}
	else
	{
		InChannelData.VortexForceAmount = 0.f;
		InChannelData.VortexForceAxis = FVector::ZeroVector;
	}

	if (bInEnabled && bCurlNoiseForceEnabled)
	{
		InChannelData.CurlNoiseForceStrength = CurlNoiseForceStrength;
		InChannelData.CurlNoiseForceFrequency = CurlNoiseForceFrequency;
	}
	else
	{
		InChannelData.CurlNoiseForceStrength = 0.f;
		InChannelData.CurlNoiseForceFrequency = 0.f;
	}

	if (bInEnabled && bAttractionForceEnabled)
	{
		InChannelData.AttractionForceStrength = AttractionForceStrength;
		InChannelData.AttractionForceFalloff = AttractionForceFalloff;
	}
	else
	{
		InChannelData.AttractionForceStrength = 0.f;
		InChannelData.AttractionForceFalloff = 0.f;
	}

	if (bInEnabled && bGravityForceEnabled)
	{
		InChannelData.GravityForceAcceleration = GravityForceAcceleration;
	}
	else
	{
		InChannelData.GravityForceAcceleration = FVector::ZeroVector;
	}

	if (bInEnabled && bDragForceEnabled)
	{
		InChannelData.DragForceLinear = DragForceLinear;
		InChannelData.DragForceRotational = DragForceRotational;
	}
	else
	{
		InChannelData.DragForceLinear = 0.f;
		InChannelData.DragForceRotational = 0.f;
	}

	if (bInEnabled && bVectorNoiseForceEnabled)
	{
		InChannelData.VectorNoiseForceAmount = VectorNoiseForceAmount;
	}
	else
	{
		InChannelData.VectorNoiseForceAmount = 0.f;
	}
}

void UCEEffectorForceEffect::OnForceOptionsChanged()
{
	UpdateExtensionParameters(/** UpdateCloners */ true);
}

void UCEEffectorForceEffect::OnExtensionDeactivated()
{
	Super::OnExtensionDeactivated();

	// Special case for forces to reset state
	if (UCEEffectorComponent* EffectorComponent = GetEffectorComponent())
	{
		EffectorComponent->RequestClonerUpdate(/** Immediate */false);
	}
}

#if WITH_EDITOR
const TCEPropertyChangeDispatcher<UCEEffectorForceEffect> UCEEffectorForceEffect::PropertyChangeDispatcher =
{
	/** Force */
	{ GET_MEMBER_NAME_CHECKED(UCEEffectorForceEffect, bForcesEnabled), &UCEEffectorForceEffect::OnForceOptionsChanged },
	{ GET_MEMBER_NAME_CHECKED(UCEEffectorForceEffect, bOrientationForceEnabled), &UCEEffectorForceEffect::OnForceOptionsChanged },
	{ GET_MEMBER_NAME_CHECKED(UCEEffectorForceEffect, OrientationForceRate), &UCEEffectorForceEffect::OnForceOptionsChanged },
	{ GET_MEMBER_NAME_CHECKED(UCEEffectorForceEffect, OrientationForceMin), &UCEEffectorForceEffect::OnForceOptionsChanged },
	{ GET_MEMBER_NAME_CHECKED(UCEEffectorForceEffect, OrientationForceMax), &UCEEffectorForceEffect::OnForceOptionsChanged },
	{ GET_MEMBER_NAME_CHECKED(UCEEffectorForceEffect, bVortexForceEnabled), &UCEEffectorForceEffect::OnForceOptionsChanged },
	{ GET_MEMBER_NAME_CHECKED(UCEEffectorForceEffect, VortexForceAmount), &UCEEffectorForceEffect::OnForceOptionsChanged },
	{ GET_MEMBER_NAME_CHECKED(UCEEffectorForceEffect, VortexForceAxis), &UCEEffectorForceEffect::OnForceOptionsChanged },
	{ GET_MEMBER_NAME_CHECKED(UCEEffectorForceEffect, bCurlNoiseForceEnabled), &UCEEffectorForceEffect::OnForceOptionsChanged },
	{ GET_MEMBER_NAME_CHECKED(UCEEffectorForceEffect, CurlNoiseForceStrength), &UCEEffectorForceEffect::OnForceOptionsChanged },
	{ GET_MEMBER_NAME_CHECKED(UCEEffectorForceEffect, CurlNoiseForceFrequency), &UCEEffectorForceEffect::OnForceOptionsChanged },
	{ GET_MEMBER_NAME_CHECKED(UCEEffectorForceEffect, bAttractionForceEnabled), &UCEEffectorForceEffect::OnForceOptionsChanged },
	{ GET_MEMBER_NAME_CHECKED(UCEEffectorForceEffect, AttractionForceStrength), &UCEEffectorForceEffect::OnForceOptionsChanged },
	{ GET_MEMBER_NAME_CHECKED(UCEEffectorForceEffect, AttractionForceFalloff), &UCEEffectorForceEffect::OnForceOptionsChanged },
	{ GET_MEMBER_NAME_CHECKED(UCEEffectorForceEffect, bGravityForceEnabled), &UCEEffectorForceEffect::OnForceOptionsChanged },
	{ GET_MEMBER_NAME_CHECKED(UCEEffectorForceEffect, GravityForceAcceleration), &UCEEffectorForceEffect::OnForceOptionsChanged },
	{ GET_MEMBER_NAME_CHECKED(UCEEffectorForceEffect, bDragForceEnabled), &UCEEffectorForceEffect::OnForceOptionsChanged },
	{ GET_MEMBER_NAME_CHECKED(UCEEffectorForceEffect, DragForceLinear), &UCEEffectorForceEffect::OnForceOptionsChanged },
	{ GET_MEMBER_NAME_CHECKED(UCEEffectorForceEffect, DragForceRotational), &UCEEffectorForceEffect::OnForceOptionsChanged },
	{ GET_MEMBER_NAME_CHECKED(UCEEffectorForceEffect, bVectorNoiseForceEnabled), &UCEEffectorForceEffect::OnForceOptionsChanged },
	{ GET_MEMBER_NAME_CHECKED(UCEEffectorForceEffect, VectorNoiseForceAmount), &UCEEffectorForceEffect::OnForceOptionsChanged },
};

void UCEEffectorForceEffect::PostEditChangeProperty(FPropertyChangedEvent& InPropertyChangedEvent)
{
	Super::PostEditChangeProperty(InPropertyChangedEvent);

	PropertyChangeDispatcher.OnPropertyChanged(this, InPropertyChangedEvent);
}
#endif
