// Copyright Epic Games, Inc. All Rights Reserved.

#include "Effector/Modes/CEEffectorCancelMode.h"

#include "Effector/Effects/CEEffectorDelayEffect.h"
#include "Effector/Effects/CEEffectorForceEffect.h"

bool UCEEffectorCancelMode::IsEffectSupported(TSubclassOf<UCEEffectorEffectBase> InEffectClass) const
{
	return !(InEffectClass->IsChildOf<UCEEffectorDelayEffect>() || InEffectClass->IsChildOf<UCEEffectorForceEffect>());
}
