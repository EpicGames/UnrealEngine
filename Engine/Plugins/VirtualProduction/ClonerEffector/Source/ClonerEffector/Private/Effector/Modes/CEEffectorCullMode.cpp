// Copyright Epic Games, Inc. All Rights Reserved.

#include "Effector/Modes/CEEffectorCullMode.h"

#include "Effector/CEEffectorComponent.h"
#include "NiagaraTypes.h"
#include "NiagaraTypeRegistry.h"

void UCEEffectorCullMode::SetBehavior(ECEEffectorCullModeBehavior InBehavior)
{
	if (Behavior == InBehavior)
	{
		return;
	}

	Behavior = InBehavior;
	OnBehaviorChanged();
}

void UCEEffectorCullMode::SetScale(const FVector& InScale)
{
	const FVector NewScale = InScale.ComponentMax(FVector(UE_KINDA_SMALL_NUMBER));

	if (NewScale.Equals(Scale))
	{
		return;
	}

	Scale = NewScale;
	UpdateExtensionParameters();
}

void UCEEffectorCullMode::OnExtensionParametersChanged(UCEEffectorComponent* InComponent)
{
	Super::OnExtensionParametersChanged(InComponent);

	FCEClonerEffectorChannelData& ChannelData = InComponent->GetChannelData();

	ChannelData.LocationDelta = FVector::ZeroVector;
	ChannelData.RotationDelta = FVector::ZeroVector;
	ChannelData.ScaleDelta = Scale.ComponentMax(FVector(UE_KINDA_SMALL_NUMBER));
	ChannelData.Frequency = static_cast<float>(Behavior);
}

void UCEEffectorCullMode::PostInitProperties()
{
	Super::PostInitProperties();

	if (HasAnyFlags(RF_ClassDefaultObject))
	{
		// Register new type def for niagara

		constexpr ENiagaraTypeRegistryFlags MeshFlags =
			ENiagaraTypeRegistryFlags::AllowAnyVariable |
			ENiagaraTypeRegistryFlags::AllowParameter;

		FNiagaraTypeRegistry::Register(FNiagaraTypeDefinition(StaticEnum<ECEEffectorCullModeBehavior>()), MeshFlags);
	}
}

void UCEEffectorCullMode::OnBehaviorChanged()
{
	UpdateExtensionParameters(/** UpdateCloner */true);		
}

#if WITH_EDITOR
const TCEPropertyChangeDispatcher<UCEEffectorCullMode> UCEEffectorCullMode::PropertyChangeDispatcher =
{
	{ GET_MEMBER_NAME_CHECKED(UCEEffectorCullMode, Behavior), &UCEEffectorCullMode::OnBehaviorChanged },
	{ GET_MEMBER_NAME_CHECKED(UCEEffectorCullMode, Scale), &UCEEffectorCullMode::OnExtensionPropertyChanged },
};

void UCEEffectorCullMode::PostEditChangeProperty(FPropertyChangedEvent& InPropertyChangedEvent)
{
	Super::PostEditChangeProperty(InPropertyChangedEvent);

	PropertyChangeDispatcher.OnPropertyChanged(this, InPropertyChangedEvent);
}
#endif
