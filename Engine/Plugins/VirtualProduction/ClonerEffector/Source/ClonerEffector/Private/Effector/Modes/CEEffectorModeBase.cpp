// Copyright Epic Games, Inc. All Rights Reserved.

#include "Effector/Modes/CEEffectorModeBase.h"

#include "CEClonerEffectorShared.h"
#include "Effector/CEEffectorComponent.h"
#include "Effector/Effects/CEEffectorEffectBase.h"
#include "Subsystems/CEEffectorSubsystem.h"

TSet<TSubclassOf<UCEEffectorEffectBase>> UCEEffectorModeBase::GetSupportedEffects() const
{
	TSet<TSubclassOf<UCEEffectorEffectBase>> SupportedEffects;

	if (const UCEEffectorSubsystem* EffectorSubsystem = UCEEffectorSubsystem::Get())
	{
		TSubclassOf<UCEEffectorModeBase> ModeClass(GetClass());
		
		for (const TSubclassOf<UCEEffectorExtensionBase>& ExtensionClass : EffectorSubsystem->GetExtensionClasses(UCEEffectorEffectBase::StaticClass()))
		{
			TSubclassOf<UCEEffectorEffectBase> EffectClass(ExtensionClass);
			
			const UCEEffectorEffectBase* Effect = EffectClass.GetDefaultObject();

			if (!Effect)
			{
				continue;
			}

			// Does the mode supports this effect
			if (!IsEffectSupported(EffectClass))
			{
				continue;
			}

			// Does the effect supports this mode
			if (!Effect->IsModeSupported(ModeClass))
			{
				continue;
			}

			SupportedEffects.Add(EffectClass);
		}
	}

	return SupportedEffects;
}

void UCEEffectorModeBase::OnExtensionParametersChanged(UCEEffectorComponent* InComponent)
{
	Super::OnExtensionParametersChanged(InComponent);

	FCEClonerEffectorChannelData& ChannelData = InComponent->GetChannelData();
	ChannelData.Mode = static_cast<ECEClonerEffectorMode>(ModeIdentifier);
}
