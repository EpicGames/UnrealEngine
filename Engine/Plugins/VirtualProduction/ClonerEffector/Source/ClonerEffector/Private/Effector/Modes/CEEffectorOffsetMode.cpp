// Copyright Epic Games, Inc. All Rights Reserved.

#include "Effector/Modes/CEEffectorOffsetMode.h"

#include "CEClonerEffectorShared.h"
#include "Effector/CEEffectorComponent.h"

void UCEEffectorOffsetMode::SetOffset(const FVector& InOffset)
{
	if (InOffset.Equals(Offset))
	{
		return;
	}

	Offset = InOffset;
	UpdateExtensionParameters();
}

void UCEEffectorOffsetMode::SetRotation(const FRotator& InRotation)
{
	if (InRotation.Equals(Rotation))
	{
		return;
	}

	constexpr float MinRotation = -180.f;
	if (InRotation.Pitch < MinRotation || InRotation.Roll < MinRotation || InRotation.Yaw < MinRotation)
	{
		return;
	}

	constexpr float MaxRotation = 180.f;
	if (InRotation.Pitch > MaxRotation || InRotation.Roll > MaxRotation || InRotation.Yaw > MaxRotation)
	{
		return;
	}

	Rotation = InRotation;
	UpdateExtensionParameters();
}

void UCEEffectorOffsetMode::SetScale(const FVector& InScale)
{
	const FVector NewScale = InScale.ComponentMax(FVector(UE_KINDA_SMALL_NUMBER));

	if (NewScale.Equals(Scale))
	{
		return;
	}

	Scale = NewScale;
	UpdateExtensionParameters();
}

void UCEEffectorOffsetMode::OnExtensionParametersChanged(UCEEffectorComponent* InComponent)
{
	Super::OnExtensionParametersChanged(InComponent);

	FCEClonerEffectorChannelData& ChannelData = InComponent->GetChannelData();
	ChannelData.LocationDelta = Offset;
	ChannelData.RotationDelta = FVector(Rotation.Yaw, Rotation.Pitch, Rotation.Roll);
	ChannelData.ScaleDelta = Scale.ComponentMax(FVector(UE_KINDA_SMALL_NUMBER));
}

#if WITH_EDITOR
const TCEPropertyChangeDispatcher<UCEEffectorOffsetMode> UCEEffectorOffsetMode::PropertyChangeDispatcher =
{
	{ GET_MEMBER_NAME_CHECKED(UCEEffectorOffsetMode, Offset), &UCEEffectorOffsetMode::OnExtensionPropertyChanged },
	{ GET_MEMBER_NAME_CHECKED(UCEEffectorOffsetMode, Rotation), &UCEEffectorOffsetMode::OnExtensionPropertyChanged },
	{ GET_MEMBER_NAME_CHECKED(UCEEffectorOffsetMode, Scale), &UCEEffectorOffsetMode::OnExtensionPropertyChanged },
};

void UCEEffectorOffsetMode::PostEditChangeProperty(FPropertyChangedEvent& InPropertyChangedEvent)
{
	Super::PostEditChangeProperty(InPropertyChangedEvent);

	PropertyChangeDispatcher.OnPropertyChanged(this, InPropertyChangedEvent);
}
#endif
