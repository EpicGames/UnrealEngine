// Copyright Epic Games, Inc. All Rights Reserved.

#include "Effector/Modes/CEEffectorProceduralMode.h"

#include "CEClonerEffectorShared.h"
#include "Effector/CEEffectorComponent.h"
#include "NiagaraTypeRegistry.h"

void UCEEffectorProceduralMode::SetPattern(ECEClonerEffectorProceduralPattern InPattern)
{
	if (InPattern == Pattern)
	{
		return;
	}

	Pattern = InPattern;
	UpdateExtensionParameters();
}

void UCEEffectorProceduralMode::SetLocationStrength(const FVector& InStrength)
{
	if (LocationStrength.Equals(InStrength))
	{
		return;
	}

	LocationStrength = InStrength;
	UpdateExtensionParameters();
}

void UCEEffectorProceduralMode::SetRotationStrength(const FRotator& InStrength)
{
	if (RotationStrength.Equals(InStrength))
	{
		return;
	}

	RotationStrength = InStrength;
	UpdateExtensionParameters();
}

void UCEEffectorProceduralMode::SetScaleStrength(const FVector& InStrength)
{
	if (ScaleStrength.Equals(InStrength))
	{
		return;
	}

	ScaleStrength = InStrength;
	UpdateExtensionParameters();
}

void UCEEffectorProceduralMode::SetPan(const FVector& InPan)
{
	if (Pan.Equals(InPan))
	{
		return;
	}

	Pan = InPan;
	UpdateExtensionParameters();
}

void UCEEffectorProceduralMode::SetFrequency(float InFrequency)
{
	InFrequency = FMath::Max(InFrequency, 0);

	if (FMath::IsNearlyEqual(Frequency, InFrequency))
	{
		return;
	}

	Frequency = InFrequency;
	UpdateExtensionParameters();
}

void UCEEffectorProceduralMode::OnExtensionParametersChanged(UCEEffectorComponent* InComponent)
{
	Super::OnExtensionParametersChanged(InComponent);

	FCEClonerEffectorChannelData& ChannelData = InComponent->GetChannelData();
	ChannelData.LocationDelta = LocationStrength;
	ChannelData.RotationDelta = FVector(RotationStrength.Yaw, RotationStrength.Pitch, RotationStrength.Roll);
	ChannelData.ScaleDelta = ScaleStrength;
	ChannelData.Frequency = Frequency;
	ChannelData.Pan = Pan;
	ChannelData.Pattern = Pattern;
}

bool UCEEffectorProceduralMode::RedirectExtensionName(FName InOldExtensionName) const
{
	return InOldExtensionName.IsEqual(TEXT("Noise")) || Super::RedirectExtensionName(InOldExtensionName);
}

void UCEEffectorProceduralMode::PostInitProperties()
{
	Super::PostInitProperties();

	if (HasAnyFlags(RF_ClassDefaultObject))
	{
		// Register new type def for niagara

		constexpr ENiagaraTypeRegistryFlags MeshFlags =
			ENiagaraTypeRegistryFlags::AllowAnyVariable |
			ENiagaraTypeRegistryFlags::AllowParameter;

		FNiagaraTypeRegistry::Register(FNiagaraTypeDefinition(StaticEnum<ECEClonerEffectorProceduralPattern>()), MeshFlags);
	}
}

#if WITH_EDITOR
const TCEPropertyChangeDispatcher<UCEEffectorProceduralMode> UCEEffectorProceduralMode::PropertyChangeDispatcher =
{
	{ GET_MEMBER_NAME_CHECKED(UCEEffectorProceduralMode, LocationStrength), &UCEEffectorProceduralMode::OnExtensionPropertyChanged },
	{ GET_MEMBER_NAME_CHECKED(UCEEffectorProceduralMode, RotationStrength), &UCEEffectorProceduralMode::OnExtensionPropertyChanged },
	{ GET_MEMBER_NAME_CHECKED(UCEEffectorProceduralMode, ScaleStrength), &UCEEffectorProceduralMode::OnExtensionPropertyChanged },
	{ GET_MEMBER_NAME_CHECKED(UCEEffectorProceduralMode, Pan), &UCEEffectorProceduralMode::OnExtensionPropertyChanged },
	{ GET_MEMBER_NAME_CHECKED(UCEEffectorProceduralMode, Frequency), &UCEEffectorProceduralMode::OnExtensionPropertyChanged },
	{ GET_MEMBER_NAME_CHECKED(UCEEffectorProceduralMode, Pattern), &UCEEffectorProceduralMode::OnExtensionPropertyChanged },
};

void UCEEffectorProceduralMode::PostEditChangeProperty(FPropertyChangedEvent& InPropertyChangedEvent)
{
	Super::PostEditChangeProperty(InPropertyChangedEvent);

	PropertyChangeDispatcher.OnPropertyChanged(this, InPropertyChangedEvent);
}
#endif
