// Copyright Epic Games, Inc. All Rights Reserved.

#include "Effector/Modes/CEEffectorPushMode.h"

#include "Effector/CEEffectorComponent.h"

void UCEEffectorPushMode::SetPushDirection(ECEClonerEffectorPushDirection InDirection)
{
	if (PushDirection == InDirection)
	{
		return;
	}

	PushDirection = InDirection;
	UpdateExtensionParameters();
}

void UCEEffectorPushMode::SetPushStrength(const FVector& InStrength)
{
	if (PushStrength.Equals(InStrength))
	{
		return;
	}

	PushStrength = InStrength;
	UpdateExtensionParameters();
}

void UCEEffectorPushMode::OnExtensionParametersChanged(UCEEffectorComponent* InComponent)
{
	Super::OnExtensionParametersChanged(InComponent);

	FCEClonerEffectorChannelData& ChannelData = InComponent->GetChannelData();
	ChannelData.LocationDelta = PushStrength;
	ChannelData.RotationDelta = FVector::ZeroVector;
	ChannelData.ScaleDelta = FVector::OneVector;
	ChannelData.Pan = FVector(0, static_cast<float>(PushDirection), 0);
}

#if WITH_EDITOR
const TCEPropertyChangeDispatcher<UCEEffectorPushMode> UCEEffectorPushMode::PropertyChangeDispatcher =
{
	{ GET_MEMBER_NAME_CHECKED(UCEEffectorPushMode, PushStrength), &UCEEffectorPushMode::OnExtensionPropertyChanged },
	{ GET_MEMBER_NAME_CHECKED(UCEEffectorPushMode, PushDirection), &UCEEffectorPushMode::OnExtensionPropertyChanged },
};

void UCEEffectorPushMode::PostEditChangeProperty(FPropertyChangedEvent& InPropertyChangedEvent)
{
	Super::PostEditChangeProperty(InPropertyChangedEvent);

	PropertyChangeDispatcher.OnPropertyChanged(this, InPropertyChangedEvent);
}
#endif
