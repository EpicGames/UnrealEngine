// Copyright Epic Games, Inc. All Rights Reserved.

#include "Effector/Modes/CEEffectorStepMode.h"

#include "Effector/CEEffectorComponent.h"

void UCEEffectorStepMode::SetStepPosition(const FVector& InPosition)
{
	if (StepPosition.Equals(InPosition))
	{
		return;
	}

	StepPosition = InPosition;
	UpdateExtensionParameters();
}

void UCEEffectorStepMode::SetStepRotation(const FRotator& InRotation)
{
	if (StepRotation.Equals(InRotation))
	{
		return;
	}

	StepRotation = InRotation;
	UpdateExtensionParameters();
}

void UCEEffectorStepMode::SetStepScale(const FVector& InScale)
{
	const FVector NewScale = InScale.ComponentMax(FVector(UE_KINDA_SMALL_NUMBER));

	if (NewScale.Equals(StepScale))
	{
		return;
	}

	StepScale = NewScale;
	UpdateExtensionParameters();
}

#if WITH_EDITOR
const TCEPropertyChangeDispatcher<UCEEffectorStepMode> UCEEffectorStepMode::PropertyChangeDispatcher =
{
	{ GET_MEMBER_NAME_CHECKED(UCEEffectorStepMode, StepPosition), &UCEEffectorStepMode::OnExtensionPropertyChanged },
	{ GET_MEMBER_NAME_CHECKED(UCEEffectorStepMode, StepRotation), &UCEEffectorStepMode::OnExtensionPropertyChanged },
	{ GET_MEMBER_NAME_CHECKED(UCEEffectorStepMode, StepScale), &UCEEffectorStepMode::OnExtensionPropertyChanged },
};

void UCEEffectorStepMode::PostEditChangeProperty(FPropertyChangedEvent& InPropertyChangedEvent)
{
	Super::PostEditChangeProperty(InPropertyChangedEvent);

	PropertyChangeDispatcher.OnPropertyChanged(this, InPropertyChangedEvent);
}
#endif

void UCEEffectorStepMode::OnExtensionParametersChanged(UCEEffectorComponent* InComponent)
{
	Super::OnExtensionParametersChanged(InComponent);

	FCEClonerEffectorChannelData& ChannelData = InComponent->GetChannelData();
	ChannelData.LocationDelta = StepPosition;
	ChannelData.RotationDelta = FVector(StepRotation.Yaw, StepRotation.Pitch, StepRotation.Roll);
	ChannelData.ScaleDelta = StepScale.ComponentMax(FVector(UE_KINDA_SMALL_NUMBER));
}
