// Copyright Epic Games, Inc. All Rights Reserved.

#include "Effector/Modes/CEEffectorTargetMode.h"

#include "CEClonerEffectorShared.h"
#include "Effector/CEEffectorComponent.h"
#include "GameFramework/Actor.h"

void UCEEffectorTargetMode::SetTargetActor(AActor* InTargetActor)
{
	if (InTargetActor == TargetActorWeak.Get())
	{
		return;
	}

	TargetActorWeak = InTargetActor;
	OnTargetActorChanged();
}

void UCEEffectorTargetMode::SetTargetActorWeak(const TWeakObjectPtr<AActor>& InTargetActor)
{
	SetTargetActor(InTargetActor.Get());
}

void UCEEffectorTargetMode::OnTargetActorChanged()
{
	UCEEffectorComponent* EffectorComponent = GetEffectorComponent();
	AActor* TargetActor = TargetActorWeak.Get();
	AActor* const InternalTargetActor = InternalTargetActorWeak.Get();

	if (TargetActor && TargetActor == InternalTargetActor)
	{
		UpdateExtensionParameters();
		return;
	}

	// unbind transform and destroy event
	if (InternalTargetActor && InternalTargetActor->GetRootComponent())
	{
		InternalTargetActor->OnDestroyed.RemoveAll(this);
		InternalTargetActor->GetRootComponent()->TransformUpdated.RemoveAll(this);
		InternalTargetActorWeak.Reset();
	}

	// set self by default if invalid
	if (!TargetActor || !TargetActor->GetRootComponent())
	{
		TargetActor = EffectorComponent->GetOwner();
	}

	// Bind to transform and destroy event
	if (TargetActor)
	{
		TargetActor->GetRootComponent()->TransformUpdated.RemoveAll(this);
		TargetActor->GetRootComponent()->TransformUpdated.AddUObject(this, &UCEEffectorTargetMode::OnTargetActorTransformChanged);

		TargetActor->OnDestroyed.RemoveAll(this);
		TargetActor->OnDestroyed.AddUniqueDynamic(this, &UCEEffectorTargetMode::OnTargetActorDestroyed);
	}

	TargetActorWeak = TargetActor;
	InternalTargetActorWeak = TargetActor;
	UpdateExtensionParameters();
}

void UCEEffectorTargetMode::OnTargetActorDestroyed(AActor* InActor)
{
	const AActor* TargetActor = TargetActorWeak.Get();

	if (TargetActor == InActor)
	{
		TargetActorWeak = GetEffectorComponent()->GetOwner();
		OnTargetActorChanged();
	}
}

void UCEEffectorTargetMode::OnTargetActorTransformChanged(USceneComponent*, EUpdateTransformFlags, ETeleportType)
{
	UpdateExtensionParameters();
}

void UCEEffectorTargetMode::OnExtensionParametersChanged(UCEEffectorComponent* InComponent)
{
	Super::OnExtensionParametersChanged(InComponent);

	if (const AActor* InternalTargetActor = InternalTargetActorWeak.Get())
	{
		FCEClonerEffectorChannelData& ChannelData = InComponent->GetChannelData();
		ChannelData.LocationDelta = InternalTargetActor->GetActorLocation();
		ChannelData.RotationDelta = FVector::ZeroVector;
		ChannelData.ScaleDelta = FVector::OneVector;
	}
}

void UCEEffectorTargetMode::OnExtensionDeactivated()
{
	Super::OnExtensionDeactivated();

	if (AActor* InternalTargetActor = InternalTargetActorWeak.Get())
	{
		InternalTargetActor->OnDestroyed.RemoveAll(this);
		InternalTargetActor->GetRootComponent()->TransformUpdated.RemoveAll(this);

		InternalTargetActorWeak.Reset();
	}
}

void UCEEffectorTargetMode::OnExtensionActivated()
{
	Super::OnExtensionActivated();

	OnTargetActorChanged();
}

#if WITH_EDITOR
const TCEPropertyChangeDispatcher<UCEEffectorTargetMode> UCEEffectorTargetMode::PropertyChangeDispatcher =
{
	{ GET_MEMBER_NAME_CHECKED(UCEEffectorTargetMode, TargetActorWeak), &UCEEffectorTargetMode::OnTargetActorChanged },
};

void UCEEffectorTargetMode::PostEditChangeProperty(FPropertyChangedEvent& InPropertyChangedEvent)
{
	Super::PostEditChangeProperty(InPropertyChangedEvent);

	PropertyChangeDispatcher.OnPropertyChanged(this, InPropertyChangedEvent);
}
#endif
