// Copyright Epic Games, Inc. All Rights Reserved.

#include "Effector/Types/CEEffectorBoundType.h"

#include "Effector/CEEffectorComponent.h"
#include "Settings/CEClonerEffectorSettings.h"

void UCEEffectorBoundType::SetInvertType(bool bInInvert)
{
	if (bInvertType == bInInvert)
	{
		return;
	}

	bInvertType = bInInvert;
	UpdateExtensionParameters();
}

void UCEEffectorBoundType::SetEasing(ECEClonerEasing InEasing)
{
	if (InEasing == Easing)
	{
		return;
	}

	Easing = InEasing;
	UpdateExtensionParameters();
}

void UCEEffectorBoundType::OnExtensionParametersChanged(UCEEffectorComponent* InComponent)
{
	Super::OnExtensionParametersChanged(InComponent);

	FCEClonerEffectorChannelData& ChannelData = InComponent->GetChannelData();
	ChannelData.Easing = Easing;
	ChannelData.Magnitude = bInvertType ? -InComponent->GetMagnitude() : InComponent->GetMagnitude();

#if WITH_EDITOR
	if (const UCEClonerEffectorSettings* ClonerEffectorSettings = GetDefault<UCEClonerEffectorSettings>())
	{
		InComponent->SetVisualizerColor(0, bInvertType ? ClonerEffectorSettings->GetVisualizerOuterColor() : ClonerEffectorSettings->GetVisualizerInnerColor());
		InComponent->SetVisualizerColor(1, bInvertType ? ClonerEffectorSettings->GetVisualizerInnerColor() : ClonerEffectorSettings->GetVisualizerOuterColor());
	}
#endif
}

void UCEEffectorBoundType::OnExtensionActivated()
{
	Super::OnExtensionActivated();
	
#if WITH_EDITOR
	if (UCEClonerEffectorSettings* ClonerEffectorSettings = GetMutableDefault<UCEClonerEffectorSettings>())
	{
		ClonerEffectorSettings->OnSettingChanged().AddUObject(this, &UCEEffectorBoundType::OnEffectorDeveloperSettingsChanged);
	}
#endif
}

void UCEEffectorBoundType::OnExtensionDeactivated()
{
	Super::OnExtensionDeactivated();

#if WITH_EDITOR
	if (UCEClonerEffectorSettings* ClonerEffectorSettings = GetMutableDefault<UCEClonerEffectorSettings>())
	{
		ClonerEffectorSettings->OnSettingChanged().RemoveAll(this);
	}
#endif
}

#if WITH_EDITOR
const TCEPropertyChangeDispatcher<UCEEffectorBoundType> UCEEffectorBoundType::PropertyChangeDispatcher =
{
	{ GET_MEMBER_NAME_CHECKED(UCEEffectorBoundType, bInvertType), &UCEEffectorBoundType::OnExtensionPropertyChanged },
	{ GET_MEMBER_NAME_CHECKED(UCEEffectorBoundType, Easing), &UCEEffectorBoundType::OnExtensionPropertyChanged }
};

void UCEEffectorBoundType::PostEditChangeProperty(FPropertyChangedEvent& InPropertyChangedEvent)
{
	Super::PostEditChangeProperty(InPropertyChangedEvent);

	PropertyChangeDispatcher.OnPropertyChanged(this, InPropertyChangedEvent);
}

void UCEEffectorBoundType::OnEffectorDeveloperSettingsChanged(UObject* InSettings, FPropertyChangedEvent& InEvent)
{
	UpdateExtensionParameters();	
}
#endif
