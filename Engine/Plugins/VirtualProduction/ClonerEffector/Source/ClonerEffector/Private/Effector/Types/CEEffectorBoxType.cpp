// Copyright Epic Games, Inc. All Rights Reserved.

#include "Effector/Types/CEEffectorBoxType.h"

#include "Effector/CEEffectorComponent.h"
#include "GeometryScript/MeshPrimitiveFunctions.h"

void UCEEffectorBoxType::SetInnerExtent(const FVector& InExtent)
{
	const FVector NewExtent = InExtent.ComponentMax(FVector::ZeroVector);

	if (NewExtent.Equals(InnerExtent))
	{
		return;
	}

	InnerExtent = NewExtent;
	MarkVisualizerDirty(InnerVisualizerFlag | OuterVisualizerFlag);
	UpdateExtensionParameters();
}

void UCEEffectorBoxType::SetOuterExtent(const FVector& InExtent)
{
	const FVector NewExtent = InExtent.ComponentMax(InnerExtent);

	if (NewExtent.Equals(OuterExtent))
	{
		return;
	}

	OuterExtent = NewExtent;
	MarkVisualizerDirty(InnerVisualizerFlag | OuterVisualizerFlag);
	UpdateExtensionParameters();
}

void UCEEffectorBoxType::OnExtensionParametersChanged(UCEEffectorComponent* InComponent)
{
	Super::OnExtensionParametersChanged(InComponent);

	InnerExtent = InnerExtent.ComponentMax(FVector::ZeroVector);
	OuterExtent = OuterExtent.ComponentMax(InnerExtent);

	FCEClonerEffectorChannelData& ChannelData = InComponent->GetChannelData();
	ChannelData.InnerExtent = InnerExtent;
	ChannelData.OuterExtent = OuterExtent;
}

void UCEEffectorBoxType::OnExtensionVisualizerDirty(int32 InDirtyFlags)
{
	Super::OnExtensionVisualizerDirty(InDirtyFlags);

	if ((InDirtyFlags & InnerVisualizerFlag) != 0)
	{
		UpdateVisualizer(InnerVisualizerFlag, [this](UDynamicMesh* InMesh)
		{
			constexpr FGeometryScriptPrimitiveOptions PrimitiveOptions;
			UGeometryScriptLibrary_MeshPrimitiveFunctions::AppendBox(InMesh, PrimitiveOptions, FTransform(FVector(0, 0, -InnerExtent.Z)), InnerExtent.X * 2, InnerExtent.Y * 2, InnerExtent.Z * 2);
		});
	}

	if ((InDirtyFlags & OuterVisualizerFlag) != 0)
	{
		UpdateVisualizer(OuterVisualizerFlag, [this](UDynamicMesh* InMesh)
		{
			constexpr FGeometryScriptPrimitiveOptions PrimitiveOptions;
			UGeometryScriptLibrary_MeshPrimitiveFunctions::AppendBox(InMesh, PrimitiveOptions, FTransform(FVector(0, 0, -OuterExtent.Z)), OuterExtent.X * 2, OuterExtent.Y * 2, OuterExtent.Z * 2);
		});
	}
}

#if WITH_EDITOR
const TCEPropertyChangeDispatcher<UCEEffectorBoxType> UCEEffectorBoxType::PropertyChangeDispatcher =
{
	{ GET_MEMBER_NAME_CHECKED(UCEEffectorBoxType, InnerExtent), &UCEEffectorBoxType::OnExtensionPropertyChanged },
	{ GET_MEMBER_NAME_CHECKED(UCEEffectorBoxType, OuterExtent), &UCEEffectorBoxType::OnExtensionPropertyChanged },
};

void UCEEffectorBoxType::PostEditChangeProperty(FPropertyChangedEvent& InPropertyChangedEvent)
{
	Super::PostEditChangeProperty(InPropertyChangedEvent);

	PropertyChangeDispatcher.OnPropertyChanged(this, InPropertyChangedEvent);
}

const TCEPropertyChangeDispatcher<UCEEffectorBoxType> UCEEffectorBoxType::PrePropertyChangeDispatcher =
{
	{ GET_MEMBER_NAME_CHECKED(UCEEffectorBoxType, InnerExtent), &UCEEffectorBoxType::OnVisualizerPropertyChanged },
	{ GET_MEMBER_NAME_CHECKED(UCEEffectorBoxType, OuterExtent), &UCEEffectorBoxType::OnVisualizerPropertyChanged },
};

void UCEEffectorBoxType::PreEditChange(FEditPropertyChain& InPropertyChain)
{
	Super::PreEditChange(InPropertyChain);

	FProperty* PropertyAboutToChange = InPropertyChain.GetActiveMemberNode()->GetValue();
	FPropertyChangedEvent Event(PropertyAboutToChange, EPropertyChangeType::Unspecified);
	PrePropertyChangeDispatcher.OnPropertyChanged(this, Event);
}
#endif
