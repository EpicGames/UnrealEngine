// Copyright Epic Games, Inc. All Rights Reserved.

#include "Effector/Types/CEEffectorPlaneType.h"

#include "Effector/CEEffectorComponent.h"
#include "GeometryScript/MeshPrimitiveFunctions.h"

void UCEEffectorPlaneType::SetPlaneSpacing(float InSpacing)
{
	InSpacing = FMath::Max(InSpacing, 0);

	if (FMath::IsNearlyEqual(InSpacing, PlaneSpacing))
	{
		return;
	}

	PlaneSpacing = InSpacing;
	MarkVisualizerDirty(InnerVisualizerFlag | OuterVisualizerFlag);
	UpdateExtensionParameters();
}

void UCEEffectorPlaneType::OnExtensionParametersChanged(UCEEffectorComponent* InComponent)
{
	Super::OnExtensionParametersChanged(InComponent);

	FCEClonerEffectorChannelData& ChannelData = InComponent->GetChannelData();
	ChannelData.InnerExtent = FVector::LeftVector;
	ChannelData.OuterExtent = FVector(PlaneSpacing);
}

void UCEEffectorPlaneType::OnExtensionVisualizerDirty(int32 InDirtyFlags)
{
	Super::OnExtensionVisualizerDirty(InDirtyFlags);

	if ((InDirtyFlags & InnerVisualizerFlag) != 0)
	{
		UpdateVisualizer(InnerVisualizerFlag, [this](UDynamicMesh* InMesh)
		{
			const FVector InnerPlane = FVector::LeftVector * FVector(-PlaneSpacing/2);
			constexpr FGeometryScriptPrimitiveOptions PrimitiveOptions;
			UGeometryScriptLibrary_MeshPrimitiveFunctions::AppendRectangleXY(InMesh, PrimitiveOptions, FTransform(FRotator(0, 0, 90), InnerPlane), 250, 250);
		});
	}

	if ((InDirtyFlags & OuterVisualizerFlag) != 0)
	{
		UpdateVisualizer(OuterVisualizerFlag, [this](UDynamicMesh* InMesh)
		{
			const FVector OuterPlane = FVector::LeftVector * FVector(PlaneSpacing/2);
			constexpr FGeometryScriptPrimitiveOptions PrimitiveOptions;
			UGeometryScriptLibrary_MeshPrimitiveFunctions::AppendRectangleXY(InMesh, PrimitiveOptions, FTransform(FRotator(0, 0, 90), OuterPlane), 500, 500);
		});
	}
}

#if WITH_EDITOR
const TCEPropertyChangeDispatcher<UCEEffectorPlaneType> UCEEffectorPlaneType::PropertyChangeDispatcher =
{
	{ GET_MEMBER_NAME_CHECKED(UCEEffectorPlaneType, PlaneSpacing), &UCEEffectorPlaneType::OnExtensionPropertyChanged }
};

void UCEEffectorPlaneType::PostEditChangeProperty(FPropertyChangedEvent& InPropertyChangedEvent)
{
	Super::PostEditChangeProperty(InPropertyChangedEvent);

	PropertyChangeDispatcher.OnPropertyChanged(this, InPropertyChangedEvent);
}

const TCEPropertyChangeDispatcher<UCEEffectorPlaneType> UCEEffectorPlaneType::PrePropertyChangeDispatcher =
{
	{ GET_MEMBER_NAME_CHECKED(UCEEffectorPlaneType, PlaneSpacing), &UCEEffectorPlaneType::OnVisualizerPropertyChanged }
};

void UCEEffectorPlaneType::PreEditChange(FEditPropertyChain& InPropertyChain)
{
	Super::PreEditChange(InPropertyChain);

	FProperty* PropertyAboutToChange = InPropertyChain.GetActiveMemberNode()->GetValue();
	FPropertyChangedEvent Event(PropertyAboutToChange, EPropertyChangeType::Unspecified);
	PrePropertyChangeDispatcher.OnPropertyChanged(this, Event);
}
#endif
