// Copyright Epic Games, Inc. All Rights Reserved.

#include "Effector/Types/CEEffectorRadialType.h"

#include "Effector/CEEffectorComponent.h"
#include "GeometryScript/MeshPrimitiveFunctions.h"

void UCEEffectorRadialType::SetRadialAngle(float InAngle)
{
	InAngle = FMath::Clamp(InAngle, 0, 360);

	if (FMath::IsNearlyEqual(InAngle, RadialAngle))
	{
		return;
	}

	RadialAngle = InAngle;
	MarkVisualizerDirty(InnerVisualizerFlag | OuterVisualizerFlag);
	UpdateExtensionParameters();
}

void UCEEffectorRadialType::SetRadialMinRadius(float InRadius)
{
	InRadius = FMath::Max(0, InRadius);

	if (FMath::IsNearlyEqual(InRadius, RadialMinRadius))
	{
		return;
	}

	RadialMinRadius = InRadius;
	MarkVisualizerDirty(InnerVisualizerFlag | OuterVisualizerFlag);
	UpdateExtensionParameters();
}

void UCEEffectorRadialType::SetRadialMaxRadius(float InRadius)
{
	InRadius = FMath::Max(0, InRadius);

	if (FMath::IsNearlyEqual(InRadius, RadialMaxRadius))
	{
		return;
	}

	RadialMaxRadius = InRadius;
	MarkVisualizerDirty(InnerVisualizerFlag | OuterVisualizerFlag);
	UpdateExtensionParameters();
}

void UCEEffectorRadialType::OnExtensionParametersChanged(UCEEffectorComponent* InComponent)
{
	Super::OnExtensionParametersChanged(InComponent);

	RadialMinRadius = FMath::Max(RadialMinRadius, 0.f);
	RadialMaxRadius = FMath::Max(RadialMinRadius, RadialMaxRadius);

	FCEClonerEffectorChannelData& ChannelData = InComponent->GetChannelData();
	ChannelData.InnerExtent = FVector::RightVector;
	ChannelData.OuterExtent = FVector(RadialAngle, RadialMinRadius, RadialMaxRadius);
}

void UCEEffectorRadialType::OnExtensionVisualizerDirty(int32 InDirtyFlags)
{
	Super::OnExtensionVisualizerDirty(InDirtyFlags);

	if ((InDirtyFlags & InnerVisualizerFlag) != 0)
	{
		UpdateVisualizer(InnerVisualizerFlag, [this](UDynamicMesh* InMesh)
		{
			constexpr FGeometryScriptPrimitiveOptions PrimitiveOptions;
			UGeometryScriptLibrary_MeshPrimitiveFunctions::AppendDisc(InMesh, PrimitiveOptions, FTransform(FRotator(0, 90, 0)), RadialMaxRadius, 16, 0, 0, RadialAngle / 2, RadialMinRadius);
		});
	}

	if ((InDirtyFlags & OuterVisualizerFlag) != 0)
	{
		UpdateVisualizer(OuterVisualizerFlag, [this](UDynamicMesh* InMesh)
		{
			constexpr FGeometryScriptPrimitiveOptions PrimitiveOptions;
			UGeometryScriptLibrary_MeshPrimitiveFunctions::AppendDisc(InMesh, PrimitiveOptions, FTransform(FRotator(0, 90, 0)), RadialMaxRadius, 16, 0, RadialAngle / 2, RadialAngle, RadialMinRadius);
		});
	}
}

#if WITH_EDITOR
const TCEPropertyChangeDispatcher<UCEEffectorRadialType> UCEEffectorRadialType::PropertyChangeDispatcher =
{
	{ GET_MEMBER_NAME_CHECKED(UCEEffectorRadialType, RadialAngle), &UCEEffectorRadialType::OnExtensionPropertyChanged },
	{ GET_MEMBER_NAME_CHECKED(UCEEffectorRadialType, RadialMinRadius), &UCEEffectorRadialType::OnExtensionPropertyChanged },
	{ GET_MEMBER_NAME_CHECKED(UCEEffectorRadialType, RadialMaxRadius), &UCEEffectorRadialType::OnExtensionPropertyChanged },
};

void UCEEffectorRadialType::PostEditChangeProperty(FPropertyChangedEvent& InPropertyChangedEvent)
{
	Super::PostEditChangeProperty(InPropertyChangedEvent);

	PropertyChangeDispatcher.OnPropertyChanged(this, InPropertyChangedEvent);
}

const TCEPropertyChangeDispatcher<UCEEffectorRadialType> UCEEffectorRadialType::PrePropertyChangeDispatcher =
{
	{ GET_MEMBER_NAME_CHECKED(UCEEffectorRadialType, RadialAngle), &UCEEffectorRadialType::OnVisualizerPropertyChanged },
	{ GET_MEMBER_NAME_CHECKED(UCEEffectorRadialType, RadialMinRadius), &UCEEffectorRadialType::OnVisualizerPropertyChanged },
	{ GET_MEMBER_NAME_CHECKED(UCEEffectorRadialType, RadialMaxRadius), &UCEEffectorRadialType::OnVisualizerPropertyChanged },
};

void UCEEffectorRadialType::PreEditChange(FEditPropertyChain& InPropertyChain)
{
	Super::PreEditChange(InPropertyChain);

	FProperty* PropertyAboutToChange = InPropertyChain.GetActiveMemberNode()->GetValue();
	FPropertyChangedEvent Event(PropertyAboutToChange, EPropertyChangeType::Unspecified);
	PrePropertyChangeDispatcher.OnPropertyChanged(this, Event);
}
#endif
