// Copyright Epic Games, Inc. All Rights Reserved.

#include "Effector/Types/CEEffectorSphereType.h"

#include "Effector/CEEffectorComponent.h"
#include "GeometryScript/MeshPrimitiveFunctions.h"

void UCEEffectorSphereType::SetOuterRadius(float InRadius)
{
	InRadius = FMath::Max(InRadius, InnerRadius);

	if (FMath::IsNearlyEqual(InRadius, OuterRadius))
	{
		return;
	}

	OuterRadius = InRadius;
	MarkVisualizerDirty(InnerVisualizerFlag | OuterVisualizerFlag);
	UpdateExtensionParameters();
}

void UCEEffectorSphereType::SetInnerRadius(float InRadius)
{
	InRadius = FMath::Max(InRadius, 0);

	if (FMath::IsNearlyEqual(InRadius, InnerRadius))
	{
		return;
	}

	InnerRadius = InRadius;
	MarkVisualizerDirty(InnerVisualizerFlag | OuterVisualizerFlag);
	UpdateExtensionParameters();
}

void UCEEffectorSphereType::OnExtensionParametersChanged(UCEEffectorComponent* InComponent)
{
	Super::OnExtensionParametersChanged(InComponent);

	InnerRadius = FMath::Max(InnerRadius, 0.f);
	OuterRadius = FMath::Max(OuterRadius, InnerRadius);

	FCEClonerEffectorChannelData& ChannelData = InComponent->GetChannelData();
	ChannelData.InnerExtent = FVector::ZeroVector;
	ChannelData.OuterExtent = FVector(InnerRadius, OuterRadius, 0.f);
}

void UCEEffectorSphereType::OnExtensionVisualizerDirty(int32 InDirtyFlags)
{
	Super::OnExtensionVisualizerDirty(InDirtyFlags);

	constexpr FGeometryScriptPrimitiveOptions PrimitiveOptions;
	constexpr int32 Phi = 20;
	constexpr int32 Theta = 32;

	if ((InDirtyFlags & InnerVisualizerFlag) != 0)
	{
		UpdateVisualizer(InnerVisualizerFlag, [this, &PrimitiveOptions, Phi, Theta](UDynamicMesh* InMesh)
		{
			UGeometryScriptLibrary_MeshPrimitiveFunctions::AppendSphereLatLong(InMesh, PrimitiveOptions, FTransform::Identity, InnerRadius, Phi, Theta);
		});
	}

	if ((InDirtyFlags & OuterVisualizerFlag) != 0)
	{
		UpdateVisualizer(OuterVisualizerFlag, [this, &PrimitiveOptions, Phi, Theta](UDynamicMesh* InMesh)
		{
			UGeometryScriptLibrary_MeshPrimitiveFunctions::AppendSphereLatLong(InMesh, PrimitiveOptions, FTransform::Identity, OuterRadius, Phi, Theta);
		});
	}
}

#if WITH_EDITOR
const TCEPropertyChangeDispatcher<UCEEffectorSphereType> UCEEffectorSphereType::PropertyChangeDispatcher =
{
	{ GET_MEMBER_NAME_CHECKED(UCEEffectorSphereType, OuterRadius), &UCEEffectorSphereType::OnExtensionPropertyChanged },
	{ GET_MEMBER_NAME_CHECKED(UCEEffectorSphereType, InnerRadius), &UCEEffectorSphereType::OnExtensionPropertyChanged },
};

void UCEEffectorSphereType::PostEditChangeProperty(FPropertyChangedEvent& InPropertyChangedEvent)
{
	Super::PostEditChangeProperty(InPropertyChangedEvent);

	PropertyChangeDispatcher.OnPropertyChanged(this, InPropertyChangedEvent);
}

const TCEPropertyChangeDispatcher<UCEEffectorSphereType> UCEEffectorSphereType::PrePropertyChangeDispatcher =
{
	{ GET_MEMBER_NAME_CHECKED(UCEEffectorSphereType, OuterRadius), &UCEEffectorSphereType::OnVisualizerPropertyChanged },
	{ GET_MEMBER_NAME_CHECKED(UCEEffectorSphereType, InnerRadius), &UCEEffectorSphereType::OnVisualizerPropertyChanged },
};

void UCEEffectorSphereType::PreEditChange(FEditPropertyChain& InPropertyChain)
{
	Super::PreEditChange(InPropertyChain);

	FProperty* PropertyAboutToChange = InPropertyChain.GetActiveMemberNode()->GetValue();
	FPropertyChangedEvent Event(PropertyAboutToChange, EPropertyChangeType::Unspecified);
	PrePropertyChangeDispatcher.OnPropertyChanged(this, Event);
}
#endif
