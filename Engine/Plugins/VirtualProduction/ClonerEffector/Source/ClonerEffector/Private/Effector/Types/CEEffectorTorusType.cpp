// Copyright Epic Games, Inc. All Rights Reserved.

#include "Effector/Types/CEEffectorTorusType.h"

#include "Effector/CEEffectorComponent.h"
#include "GeometryScript/MeshPrimitiveFunctions.h"

void UCEEffectorTorusType::SetTorusRadius(float InRadius)
{
	InRadius = FMath::Max(0, InRadius);

	if (FMath::IsNearlyEqual(InRadius, TorusRadius))
	{
		return;
	}

	TorusRadius = InRadius;
	MarkVisualizerDirty(InnerVisualizerFlag | OuterVisualizerFlag);
	UpdateExtensionParameters();
}

void UCEEffectorTorusType::SetTorusInnerRadius(float InRadius)
{
	InRadius = FMath::Max(0, InRadius);

	if (FMath::IsNearlyEqual(InRadius, TorusInnerRadius))
	{
		return;
	}

	TorusInnerRadius = InRadius;
	MarkVisualizerDirty(InnerVisualizerFlag | OuterVisualizerFlag);
	UpdateExtensionParameters();
}

void UCEEffectorTorusType::SetTorusOuterRadius(float InRadius)
{
	InRadius = FMath::Max(0, InRadius);

	if (FMath::IsNearlyEqual(InRadius, TorusOuterRadius))
	{
		return;
	}

	TorusOuterRadius = InRadius;
	MarkVisualizerDirty(InnerVisualizerFlag | OuterVisualizerFlag);
	UpdateExtensionParameters();
}

void UCEEffectorTorusType::OnExtensionParametersChanged(UCEEffectorComponent* InComponent)
{
	Super::OnExtensionParametersChanged(InComponent);

	TorusOuterRadius = FMath::Min(TorusRadius, TorusOuterRadius);
	TorusInnerRadius = FMath::Min(TorusInnerRadius, TorusOuterRadius);
	TorusOuterRadius = FMath::Max(TorusInnerRadius, TorusOuterRadius);

	FCEClonerEffectorChannelData& ChannelData = InComponent->GetChannelData();
	ChannelData.InnerExtent = FVector::ZAxisVector;
	ChannelData.OuterExtent = FVector(TorusInnerRadius, TorusOuterRadius, TorusRadius);
}

void UCEEffectorTorusType::OnExtensionVisualizerDirty(int32 InDirtyFlags)
{
	Super::OnExtensionVisualizerDirty(InDirtyFlags);

	constexpr FGeometryScriptPrimitiveOptions PrimitiveOptions;
	constexpr FGeometryScriptRevolveOptions RevolveOptions;
	constexpr int32 MajorSteps = 32;
	constexpr int32 MinorSteps = 16;

	if ((InDirtyFlags & InnerVisualizerFlag) != 0)
	{
		UpdateVisualizer(InnerVisualizerFlag, [this, &PrimitiveOptions, &RevolveOptions, MajorSteps, MinorSteps](UDynamicMesh* InMesh)
		{
			UGeometryScriptLibrary_MeshPrimitiveFunctions::AppendTorus(InMesh, PrimitiveOptions, FTransform(FVector(0, 0, -TorusInnerRadius)), RevolveOptions, TorusRadius, TorusInnerRadius, MajorSteps, MinorSteps);
		});
	}

	if ((InDirtyFlags & OuterVisualizerFlag) != 0)
	{
		UpdateVisualizer(OuterVisualizerFlag, [this, &PrimitiveOptions, &RevolveOptions, MajorSteps, MinorSteps](UDynamicMesh* InMesh)
		{
			UGeometryScriptLibrary_MeshPrimitiveFunctions::AppendTorus(InMesh, PrimitiveOptions, FTransform(FVector(0, 0, -TorusOuterRadius)), RevolveOptions, TorusRadius, TorusOuterRadius, MajorSteps, MinorSteps);
		});
	}
}

#if WITH_EDITOR
const TCEPropertyChangeDispatcher<UCEEffectorTorusType> UCEEffectorTorusType::PropertyChangeDispatcher =
{
	{ GET_MEMBER_NAME_CHECKED(UCEEffectorTorusType, TorusRadius), &UCEEffectorTorusType::OnExtensionPropertyChanged },
	{ GET_MEMBER_NAME_CHECKED(UCEEffectorTorusType, TorusInnerRadius), &UCEEffectorTorusType::OnExtensionPropertyChanged },
	{ GET_MEMBER_NAME_CHECKED(UCEEffectorTorusType, TorusOuterRadius), &UCEEffectorTorusType::OnExtensionPropertyChanged },
};

void UCEEffectorTorusType::PostEditChangeProperty(FPropertyChangedEvent& InPropertyChangedEvent)
{
	Super::PostEditChangeProperty(InPropertyChangedEvent);

	PropertyChangeDispatcher.OnPropertyChanged(this, InPropertyChangedEvent);
}

const TCEPropertyChangeDispatcher<UCEEffectorTorusType> UCEEffectorTorusType::PrePropertyChangeDispatcher =
{
	{ GET_MEMBER_NAME_CHECKED(UCEEffectorTorusType, TorusRadius), &UCEEffectorTorusType::OnVisualizerPropertyChanged },
	{ GET_MEMBER_NAME_CHECKED(UCEEffectorTorusType, TorusInnerRadius), &UCEEffectorTorusType::OnVisualizerPropertyChanged },
	{ GET_MEMBER_NAME_CHECKED(UCEEffectorTorusType, TorusOuterRadius), &UCEEffectorTorusType::OnVisualizerPropertyChanged },
};

void UCEEffectorTorusType::PreEditChange(FEditPropertyChain& InPropertyChain)
{
	Super::PreEditChange(InPropertyChain);

	FProperty* PropertyAboutToChange = InPropertyChain.GetActiveMemberNode()->GetValue();
	FPropertyChangedEvent Event(PropertyAboutToChange, EPropertyChangeType::Unspecified);
	PrePropertyChangeDispatcher.OnPropertyChanged(this, Event);
}
#endif
