// Copyright Epic Games, Inc. All Rights Reserved.

#include "Effector/Types/CEEffectorTypeBase.h"

#include "Effector/CEEffectorComponent.h"

void UCEEffectorTypeBase::OnExtensionParametersChanged(UCEEffectorComponent* InComponent)
{
	Super::OnExtensionParametersChanged(InComponent);

	FCEClonerEffectorChannelData& ChannelData = InComponent->GetChannelData();
	ChannelData.Type = static_cast<ECEClonerEffectorType>(TypeIdentifier);
}

#if WITH_EDITOR
void UCEEffectorTypeBase::PostEditUndo()
{
	Super::PostEditUndo();

	UpdateExtensionParameters();

	MarkVisualizerDirty(InnerVisualizerFlag | OuterVisualizerFlag);
}

void UCEEffectorTypeBase::OnVisualizerPropertyChanged()
{
	MarkVisualizerDirty(InnerVisualizerFlag | OuterVisualizerFlag);
}

bool UCEEffectorTypeBase::OnVisualizerTick(float InDeltaTime)
{
	OnExtensionVisualizerDirty(DirtyVisualizerFlags);
	DirtyVisualizerFlags = 0;
	VisualizerTickHandle.Reset();
	return false;
}
#endif

void UCEEffectorTypeBase::MarkVisualizerDirty(int32 InDirtyFlags)
{
#if WITH_EDITOR
	if (InDirtyFlags > 0)
	{
		DirtyVisualizerFlags |= InDirtyFlags;

		if (IsExtensionActive() && !VisualizerTickHandle.IsValid())
		{
			constexpr float Delay = 0.02f;
			VisualizerTickHandle = FTSTicker::GetCoreTicker().AddTicker(FTickerDelegate::CreateUObject(this, &UCEEffectorTypeBase::OnVisualizerTick), Delay);
		}
	}
#endif
}

int32 UCEEffectorTypeBase::VisualizerFlagToIdentifier(int32 InVisualizerFlag)
{
	if (InVisualizerFlag <= 0)
	{
		return INDEX_NONE;
	}

	return FMath::FloorLog2(InVisualizerFlag);
}

void UCEEffectorTypeBase::UpdateVisualizer(int32 InVisualizerFlag, TFunctionRef<void(UDynamicMesh*)> InMeshFunction) const
{
#if WITH_EDITOR
	if (UCEEffectorComponent* EffectorComponent = GetEffectorComponent())
	{
		EffectorComponent->UpdateVisualizer(VisualizerFlagToIdentifier(InVisualizerFlag), InMeshFunction);
	}
#endif
}

void UCEEffectorTypeBase::OnExtensionActivated()
{
	Super::OnExtensionActivated();

	MarkVisualizerDirty(InnerVisualizerFlag | OuterVisualizerFlag);
}
