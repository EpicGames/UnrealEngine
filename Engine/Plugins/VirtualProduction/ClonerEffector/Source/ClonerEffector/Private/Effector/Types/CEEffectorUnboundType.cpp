// Copyright Epic Games, Inc. All Rights Reserved.

#include "Effector/Types/CEEffectorUnboundType.h"

void UCEEffectorUnboundType::OnExtensionVisualizerDirty(int32 InDirtyFlags)
{
	Super::OnExtensionVisualizerDirty(InDirtyFlags);

	if ((InDirtyFlags & InnerVisualizerFlag) != 0)
	{
		UpdateVisualizer(InnerVisualizerFlag, [this](UDynamicMesh* InMesh)
		{
			// add nothing to the mesh
		});
	}

	if ((InDirtyFlags & OuterVisualizerFlag) != 0)
	{
		UpdateVisualizer(OuterVisualizerFlag, [this](UDynamicMesh* InMesh)
		{
			// add nothing to the mesh
		});
	}
}
