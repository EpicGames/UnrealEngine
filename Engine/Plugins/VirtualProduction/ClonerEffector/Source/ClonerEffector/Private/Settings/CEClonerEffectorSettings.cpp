// Copyright Epic Games, Inc. All Rights Reserved.

#include "Settings/CEClonerEffectorSettings.h"

#include "Engine/StaticMesh.h"
#include "Materials/MaterialInterface.h"
#include "UObject/SoftObjectPath.h"

#if WITH_EDITOR
#include "HAL/IConsoleManager.h"
#include "ISettingsModule.h"
#include "Modules/ModuleManager.h"
#endif

UCEClonerEffectorSettings::UCEClonerEffectorSettings()
{
	CategoryName = TEXT("Motion Design");
	SectionName = TEXT("Cloner & Effector");

	DefaultStaticMesh = TSoftObjectPtr<UStaticMesh>(FSoftObjectPath(DefaultStaticMeshPath));
	DefaultMaterial = TSoftObjectPtr<UMaterialInterface>(FSoftObjectPath(DefaultMaterialPath));
}

UStaticMesh* UCEClonerEffectorSettings::GetDefaultStaticMesh() const
{
	return DefaultStaticMesh.LoadSynchronous();
}

UMaterialInterface* UCEClonerEffectorSettings::GetDefaultMaterial() const
{
	return DefaultMaterial.LoadSynchronous();
}

void UCEClonerEffectorSettings::PostInitProperties()
{
	Super::PostInitProperties();

#if WITH_EDITOR
	CVarTSRShadingRejectionFlickeringPeriod = IConsoleManager::Get().FindConsoleVariable(TEXT("r.TSR.ShadingRejection.Flickering.Period"));

	if (CVarTSRShadingRejectionFlickeringPeriod)
	{
		CVarTSRShadingRejectionFlickeringPeriod->OnChangedDelegate().RemoveAll(this);
		CVarTSRShadingRejectionFlickeringPeriod->OnChangedDelegate().AddUObject(this, &UCEClonerEffectorSettings::OnTSRShadingRejectionFlickeringPeriodChanged);
	}

	OnReduceMotionGhostingChanged();
#endif
}

#if WITH_EDITOR
void UCEClonerEffectorSettings::OpenEditorSettingsWindow() const
{
	if (ISettingsModule* SettingsModule = FModuleManager::GetModulePtr<ISettingsModule>(TEXT("Settings")))
	{
		SettingsModule->ShowViewer(GetContainerName(), GetCategoryName(), GetSectionName());
	}
}

void UCEClonerEffectorSettings::PostEditChangeProperty(FPropertyChangedEvent& InPropertyChangedEvent)
{
	if (InPropertyChangedEvent.GetMemberPropertyName() == GET_MEMBER_NAME_CHECKED(UCEClonerEffectorSettings, bReduceMotionGhosting))
	{
		OnReduceMotionGhostingChanged();
	}

	Super::PostEditChangeProperty(InPropertyChangedEvent);
}

void UCEClonerEffectorSettings::EnableNoFlicker()
{
	if (IsNoFlickerEnabled())
	{
		return;
	}

	PreviousCVarValue = CVarTSRShadingRejectionFlickeringPeriod->GetInt();
	CVarTSRShadingRejectionFlickeringPeriod->Set(NoFlicker);
}

void UCEClonerEffectorSettings::DisableNoFlicker()
{
	if (!IsNoFlickerEnabled())
	{
		return;
	}

	if (PreviousCVarValue.IsSet())
	{
		CVarTSRShadingRejectionFlickeringPeriod->Set(PreviousCVarValue.GetValue());
	}
	else
	{
		CVarTSRShadingRejectionFlickeringPeriod->Set(*CVarTSRShadingRejectionFlickeringPeriod->GetDefaultValue());
	}
}

bool UCEClonerEffectorSettings::IsNoFlickerEnabled() const
{
	if (!CVarTSRShadingRejectionFlickeringPeriod)
	{
		return false;
	}

	return CVarTSRShadingRejectionFlickeringPeriod->GetInt() == NoFlicker;
}

void UCEClonerEffectorSettings::OnReduceMotionGhostingChanged()
{
	if (bReduceMotionGhosting)
	{
		EnableNoFlicker();
	}
	else
	{
		DisableNoFlicker();
	}
}

void UCEClonerEffectorSettings::OnTSRShadingRejectionFlickeringPeriodChanged(IConsoleVariable* InCVar)
{
	if (InCVar == CVarTSRShadingRejectionFlickeringPeriod)
	{
		bReduceMotionGhosting = IsNoFlickerEnabled();
	}
}
#endif
