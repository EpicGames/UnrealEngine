// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Engine/DeveloperSettings.h"
#include "UObject/SoftObjectPtr.h"
#include "CEClonerEffectorSettings.generated.h"

class UMaterialInterface;
class UStaticMesh;
struct FLinearColor;

/** Settings for motion design Cloner and Effector plugin */
UCLASS(Config=Engine, meta=(DisplayName="Cloner & Effector"))
class UCEClonerEffectorSettings : public UDeveloperSettings
{
	GENERATED_BODY()

public:
	static constexpr int32 NoFlicker = 1;
	static constexpr TCHAR DefaultStaticMeshPath[] = TEXT("/Script/Engine.StaticMesh'/Engine/BasicShapes/Cube.Cube'");
	static constexpr TCHAR DefaultMaterialPath[] = TEXT("/Script/Engine.Material'/ClonerEffector/Materials/DefaultClonerMaterial.DefaultClonerMaterial'");

	FLinearColor GetVisualizerInnerColor() const
	{
		return VisualizerInnerColor;
	}

	FLinearColor GetVisualizerOuterColor() const
	{
		return VisualizerOuterColor;
	}

	bool GetSpawnDefaultActorAttached() const
	{
		return bSpawnDefaultActorAttached;
	}

	UStaticMesh* GetDefaultStaticMesh() const;

	UMaterialInterface* GetDefaultMaterial() const;

#if WITH_EDITOR
	bool GetReduceMotionGhosting() const
	{
		return bReduceMotionGhosting;
	}

	void OpenEditorSettingsWindow() const;
#endif

protected:
	UCEClonerEffectorSettings();

	//~ Begin UObject
	virtual void PostInitProperties() override;
#if WITH_EDITOR
	virtual void PostEditChangeProperty(FPropertyChangedEvent& InPropertyChangedEvent) override;
#endif
	//~ End UObject

	/** Inner visualizer color for effectors */
	UPROPERTY(Config, EditAnywhere, Category="Effector")
	FLinearColor VisualizerInnerColor = FLinearColor(255, 0, 0, 0.1f);

	/** Outer visualizer color for effectors */
	UPROPERTY(Config, EditAnywhere, Category="Effector")
	FLinearColor VisualizerOuterColor = FLinearColor(0, 0, 255, 0.1f);

	/** Spawns a default actor attached to the cloner on spawn */
	UPROPERTY(Config, EditAnywhere, Category="Cloner")
	bool bSpawnDefaultActorAttached = true;

	/** Default static mesh used when spawning default actor attached */
	UPROPERTY(Config, EditAnywhere, Category="Cloner")
	TSoftObjectPtr<UStaticMesh> DefaultStaticMesh;

	/** Default material used when spawning default actor attached */
	UPROPERTY(Config, EditAnywhere, Category="Cloner")
	TSoftObjectPtr<UMaterialInterface> DefaultMaterial;

#if WITH_EDITORONLY_DATA
	/** Reduces the r.TSR.ShadingRejection.Flickering.Period from 3 (default) to 1 if enabled to avoid ghosting artifacts when moving */
	UPROPERTY(Config, EditAnywhere, Category="Cloner")
	bool bReduceMotionGhosting = true;
#endif

private:
#if WITH_EDITORONLY_DATA
	void EnableNoFlicker();
	void DisableNoFlicker();
	bool IsNoFlickerEnabled() const;

	void OnReduceMotionGhostingChanged();
	void OnTSRShadingRejectionFlickeringPeriodChanged(IConsoleVariable* InCVar);

	/** Allows to reduce ghosting artifacts when moving cloner instances */
	IConsoleVariable* CVarTSRShadingRejectionFlickeringPeriod = nullptr;

	/** Previous value to restore it when disabled */
	UPROPERTY()
	TOptional<int32> PreviousCVarValue;
#endif
};
