// Copyright Epic Games, Inc. All Rights Reserved.

#include "Subsystems/CEClonerSubsystem.h"

#include "Cloner/CEClonerActor.h"
#include "Cloner/CEClonerComponent.h"
#include "Cloner/Attachments/CEClonerAttachmentTreeBehavior.h"
#include "Cloner/Attachments/CEClonerSceneTreeCustomResolver.h"
#include "Cloner/Extensions/CEClonerEffectorExtension.h"
#include "Cloner/Extensions/CEClonerExtensionBase.h"
#include "Cloner/Layouts/CEClonerCircleLayout.h"
#include "Cloner/Layouts/CEClonerCylinderLayout.h"
#include "Cloner/Layouts/CEClonerGridLayout.h"
#include "Cloner/Layouts/CEClonerHoneycombLayout.h"
#include "Cloner/Layouts/CEClonerLayoutBase.h"
#include "Cloner/Layouts/CEClonerLineLayout.h"
#include "Cloner/Layouts/CEClonerMeshLayout.h"
#include "Cloner/Layouts/CEClonerSphereRandomLayout.h"
#include "Cloner/Layouts/CEClonerSphereUniformLayout.h"
#include "Cloner/Layouts/CEClonerSplineLayout.h"
#include "Cloner/Logs/CEClonerLogs.h"
#include "Effector/CEEffectorActor.h"
#include "Engine/Engine.h"
#include "Engine/World.h"
#include "GameFramework/Actor.h"
#include "Materials/Material.h"
#include "UObject/Class.h"
#include "UObject/UObjectIterator.h"
#include "Utilities/CEClonerEffectorUtilities.h"

#if WITH_EDITOR
#include "Editor.h"
#include "Editor/EditorEngine.h"
#include "ScopedTransaction.h"
#endif

UCEClonerSubsystem::FOnSubsystemInitialized UCEClonerSubsystem::OnSubsystemInitializedDelegate;
UCEClonerSubsystem::FOnClonerSetEnabled UCEClonerSubsystem::OnClonerSetEnabledDelegate;
UCEClonerSubsystem::FOnGetSceneTreeResolver UCEClonerSubsystem::OnGetSceneTreeResolverDelegate;

#define LOCTEXT_NAMESPACE "CEEffectorSubsystem"

UCEClonerSubsystem* UCEClonerSubsystem::Get()
{
	if (GEngine)
	{
		return GEngine->GetEngineSubsystem<UCEClonerSubsystem>();
	}

	return nullptr;
}

void UCEClonerSubsystem::Initialize(FSubsystemCollectionBase& Collection)
{
	Super::Initialize(Collection);

	// Register them here to match old order of layout enum
	RegisterLayoutClass(UCEClonerGridLayout::StaticClass());
	RegisterLayoutClass(UCEClonerLineLayout::StaticClass());
	RegisterLayoutClass(UCEClonerCircleLayout::StaticClass());
	RegisterLayoutClass(UCEClonerCylinderLayout::StaticClass());
	RegisterLayoutClass(UCEClonerSphereUniformLayout::StaticClass());
	RegisterLayoutClass(UCEClonerHoneycombLayout::StaticClass());
	RegisterLayoutClass(UCEClonerMeshLayout::StaticClass());
	RegisterLayoutClass(UCEClonerSplineLayout::StaticClass());
	RegisterLayoutClass(UCEClonerSphereRandomLayout::StaticClass());

	// Scan for new layouts
	ScanForRegistrableClasses();

	// Tree implementation
	{
		TFunction<TSharedRef<ICEClonerAttachmentTreeBehavior>()> GroupImplCreator = []()
		{
			return MakeShared<FCEClonerAttachmentGroupBehavior>();
		};
		RegisterAttachmentTreeBehavior(TEXT("Group"), GroupImplCreator);
		
		TFunction<TSharedRef<ICEClonerAttachmentTreeBehavior>()> FlatImplCreator = []()
		{
			return MakeShared<FCEClonerAttachmentFlatBehavior>();
		};
		RegisterAttachmentTreeBehavior(TEXT("Flat"), FlatImplCreator);
	}

	OnSubsystemInitializedDelegate.Broadcast();

	FWorldDelegates::OnWorldCleanup.AddUObject(this, &UCEClonerSubsystem::OnWorldCleanup);
}

void UCEClonerSubsystem::Deinitialize()
{
	Super::Deinitialize();

	FWorldDelegates::OnWorldCleanup.RemoveAll(this);
}

bool UCEClonerSubsystem::RegisterLayoutClass(UClass* InClonerLayoutClass)
{
	if (!IsValid(InClonerLayoutClass))
	{
		return false;
	}

	if (!InClonerLayoutClass->IsChildOf(UCEClonerLayoutBase::StaticClass())
		|| InClonerLayoutClass->HasAnyClassFlags(CLASS_Abstract | CLASS_Deprecated | CLASS_NewerVersionExists))
	{
		return false;
	}

	if (IsLayoutClassRegistered(InClonerLayoutClass))
	{
		return false;
	}

	const UCEClonerLayoutBase* CDO = InClonerLayoutClass->GetDefaultObject<UCEClonerLayoutBase>();

	if (!CDO)
	{
		return false;
	}

	// Check niagara asset is valid
	if (!CDO->IsLayoutValid())
	{
		return false;
	}

	// Does not overwrite existing layouts
	const FName LayoutName = CDO->GetLayoutName();
	if (LayoutName.IsNone() || LayoutClasses.Contains(LayoutName))
	{
		return false;
	}

	LayoutClasses.Add(LayoutName, CDO->GetClass());

	return true;
}

bool UCEClonerSubsystem::UnregisterLayoutClass(UClass* InClonerLayoutClass)
{
	if (!IsValid(InClonerLayoutClass))
	{
		return false;
	}

	TSubclassOf<UCEClonerLayoutBase> LayoutClass(InClonerLayoutClass);
	if (const FName* LayoutName = LayoutClasses.FindKey(LayoutClass))
	{
		LayoutClasses.Remove(*LayoutName);
		return true;
	}

	return false;
}

bool UCEClonerSubsystem::IsLayoutClassRegistered(UClass* InClonerLayoutClass)
{
	if (!IsValid(InClonerLayoutClass))
	{
		return false;
	}

	TSubclassOf<UCEClonerLayoutBase> LayoutClass(InClonerLayoutClass);
	if (const FName* LayoutName = LayoutClasses.FindKey(LayoutClass))
	{
		return true;
	}

	return false;
}

bool UCEClonerSubsystem::RegisterExtensionClass(UClass* InClass)
{
	if (!IsValid(InClass))
	{
		return false;
	}

	if (!InClass->IsChildOf(UCEClonerExtensionBase::StaticClass())
		|| InClass->HasAnyClassFlags(CLASS_Abstract | CLASS_Deprecated | CLASS_NewerVersionExists))
	{
		return false;
	}

	if (IsExtensionClassRegistered(InClass))
	{
		return false;
	}

	const UCEClonerExtensionBase* CDO = InClass->GetDefaultObject<UCEClonerExtensionBase>();

	if (!CDO)
	{
		return false;
	}

	const FName ExtensionName = CDO->GetExtensionName();
	if (ExtensionName.IsNone() || ExtensionClasses.Contains(ExtensionName))
	{
		return false;
	}

	ExtensionClasses.Add(ExtensionName, CDO->GetClass());

	return true;
}

bool UCEClonerSubsystem::UnregisterExtensionClass(UClass* InClass)
{
	if (!IsValid(InClass))
	{
		return false;
	}

	TSubclassOf<UCEClonerExtensionBase> ExtensionClass(InClass);
	if (const FName* ExtensionName = ExtensionClasses.FindKey(ExtensionClass))
	{
		ExtensionClasses.Remove(*ExtensionName);
		return true;
	}

	return false;
}

bool UCEClonerSubsystem::IsExtensionClassRegistered(UClass* InClass) const
{
	if (!IsValid(InClass))
	{
		return false;
	}

	TSubclassOf<UCEClonerExtensionBase> ExtensionClass(InClass);
	return !!ExtensionClasses.FindKey(ExtensionClass);
}

TSet<FName> UCEClonerSubsystem::GetExtensionNames() const
{
	TArray<FName> ExtensionNames;
	ExtensionClasses.GenerateKeyArray(ExtensionNames);
	return TSet<FName>(ExtensionNames);
}

TSet<TSubclassOf<UCEClonerExtensionBase>> UCEClonerSubsystem::GetExtensionClasses() const
{
	TArray<TSubclassOf<UCEClonerExtensionBase>> Extensions;
	ExtensionClasses.GenerateValueArray(Extensions);
	return TSet<TSubclassOf<UCEClonerExtensionBase>>(Extensions);
}

FName UCEClonerSubsystem::FindExtensionName(TSubclassOf<UCEClonerExtensionBase> InClass) const
{
	if (const FName* Key = ExtensionClasses.FindKey(InClass))
	{
		return *Key;
	}

	return NAME_None;
}

UCEClonerExtensionBase* UCEClonerSubsystem::CreateNewExtension(FName InExtensionName, UCEClonerComponent* InCloner)
{
	if (!IsValid(InCloner))
	{
		return nullptr;
	}

	TSubclassOf<UCEClonerExtensionBase> const* ExtensionClass = ExtensionClasses.Find(InExtensionName);

	if (!ExtensionClass)
	{
		return nullptr;
	}

	return NewObject<UCEClonerExtensionBase>(InCloner, ExtensionClass->Get(), NAME_None, RF_Transactional);
}

void UCEClonerSubsystem::SetClonersEnabled(const TSet<UCEClonerComponent*>& InCloners, bool bInEnable, bool bInShouldTransact)
{
	if (InCloners.IsEmpty())
	{
		return;
	}

#if WITH_EDITOR
	const FText TransactionText = bInEnable
		? LOCTEXT("SetClonersEnabled", "Cloners enabled")
		: LOCTEXT("SetClonersDisabled", "Cloners disabled");

	FScopedTransaction Transaction(TransactionText, bInShouldTransact);
#endif

	for (UCEClonerComponent* Cloner : InCloners)
	{
		if (!IsValid(Cloner))
		{
			continue;
		}

#if WITH_EDITOR
		Cloner->Modify();
#endif

		Cloner->SetEnabled(bInEnable);
	}
}

void UCEClonerSubsystem::SetLevelClonersEnabled(const UWorld* InWorld, bool bInEnable, bool bInShouldTransact)
{
	if (!IsValid(InWorld))
	{
		return;
	}

#if WITH_EDITOR
	const FText TransactionText = bInEnable
		? LOCTEXT("SetLevelClonersEnabled", "Level cloners enabled")
		: LOCTEXT("SetLevelClonersDisabled", "Level cloners disabled");

	FScopedTransaction Transaction(TransactionText, bInShouldTransact);
#endif

	OnClonerSetEnabledDelegate.Broadcast(InWorld, bInEnable, bInShouldTransact);
}

#if WITH_EDITOR
void UCEClonerSubsystem::ConvertCloners(const TSet<UCEClonerComponent*>& InCloners, ECEClonerMeshConversion InMeshConversion)
{
	if (InCloners.IsEmpty())
	{
		return;
	}

	using namespace UE::ClonerEffector::Conversion;

	for (UCEClonerComponent* ClonerComponent : InCloners)
	{
		if (!IsValid(ClonerComponent) || !ClonerComponent->GetEnabled())
		{
			continue;
		}

		switch(InMeshConversion)
		{
			case ECEClonerMeshConversion::StaticMesh:
				ClonerComponent->ConvertToStaticMesh();
			break;

			case ECEClonerMeshConversion::StaticMeshes:
				ClonerComponent->ConvertToStaticMeshes();
			break;

			case ECEClonerMeshConversion::DynamicMesh:
				ClonerComponent->ConvertToDynamicMesh();
			break;

			case ECEClonerMeshConversion::DynamicMeshes:
				ClonerComponent->ConvertToDynamicMeshes();
			break;

			case ECEClonerMeshConversion::InstancedStaticMesh:
				ClonerComponent->ConvertToInstancedStaticMeshes();
			break;

			default:;
		}
	}
}
#endif

TArray<UCEEffectorComponent*> UCEClonerSubsystem::CreateLinkedEffectors(const TArray<UCEClonerComponent*>& InCloners, ECEClonerActionFlags InFlags, TFunctionRef<void(UCEEffectorComponent*)> InGenerator)
{
	TArray<UCEEffectorComponent*> LinkedEffectors;

	if (InCloners.IsEmpty())
	{
		return LinkedEffectors;
	}

	LinkedEffectors.Reserve(InCloners.Num());

#if WITH_EDITOR
	const bool bSelect = EnumHasAnyFlags(InFlags, ECEClonerActionFlags::ShouldSelect);
	const bool bShouldTransact = EnumHasAnyFlags(InFlags, ECEClonerActionFlags::ShouldTransact);

	if (bSelect && GEditor)
	{
		GEditor->SelectNone(/** Notify */false, /** DeselectBSP */true);
	}

	const FText TransactionText = FText::Format(LOCTEXT("CreateLinkedEffectors", "Create {0} linked effector(s)"), FText::AsNumber(InCloners.Num()));
	FScopedTransaction Transaction(TransactionText, bShouldTransact && !GIsTransacting);
#endif

	for (const UCEClonerComponent* ClonerComponent : InCloners)
	{
		if (!IsValid(ClonerComponent))
		{
			continue;
		}

		UWorld* ClonerWorld = ClonerComponent->GetWorld();
		if (!IsValid(ClonerWorld))
		{
			continue;
		}

		FActorSpawnParameters SpawnParameters;
		SpawnParameters.OverrideLevel = ClonerComponent->GetComponentLevel();
#if WITH_EDITOR
		SpawnParameters.InitialActorLabel = ACEEffectorActor::DefaultLabel;
		SpawnParameters.ObjectFlags = RF_Transactional;
		SpawnParameters.bTemporaryEditorActor = false;
#endif

		const FVector ClonerLocation = ClonerComponent->GetComponentLocation();
		const FRotator ClonerRotation = ClonerComponent->GetComponentRotation();

		ACEEffectorActor* EffectorActor = ClonerWorld->SpawnActor<ACEEffectorActor>(ACEEffectorActor::StaticClass(), ClonerLocation, ClonerRotation, SpawnParameters);
		if (!EffectorActor)
		{
			continue;
		}

		if (UCEClonerEffectorExtension* EffectorExtension = ClonerComponent->GetExtension<UCEClonerEffectorExtension>())
		{
#if WITH_EDITOR
			EffectorExtension->Modify();
#endif

			EffectorExtension->LinkEffector(EffectorActor);
		}

		InGenerator(LinkedEffectors.Add_GetRef(EffectorActor->GetEffectorComponent()));

#if WITH_EDITOR
		if (bSelect && GEditor)
		{
			GEditor->SelectActor(EffectorActor, /** Selected */true, /** Notify */true);
		}
#endif
	}

#if WITH_EDITOR
	if (LinkedEffectors.IsEmpty())
	{
		Transaction.Cancel();
	}
#endif

	return LinkedEffectors;
}

UCEClonerComponent* UCEClonerSubsystem::CreateClonerWithActors(UWorld* InWorld, const TSet<AActor*>& InActors, ECEClonerActionFlags InFlags)
{
	UCEClonerComponent* NewCloner = nullptr;

	if (!IsValid(InWorld))
	{
		return NewCloner;
	}

#if WITH_EDITOR
	const bool bSelect = EnumHasAnyFlags(InFlags, ECEClonerActionFlags::ShouldSelect);
	const bool bShouldTransact = EnumHasAnyFlags(InFlags, ECEClonerActionFlags::ShouldTransact);

	FScopedTransaction Transaction(
		LOCTEXT("CreateClonerWithActors", "Create cloner with actors attached"),
		bShouldTransact && !GIsTransacting
	);
#endif

	FActorSpawnParameters Parameters;
#if WITH_EDITOR
	Parameters.InitialActorLabel = ACEClonerActor::DefaultLabel;
	Parameters.ObjectFlags = RF_Transactional;
	Parameters.bTemporaryEditorActor = false;
#endif

	if (ACEClonerActor* NewClonerActor = InWorld->SpawnActor<ACEClonerActor>(Parameters))
	{
		NewCloner = NewClonerActor->GetClonerComponent();

		if (!InActors.IsEmpty())
		{
			FVector NewAverageLocation;

			for (const AActor* Actor : InActors)
			{
				if (IsValid(Actor))
				{
					NewAverageLocation += Actor->GetActorLocation() / InActors.Num();
				}
			}

			NewClonerActor->SetActorLocation(NewAverageLocation);

			for (AActor* Actor : InActors)
			{
				if (IsValid(Actor))
				{
#if WITH_EDITOR
					Actor->Modify();
#endif

					if (USceneComponent* RootComponent = Actor->GetRootComponent())
					{
						RootComponent->SetMobility(EComponentMobility::Type::Movable);
					}

					Actor->AttachToActor(NewClonerActor, FAttachmentTransformRules::KeepWorldTransform);
				}
			}
		}

#if WITH_EDITOR
		if (bSelect && GEditor)
		{
			GEditor->SelectNone(/** SelectionChange */false, /** DeselectBSP */true);
			GEditor->SelectActor(NewClonerActor, /** Selected */true, /** Notify */true);
		}
#endif
	}

	return NewCloner;
}

void UCEClonerSubsystem::FireMaterialWarning(const AActor* InClonerActor, const AActor* InContextActor, const TArray<TWeakObjectPtr<UMaterialInterface>>& InUnsetMaterials)
{
	if (!IsValid(InContextActor) || !IsValid(InClonerActor) || InUnsetMaterials.IsEmpty())
	{
		return;
	}

	UE_LOGF(LogCECloner, Warning, "%ls : %i unsupported material(s) detected due to missing niagara usage flag (bUsedWithNiagaraMeshParticles) on actor (%ls), see logs below", *InClonerActor->GetActorNameOrLabel(), InUnsetMaterials.Num(), *InContextActor->GetActorNameOrLabel());

	for (const TWeakObjectPtr<UMaterialInterface>& UnsetMaterialWeak : InUnsetMaterials)
	{
		if (UMaterialInterface* UnsetMaterial = UnsetMaterialWeak.Get())
		{
			UE_LOGF(LogCECloner, Warning, "%ls : The following materials (%ls) on actor (%ls) does not have the usage flag (bUsedWithNiagaraMeshParticles) set to work with the cloner, set the flag and resave the asset to avoid this warning", *InClonerActor->GetActorNameOrLabel(), *UnsetMaterial->GetMaterial()->GetPathName(), *InContextActor->GetActorNameOrLabel());
		}
	}

#if WITH_EDITOR
	using namespace UE::ClonerEffector::Utilities;

	// Fire warning notification when invalid materials are found and at least 5s has elapsed since last notification
	constexpr double MinNotificationElapsedTime = 5.0;
	const double CurrentTime = FApp::GetCurrentTime();

	if (CurrentTime - LastNotificationTime > MinNotificationElapsedTime)
	{
		LastNotificationTime = CurrentTime;
		ShowWarning(FText::Format(GetMaterialWarningText(), InUnsetMaterials.Num()));
	}
#endif
}

bool UCEClonerSubsystem::RegisterAttachmentTreeBehavior(FName InName, TFunction<TSharedRef<ICEClonerAttachmentTreeBehavior>()>& InCreator)
{
	if (!TreeBehaviorCreators.Contains(InName))
	{
		TreeBehaviorCreators.Add(InName, InCreator);
		return true;
	}

	return false;
}

bool UCEClonerSubsystem::UnregisterAttachmentTreeBehavior(FName InName)
{
	return TreeBehaviorCreators.Remove(InName) > 0;
}

TArray<FName> UCEClonerSubsystem::GetAttachmentTreeBehaviorNames() const
{
	TArray<FName> TreeImplNames;
	TreeBehaviorCreators.GenerateKeyArray(TreeImplNames);
	return TreeImplNames;
}

TSharedPtr<ICEClonerAttachmentTreeBehavior> UCEClonerSubsystem::CreateAttachmentTreeBehavior(FName InName) const
{
	TSharedPtr<ICEClonerAttachmentTreeBehavior> TreeBehavior = nullptr;

	if (const TFunction<TSharedRef<ICEClonerAttachmentTreeBehavior>()>* BehaviorCreator = TreeBehaviorCreators.Find(InName))
	{
		TreeBehavior = (*BehaviorCreator)();
	}

	return TreeBehavior;
}

TSharedPtr<ICEClonerSceneTreeCustomResolver> UCEClonerSubsystem::FindCustomLevelSceneTreeResolver(ULevel* InLevel)
{
	TSharedPtr<ICEClonerSceneTreeCustomResolver> Resolver = nullptr;

	if (!InLevel)
	{
		return Resolver;
	}

	if (const TSharedRef<ICEClonerSceneTreeCustomResolver>* CachedResolver = LevelCustomResolvers.Find(InLevel))
	{
		Resolver = *CachedResolver;
	}

	if (!Resolver.IsValid() && OnGetSceneTreeResolverDelegate.IsBound())
	{
		Resolver = OnGetSceneTreeResolverDelegate.Execute(InLevel);

		if (Resolver.IsValid())
		{
			LevelCustomResolvers.Add(InLevel, Resolver.ToSharedRef());
			InLevel->OnCleanupLevel.AddUObject(this, &UCEClonerSubsystem::OnLevelCleanup, InLevel);
			Resolver->Activate();
		}
	}

	return Resolver;
}

const UNiagaraSystem* UCEClonerSubsystem::FindOrLoadTemplateNiagaraSystem(const UCEClonerLayoutBase* InLayout)
{
	if (!InLayout)
	{
		return nullptr;
	}

	TObjectPtr<UNiagaraSystem>& TemplateNiagaraSystem = TemplateNiagaraSystems.FindOrAdd(InLayout->GetLayoutName());
	if (!TemplateNiagaraSystem)
	{
		TemplateNiagaraSystem = LoadObject<UNiagaraSystem>(nullptr, *InLayout->GetLayoutAssetPath());
	}
	return TemplateNiagaraSystem;
}

const UNiagaraSystem* UCEClonerSubsystem::FindOrLoadBaseNiagaraSystem()
{
	if (!BaseNiagaraSystem)
	{
		BaseNiagaraSystem = LoadObject<UNiagaraSystem>(nullptr, UCEClonerLayoutBase::LayoutBaseAssetPath);
	}
	return BaseNiagaraSystem;
}

TSet<FName> UCEClonerSubsystem::GetLayoutNames() const
{
	TArray<FName> LayoutNames;
	LayoutClasses.GenerateKeyArray(LayoutNames);
	return TSet<FName>(LayoutNames);
}

TSet<TSubclassOf<UCEClonerLayoutBase>> UCEClonerSubsystem::GetLayoutClasses() const
{
	TArray<TSubclassOf<UCEClonerLayoutBase>> Layouts;
	LayoutClasses.GenerateValueArray(Layouts);
	return TSet<TSubclassOf<UCEClonerLayoutBase>>(Layouts);
}

FName UCEClonerSubsystem::FindLayoutName(TSubclassOf<UCEClonerLayoutBase> InLayoutClass) const
{
	if (const FName* Key = LayoutClasses.FindKey(InLayoutClass))
	{
		return *Key;
	}

	return NAME_None;
}

TSubclassOf<UCEClonerLayoutBase> UCEClonerSubsystem::FindLayoutClass(FName InLayoutName) const
{
	if (const TSubclassOf<UCEClonerLayoutBase>* Value = LayoutClasses.Find(InLayoutName))
	{
		return *Value;
	}

	return TSubclassOf<UCEClonerLayoutBase>();
}

UCEClonerLayoutBase* UCEClonerSubsystem::CreateNewLayout(FName InLayoutName, UCEClonerComponent* InCloner)
{
	if (!IsValid(InCloner))
	{
		return nullptr;
	}

	TSubclassOf<UCEClonerLayoutBase> const* LayoutClass = LayoutClasses.Find(InLayoutName);

	if (!LayoutClass)
	{
		return nullptr;
	}

	return NewObject<UCEClonerLayoutBase>(InCloner, LayoutClass->Get());
}

void UCEClonerSubsystem::ScanForRegistrableClasses()
{
	{
		TArray<UClass*> DerivedLayoutClasses;
		GetDerivedClasses(UCEClonerLayoutBase::StaticClass(), DerivedLayoutClasses, true);

		for (UClass* LayoutClass : DerivedLayoutClasses)
		{
			RegisterLayoutClass(LayoutClass);
		}
	}

	{
		TArray<UClass*> DerivedExtensionClasses;
		GetDerivedClasses(UCEClonerExtensionBase::StaticClass(), DerivedExtensionClasses, true);

		for (UClass* ExtensionClass : DerivedExtensionClasses)
		{
			RegisterExtensionClass(ExtensionClass);
		}
	}
}

void UCEClonerSubsystem::OnLevelCleanup(ULevel* InLevel)
{
	if (const TSharedRef<ICEClonerSceneTreeCustomResolver>* CustomResolver = LevelCustomResolvers.Find(InLevel))
	{
		InLevel->OnCleanupLevel.RemoveAll(this);
		(*CustomResolver)->Deactivate();
		LevelCustomResolvers.Remove(InLevel);
	}
}

void UCEClonerSubsystem::OnWorldCleanup(UWorld* InWorld, bool, bool bInCleanupResources)
{
	if (bInCleanupResources)
	{
		for (ULevel* Level : InWorld->GetLevels())
		{
			OnLevelCleanup(Level);
		}
	}
}

#undef LOCTEXT_NAMESPACE
