// Copyright Epic Games, Inc. All Rights Reserved.

#include "Subsystems/CEEffectorSubsystem.h"

#include "Algo/Transform.h"
#include "Effector/CEEffectorComponent.h"
#include "Effector/CEEffectorExtensionBase.h"
#include "Effector/Logs/CEEffectorLogs.h"
#include "Effector/Modes/CEEffectorOffsetMode.h"
#include "Effector/Modes/CEEffectorProceduralMode.h"
#include "Effector/Modes/CEEffectorPushMode.h"
#include "Effector/Modes/CEEffectorTargetMode.h"
#include "Effector/Types/CEEffectorBoxType.h"
#include "Effector/Types/CEEffectorPlaneType.h"
#include "Effector/Types/CEEffectorRadialType.h"
#include "Effector/Types/CEEffectorSphereType.h"
#include "Effector/Types/CEEffectorTorusType.h"
#include "Effector/Types/CEEffectorUnboundType.h"
#include "Engine/Engine.h"
#include "NiagaraDataChannel.h"
#include "NiagaraDataChannelAccessor.h"
#include "NiagaraDataChannelPublic.h"

#if WITH_EDITOR
#include "ScopedTransaction.h"
#endif

UCEEffectorSubsystem::FOnSubsystemInitialized UCEEffectorSubsystem::OnSubsystemInitializedDelegate;
UCEEffectorSubsystem::FOnEffectorIdentifierChanged UCEEffectorSubsystem::OnEffectorIdentifierChangedDelegate;
UCEEffectorSubsystem::FOnEffectorSetEnabled UCEEffectorSubsystem::OnEffectorSetEnabledDelegate;

#define LOCTEXT_NAMESPACE "CEEffectorSubsystem"

UCEEffectorSubsystem* UCEEffectorSubsystem::Get()
{
	if (GEngine)
	{
		return GEngine->GetEngineSubsystem<UCEEffectorSubsystem>();
	}

	return nullptr;
}

void UCEEffectorSubsystem::Initialize(FSubsystemCollectionBase& Collection)
{
	Super::Initialize(Collection);

	// Load niagara data channel asset for effectors and cache it
	EffectorDataChannelAsset = LoadObject<UNiagaraDataChannelAsset>(nullptr, DataChannelAssetPath);

	check(EffectorDataChannelAsset->Get());

	// Register types
	RegisterExtensionClass(UCEEffectorSphereType::StaticClass());
	RegisterExtensionClass(UCEEffectorPlaneType::StaticClass());
	RegisterExtensionClass(UCEEffectorBoxType::StaticClass());
	RegisterExtensionClass(UCEEffectorUnboundType::StaticClass());
	RegisterExtensionClass(UCEEffectorRadialType::StaticClass());
	RegisterExtensionClass(UCEEffectorTorusType::StaticClass());

	// Register modes
	RegisterExtensionClass(UCEEffectorOffsetMode::StaticClass());
	RegisterExtensionClass(UCEEffectorTargetMode::StaticClass());
	RegisterExtensionClass(UCEEffectorProceduralMode::StaticClass());
	RegisterExtensionClass(UCEEffectorPushMode::StaticClass());

	ScanForRegistrableClasses();

	OnSubsystemInitializedDelegate.Broadcast();

	FWorldDelegates::OnWorldTickStart.AddWeakLambda(this, [this](UWorld* InWorld, ELevelTick InLevel, float InDelta)
	{
		const bool bValidWorld = (InWorld->IsGameWorld() || InWorld->IsEditorWorld()) && !InWorld->IsPreviewWorld();

		if (bValidWorld && InLevel != LEVELTICK_TimeOnly)
		{
			UpdateEffectorChannel(InWorld);
		}
	});
}

void UCEEffectorSubsystem::Deinitialize()
{
	Super::Deinitialize();

	FWorldDelegates::OnWorldTickStart.RemoveAll(this);
}

bool UCEEffectorSubsystem::RegisterChannelEffector(UCEEffectorComponent* InEffector)
{
	if (!IsValid(InEffector) || !InEffector->GetOwner())
	{
		return false;
	}

	int32 EffectorIndex = EffectorsWeak.Find(InEffector);

	if (EffectorIndex == INDEX_NONE)
	{
		EffectorIndex = EffectorsWeak.Add(InEffector);
		UE_LOGF(LogCEEffector, Verbose, "%ls effector registered in channel %i", *InEffector->GetOwner()->GetActorNameOrLabel(), EffectorIndex);
	}

	if (InEffector->GetChannelData().Identifier != EffectorIndex)
	{
		const int32 OldIdentifier = InEffector->GetChannelData().Identifier;
		InEffector->GetChannelData().Identifier = EffectorIndex;
		OnEffectorIdentifierChangedDelegate.Broadcast(InEffector, OldIdentifier, EffectorIndex);
	}

	return true;
}

bool UCEEffectorSubsystem::UnregisterChannelEffector(UCEEffectorComponent* InEffector)
{
	if (!InEffector || !InEffector->GetOwner())
	{
		return false;
	}

	const bool bUnregistered = EffectorsWeak.Remove(InEffector) > 0;

	if (bUnregistered)
	{
		UE_LOGF(LogCEEffector, Verbose, "%ls effector unregistered from channel", *InEffector->GetOwner()->GetActorNameOrLabel());

		const int32 OldIdentifier = InEffector->GetChannelData().Identifier;
		InEffector->GetChannelData().Identifier = INDEX_NONE;
		OnEffectorIdentifierChangedDelegate.Broadcast(InEffector, OldIdentifier, InEffector->GetChannelData().Identifier);
	}

	return bUnregistered;
}

UCEEffectorComponent* UCEEffectorSubsystem::GetEffectorByChannelIdentifier(int32 InIdentifier) const
{
	if (EffectorsWeak.IsValidIndex(InIdentifier))
	{
		if (UCEEffectorComponent* Effector = EffectorsWeak[InIdentifier].Get())
		{
			if (Effector->GetChannelIdentifier() == InIdentifier)
			{
				return Effector;
			}
		}
	}

	return nullptr;
}

bool UCEEffectorSubsystem::RegisterExtensionClass(UClass* InClass)
{
	if (!IsValid(InClass))
	{
		return false;
	}

	if (!InClass->IsChildOf(UCEEffectorExtensionBase::StaticClass())
		|| InClass->HasAnyClassFlags(CLASS_Abstract | CLASS_Deprecated | CLASS_NewerVersionExists))
	{
		return false;
	}

	if (IsExtensionClassRegistered(InClass))
	{
		return false;
	}

	const UCEEffectorExtensionBase* CDO = InClass->GetDefaultObject<UCEEffectorExtensionBase>();

	if (!CDO)
	{
		return false;
	}

	const FName ExtensionName = CDO->GetExtensionName();

	if (ExtensionName.IsNone() || ExtensionClasses.Contains(ExtensionName))
	{
		return false;
	}

	ExtensionClasses.Add(ExtensionName, CDO->GetClass());

	return true;
}

bool UCEEffectorSubsystem::UnregisterExtensionClass(UClass* InClass)
{
	if (!IsValid(InClass))
	{
		return false;
	}

	TSubclassOf<UCEEffectorExtensionBase> ExtensionClass(InClass);

	if (const FName* ExtensionName = ExtensionClasses.FindKey(ExtensionClass))
	{
		ExtensionClasses.Remove(*ExtensionName);
		return true;
	}

	return false;
}

bool UCEEffectorSubsystem::IsExtensionClassRegistered(UClass* InClass) const
{
	TSubclassOf<UCEEffectorExtensionBase> ExtensionClass(InClass);
	return !!ExtensionClasses.FindKey(ExtensionClass);
}

TSet<FName> UCEEffectorSubsystem::GetExtensionNames(TSubclassOf<UCEEffectorExtensionBase> InExtensionClass) const
{
	TSet<FName> ExtensionNames;

	const FName ExtensionName = FindExtensionName(InExtensionClass);

	if (!ExtensionName.IsNone())
	{
		ExtensionNames.Add(ExtensionName);
	}
	else
	{
		for (const TPair<FName, TSubclassOf<UCEEffectorExtensionBase>>& ExtensionPair : ExtensionClasses)
		{
			if (ExtensionPair.Value && ExtensionPair.Value->IsChildOf(InExtensionClass))
			{
				ExtensionNames.Add(ExtensionPair.Key);
			}
		}
	}

	return ExtensionNames;
}

TSet<TSubclassOf<UCEEffectorExtensionBase>> UCEEffectorSubsystem::GetExtensionClasses(TSubclassOf<UCEEffectorExtensionBase> InExtensionClass) const
{
	TSet<TSubclassOf<UCEEffectorExtensionBase>> Extensions;

	if (!InExtensionClass.Get())
	{
		return Extensions;
	}

	Extensions.Empty(ExtensionClasses.Num());

	for (const TPair<FName, TSubclassOf<UCEEffectorExtensionBase>>& ExtensionPair : ExtensionClasses)
	{
		if (ExtensionPair.Value
			&& (ExtensionPair.Value == InExtensionClass
				|| ExtensionPair.Value->IsChildOf(InExtensionClass)))
		{
			Extensions.Add(ExtensionPair.Value);
		}
	}

	return Extensions;
}

FName UCEEffectorSubsystem::FindExtensionName(TSubclassOf<UCEEffectorExtensionBase> InClass) const
{
	if (const FName* ExtensionName = ExtensionClasses.FindKey(InClass))
	{
		return *ExtensionName;
	}

	return NAME_None;
}

UCEEffectorExtensionBase* UCEEffectorSubsystem::CreateNewExtension(FName InExtensionName, UCEEffectorComponent* InEffector)
{
	if (!IsValid(InEffector))
	{
		return nullptr;
	}

	TSubclassOf<UCEEffectorExtensionBase> const* ExtensionClass = ExtensionClasses.Find(InExtensionName);

	if (!ExtensionClass)
	{
		return nullptr;
	}

	return NewObject<UCEEffectorExtensionBase>(InEffector, ExtensionClass->Get(), NAME_None, RF_Transactional);
}

void UCEEffectorSubsystem::SetEffectorsEnabled(const TSet<UCEEffectorComponent*>& InEffectors, bool bInEnable, bool bInShouldTransact)
{
	if (InEffectors.IsEmpty())
	{
		return;
	}

#if WITH_EDITOR
	const FText TransactionText = bInEnable
		? LOCTEXT("SetEffectorsEnabled", "Effectors enabled")
		: LOCTEXT("SetEffectorsDisabled", "Effectors disabled");

	FScopedTransaction Transaction(TransactionText, bInShouldTransact);
#endif

	for (UCEEffectorComponent* Effector : InEffectors)
	{
		if (!IsValid(Effector))
		{
			continue;
		}

#if WITH_EDITOR
		Effector->Modify();
#endif

		Effector->SetEnabled(bInEnable);
	}
}

void UCEEffectorSubsystem::SetLevelEffectorsEnabled(const UWorld* InWorld, bool bInEnable, bool bInShouldTransact)
{
	if (!IsValid(InWorld))
	{
		return;
	}

#if WITH_EDITOR
	const FText TransactionText = bInEnable
		? LOCTEXT("SetLevelEffectorsEnabled", "Level effectors enabled")
		: LOCTEXT("SetLevelEffectorsDisabled", "Level effectors disabled");

	FScopedTransaction Transaction(TransactionText, bInShouldTransact);
#endif

	OnEffectorSetEnabledDelegate.Broadcast(InWorld, bInEnable, bInShouldTransact);
}

void UCEEffectorSubsystem::UpdateEffectorChannel(const UWorld* InWorld)
{
	if (!IsValid(InWorld)
		|| !InWorld->IsInitialized()
		|| InWorld->IsBeingCleanedUp()
		|| EffectorsWeak.IsEmpty())
	{
		return;
	}

	// Retrieve effectors in the input world
	TArray<UCEEffectorComponent*> WorldEffectors;
	Algo::TransformIf(
		EffectorsWeak,
		WorldEffectors,
		[InWorld](const TWeakObjectPtr<UCEEffectorComponent>& InEffectorWeak)
		{
			const UCEEffectorComponent* Effector = InEffectorWeak.Get();
			return IsValid(Effector) && Effector->GetWorld() == InWorld;
		},
		[](const TWeakObjectPtr<UCEEffectorComponent>& InEffectorWeak)
		{
			return InEffectorWeak.Get();
		}
	);

	if (WorldEffectors.IsEmpty())
	{
		return;
	}

	// Reserve space in channel for each effectors
	static const FNiagaraDataChannelSearchParameters SearchParameters;
	UNiagaraDataChannelWriter* ChannelWriter = UNiagaraDataChannelLibrary::WriteToNiagaraDataChannel(InWorld, EffectorDataChannelAsset.Get(), SearchParameters, WorldEffectors.Num(), true, true, true, UCEEffectorSubsystem::StaticClass()->GetName());

	if (!ChannelWriter)
	{
		UE_LOGF(LogCEEffector, Warning, "Effector data channel writer is invalid");
		return;
	}

	// Remove invalid effectors and push updates to effector assigned channel indexes
	int32 EffectorIndex = 0;
	for (UCEEffectorComponent* Effector : WorldEffectors)
	{
		FCEClonerEffectorChannelData& ChannelData = Effector->GetChannelData();

		const int32 PreviousIdentifier = ChannelData.Identifier;
		const bool bIdentifierChanged = PreviousIdentifier != EffectorIndex;

		// Set channel before writing
		ChannelData.Identifier = EffectorIndex++;

		// Push effector data to channel
		ChannelData.Write(ChannelWriter);

		// When changed, update cloners DI linked to this effector
		if (bIdentifierChanged)
		{
			OnEffectorIdentifierChangedDelegate.Broadcast(Effector, PreviousIdentifier, ChannelData.Identifier);
		}
	}
}

void UCEEffectorSubsystem::ScanForRegistrableClasses()
{
	TArray<UClass*> DerivedExtensionClasses;
	GetDerivedClasses(UCEEffectorExtensionBase::StaticClass(), DerivedExtensionClasses, true);

	for (UClass* ExtensionClass : DerivedExtensionClasses)
	{
		RegisterExtensionClass(ExtensionClass);
	}
}

TStatId UCEEffectorSubsystem::GetStatId() const
{
	RETURN_QUICK_DECLARE_CYCLE_STAT(UCEEffectorSubsystem, STATGROUP_Tickables);
}

void UCEEffectorSubsystem::Tick(float InDeltaTime)
{
}

bool UCEEffectorSubsystem::IsTickable() const
{
	return false;
}

#undef LOCTEXT_NAMESPACE
