// Copyright Epic Games, Inc. All Rights Reserved.

#include "Utilities/CEClonerEffectorUtilities.h"

#include "Cloner/CEClonerComponent.h"
#include "Materials/Material.h"
#include "UObject/Package.h"

#if WITH_EDITOR
#include "Framework/Notifications/NotificationManager.h"
#include "Widgets/Notifications/SNotificationList.h"
#endif

#define LOCTEXT_NAMESPACE "CEClonerEffectorUtilities"

#if WITH_EDITOR
const FText& UE::ClonerEffector::Utilities::GetMaterialWarningText()
{
	static const FText MaterialWarningText = LOCTEXT("MaterialsMissingUsageFlag", "Detected {0} material(s) with missing usage flag required to work properly with cloner (See logs)");
	return MaterialWarningText;
}

void UE::ClonerEffector::Utilities::ShowWarning(const FText& InWarning)
{
	if (!InWarning.IsEmpty())
	{
		FNotificationInfo NotificationInfo(InWarning);
		NotificationInfo.ExpireDuration = 10.f;
		NotificationInfo.bFireAndForget = true;
		NotificationInfo.Image = FAppStyle::GetBrush("Icons.WarningWithColor");

		FSlateNotificationManager::Get().AddNotification(NotificationInfo);
	}
}
#endif

bool UE::ClonerEffector::Utilities::IsMaterialDirtyable(const UMaterialInterface* InMaterial)
{
	const UMaterial* BaseMaterial = InMaterial->GetMaterial_Concurrent();
	const FString ContentFolder = FPaths::ConvertRelativePathToFull(FPaths::ProjectContentDir());

	const UPackage* MaterialPackage = BaseMaterial->GetPackage();
	const FPackagePath& LoadedPath = MaterialPackage->GetLoadedPath();
	const FString PackagePath = FPaths::ConvertRelativePathToFull(LoadedPath.GetLocalFullPath());
	const FString MaterialPath = BaseMaterial->GetPathName();

	const bool bTransientPackage = MaterialPackage == GetTransientPackage() || MaterialPath.StartsWith("/Temp/");
	const bool bContentFolder = PackagePath.StartsWith(ContentFolder);

	return bTransientPackage || bContentFolder;
}

bool UE::ClonerEffector::Utilities::IsMaterialUsageFlagSet(const UMaterialInterface* InMaterial)
{
	if (InMaterial)
	{
		return InMaterial->GetUsageByFlag(EMaterialUsage::MATUSAGE_NiagaraMeshParticles);
	}

	return false;
}

bool UE::ClonerEffector::Utilities::FilterSupportedMaterials(TArray<TWeakObjectPtr<UMaterialInterface>>& InMaterials, TArray<TWeakObjectPtr<UMaterialInterface>>& OutUnsetMaterials, UMaterialInterface* InDefaultMaterial)
{
	check(InDefaultMaterial)

	OutUnsetMaterials.Reset(InMaterials.Num());

	for (int32 Index = 0; Index < InMaterials.Num(); Index++)
	{
		UMaterialInterface* PreviousMaterialInterface = InMaterials[Index].Get();

		UMaterialInterface* NewMaterialInterface = PreviousMaterialInterface;

		if (FilterSupportedMaterial(NewMaterialInterface, InDefaultMaterial))
		{
			// Add original material to unset list
			OutUnsetMaterials.Add(PreviousMaterialInterface);
		}

		// Replace material
		InMaterials[Index] = NewMaterialInterface;
	}

	return OutUnsetMaterials.IsEmpty();
}

bool UE::ClonerEffector::Utilities::FilterSupportedMaterial(UMaterialInterface*& InMaterial, UMaterialInterface* InDefaultMaterial)
{
	if (InMaterial && !IsMaterialUsageFlagSet(InMaterial))
	{
		// Replace material if dirtyable and not in read only location
		if (!IsMaterialDirtyable(InMaterial))
		{
			InMaterial = InDefaultMaterial;
		}

		return true;
	}

	return false;
}

void UE::ClonerEffector::Utilities::SetActorVisibility(AActor* InActor, bool bInVisibility, ECEClonerActorVisibility InTarget)
{
	if (!InActor)
	{
		return;
	}

#if WITH_EDITOR
	if (EnumHasAnyFlags(InTarget, ECEClonerActorVisibility::Editor))
	{
		InActor->SetIsTemporarilyHiddenInEditor(!bInVisibility);
	}
#endif

	if (EnumHasAnyFlags(InTarget, ECEClonerActorVisibility::Game))
	{
		InActor->SetActorHiddenInGame(!bInVisibility);
	}
}

AActor* UE::ClonerEffector::Utilities::FindClonerActor(AActor* InActor)
{
	if (!InActor)
	{
		return nullptr;
	}

	if (!!InActor->FindComponentByClass<UCEClonerComponent>())
	{
		return InActor;
	}

	return FindClonerActor(InActor->GetAttachParentActor());
}

#undef LOCTEXT_NAMESPACE
