// Copyright Epic Games, Inc. All Rights Reserved.

#include "Utilities/CEClonerLayoutLatentAction.h"

#include "Cloner/CEClonerComponent.h"
#include "Cloner/Layouts/CEClonerLayoutBase.h"
#include "Engine/LatentActionManager.h"
#include "HAL/Platform.h"

FCEClonerLayoutLatentAction::FCEClonerLayoutLatentAction(const FLatentActionInfo& InLatentInfo, UCEClonerComponent* InCloner, TSubclassOf<UCEClonerLayoutBase> InLayoutClass, bool& bOutSuccess, UCEClonerLayoutBase*& OutLayout)
	: ExecutionFunction(InLatentInfo.ExecutionFunction)
	, OutputLink(InLatentInfo.Linkage)
	, CallbackTarget(InLatentInfo.CallbackTarget)
	, bLayoutSet(false)
	, ClonerWeak(InCloner)
	, LayoutClass(InLayoutClass)
	, bSuccess(bOutSuccess)
	, Layout(OutLayout)
{
}

FCEClonerLayoutLatentAction::~FCEClonerLayoutLatentAction()
{
	UCEClonerComponent::OnClonerLayoutLoaded().RemoveAll(this);
}

#if WITH_EDITOR
FString FCEClonerLayoutLatentAction::GetDescription() const
{
	const UCEClonerComponent* Cloner = ClonerWeak.Get();
	return FString::Printf(TEXT("Cloner %s layout %s loaded : %i"), Cloner ? *Cloner->GetName() : TEXT("Invalid"), *LayoutClass->GetName(), bSuccess);
}
#endif

void FCEClonerLayoutLatentAction::OnClonerLayoutLoaded(UCEClonerComponent* InClonerComponent, UCEClonerLayoutBase* InClonerLayoutBase) const
{
	if (InClonerComponent == ClonerWeak.Get())
	{
		if (InClonerLayoutBase && InClonerLayoutBase->GetClass() == LayoutClass)
		{
			Layout = InClonerLayoutBase;
			bSuccess = true;
		}
	}
}

void FCEClonerLayoutLatentAction::UpdateOperation(FLatentResponse& OutResponse)
{
	UCEClonerComponent* Cloner = ClonerWeak.Get();

	if (Cloner && !bLayoutSet)
	{
		bLayoutSet = true;

		if (Cloner->GetLayoutClass() != LayoutClass)
		{
			UCEClonerComponent::OnClonerLayoutLoaded().AddRaw(this, &FCEClonerLayoutLatentAction::OnClonerLayoutLoaded);

			if (Cloner)
			{
				Cloner->SetLayoutClass(LayoutClass);
			}
		}
		else
		{
			OnClonerLayoutLoaded(Cloner, Cloner->GetActiveLayout());
		}
	}

	bool bDone = bSuccess;

	if (!Cloner)
	{
		bSuccess = false;
		bDone = true;
	}

	OutResponse.FinishAndTriggerIf(bDone, ExecutionFunction, OutputLink, CallbackTarget);
}