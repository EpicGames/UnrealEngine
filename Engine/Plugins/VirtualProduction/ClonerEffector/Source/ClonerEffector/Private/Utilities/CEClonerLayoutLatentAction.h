// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "LatentActions.h"
#include "Templates/SubclassOf.h"
#include "UObject/WeakObjectPtrFwd.h"

class UCEClonerLayoutBase;
class UCEClonerComponent;
struct FLatentActionInfo;

/** Latent action to set cloner layout and wait until layout is active and ready to use */
class FCEClonerLayoutLatentAction : public FPendingLatentAction
{
public:
	FCEClonerLayoutLatentAction(const FLatentActionInfo& InLatentInfo, UCEClonerComponent* InCloner, TSubclassOf<UCEClonerLayoutBase> InLayoutClass, bool& bOutSuccess, UCEClonerLayoutBase*& OutLayout);

	virtual ~FCEClonerLayoutLatentAction();

protected:
	//~ Begin FPendingLatentAction
	virtual void UpdateOperation(FLatentResponse& OutResponse) override;
#if WITH_EDITOR
	virtual FString GetDescription() const override;
#endif
	//~ End FPendingLatentAction

	void OnClonerLayoutLoaded(UCEClonerComponent* InClonerComponent, UCEClonerLayoutBase* InClonerLayoutBase) const;

	FName ExecutionFunction;
	int32 OutputLink;
	FWeakObjectPtr CallbackTarget;

	bool bLayoutSet;
	TWeakObjectPtr<UCEClonerComponent> ClonerWeak;
	TSubclassOf<UCEClonerLayoutBase> LayoutClass;

	bool& bSuccess;
	UCEClonerLayoutBase*& Layout;
};
