// Copyright Epic Games, Inc. All Rights Reserved.

#include "Utilities/CEClonerLibrary.h"

#include "CEClonerLayoutLatentAction.h"
#include "Cloner/CEClonerComponent.h"
#include "Containers/Set.h"
#include "Engine/Engine.h"
#include "Engine/LatentActionManager.h"
#include "Engine/World.h"
#include "Subsystems/CEClonerSubsystem.h"

void UCEClonerLibrary::GetClonerLayoutClasses(TSet<TSubclassOf<UCEClonerLayoutBase>>& OutLayoutClasses)
{
	OutLayoutClasses.Empty();

	if (const UCEClonerSubsystem* ClonerSubsystem = UCEClonerSubsystem::Get())
	{
		OutLayoutClasses = ClonerSubsystem->GetLayoutClasses();
	}
}

void UCEClonerLibrary::GetClonerExtensionClasses(TSet<TSubclassOf<UCEClonerExtensionBase>>& OutExtensionClasses)
{
	OutExtensionClasses.Empty();

	if (const UCEClonerSubsystem* ClonerSubsystem = UCEClonerSubsystem::Get())
	{
		OutExtensionClasses = ClonerSubsystem->GetExtensionClasses();
	}
}

bool UCEClonerLibrary::GetClonerLayoutName(TSubclassOf<UCEClonerLayoutBase> InLayoutClass, FName& OutLayoutName)
{
	if (const UCEClonerSubsystem* ClonerSubsystem = UCEClonerSubsystem::Get())
	{
		OutLayoutName = ClonerSubsystem->FindLayoutName(InLayoutClass);
	}

	return !OutLayoutName.IsNone();
}

bool UCEClonerLibrary::GetClonerLayoutNames(TSet<FName>& OutLayoutNames)
{
	if (const UCEClonerSubsystem* ClonerSubsystem = UCEClonerSubsystem::Get())
	{
		OutLayoutNames = ClonerSubsystem->GetLayoutNames();
	}

	return !OutLayoutNames.IsEmpty();
}

bool UCEClonerLibrary::GetClonerLayoutClass(FName InLayoutName, TSubclassOf<UCEClonerLayoutBase>& OutLayoutClass)
{
	if (const UCEClonerSubsystem* ClonerSubsystem = UCEClonerSubsystem::Get())
	{
		OutLayoutClass = ClonerSubsystem->FindLayoutClass(InLayoutName);
	}

	return !!OutLayoutClass.Get();
}

void UCEClonerLibrary::SetClonerLayoutByClass(const UObject* InWorldContext, FLatentActionInfo InLatentInfo, UCEClonerComponent* InCloner, TSubclassOf<UCEClonerLayoutBase> InLayoutClass, bool& bOutSuccess, UCEClonerLayoutBase*& OutLayout)
{
	if (UWorld* World = GEngine->GetWorldFromContextObject(InWorldContext, EGetWorldErrorMode::LogAndReturnNull))
	{
		FLatentActionManager& LatentActionManager = World->GetLatentActionManager();

		if (!LatentActionManager.FindExistingAction<FCEClonerLayoutLatentAction>(InLatentInfo.CallbackTarget, InLatentInfo.UUID))
		{
			LatentActionManager.AddNewAction(InLatentInfo.CallbackTarget, InLatentInfo.UUID, new FCEClonerLayoutLatentAction(InLatentInfo, InCloner, InLayoutClass, bOutSuccess, OutLayout));
		}
	}
}

void UCEClonerLibrary::SetClonerLayoutByName(const UObject* InWorldContext, FLatentActionInfo InLatentInfo, UCEClonerComponent* InCloner, FName InLayoutName, bool& bOutSuccess, UCEClonerLayoutBase*& OutLayout)
{
	TSubclassOf<UCEClonerLayoutBase> LayoutClass;
	if (UCEClonerLibrary::GetClonerLayoutClass(InLayoutName, LayoutClass))
	{
		UCEClonerLibrary::SetClonerLayoutByClass(InWorldContext, InLatentInfo, InCloner, LayoutClass, bOutSuccess, OutLayout);
	}
}
