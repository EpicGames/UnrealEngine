// Copyright Epic Games, Inc. All Rights Reserved.

#include "Utilities/CEEffectorLibrary.h"

#include "Containers/Set.h"
#include "Effector/Effects/CEEffectorEffectBase.h"
#include "Effector/Modes/CEEffectorModeBase.h"
#include "Effector/Types/CEEffectorTypeBase.h"
#include "Subsystems/CEEffectorSubsystem.h"

void UCEEffectorLibrary::GetEffectorModeClasses(TSet<TSubclassOf<UCEEffectorModeBase>>& OutModeClasses)
{
	OutModeClasses.Empty();

	if (const UCEEffectorSubsystem* EffectorSubsystem = UCEEffectorSubsystem::Get())
	{
		for (TSubclassOf<UCEEffectorExtensionBase> ModeClass : EffectorSubsystem->GetExtensionClasses<UCEEffectorModeBase>())
		{
			OutModeClasses.Add(ModeClass.Get());
		}
	}
}

void UCEEffectorLibrary::GetEffectorTypeClasses(TSet<TSubclassOf<UCEEffectorTypeBase>>& OutTypeClasses)
{
	OutTypeClasses.Empty();

	if (const UCEEffectorSubsystem* EffectorSubsystem = UCEEffectorSubsystem::Get())
	{
		for (TSubclassOf<UCEEffectorExtensionBase> TypeClass : EffectorSubsystem->GetExtensionClasses<UCEEffectorTypeBase>())
		{
			OutTypeClasses.Add(TypeClass.Get());
		}
	}
}

void UCEEffectorLibrary::GetEffectorTypeNames(TSet<FName>& OutTypeNames)
{
	if (const UCEEffectorSubsystem* EffectorSubsystem = UCEEffectorSubsystem::Get())
	{
		OutTypeNames = EffectorSubsystem->GetExtensionNames<UCEEffectorTypeBase>();
	}
}

void UCEEffectorLibrary::GetEffectorEffectClasses(TSet<TSubclassOf<UCEEffectorEffectBase>>& OutEffectClasses)
{
	OutEffectClasses.Empty();

	if (const UCEEffectorSubsystem* EffectorSubsystem = UCEEffectorSubsystem::Get())
	{
		for (TSubclassOf<UCEEffectorExtensionBase> EffectClass : EffectorSubsystem->GetExtensionClasses<UCEEffectorEffectBase>())
		{
			OutEffectClasses.Add(EffectClass.Get());
		}
	}
}
