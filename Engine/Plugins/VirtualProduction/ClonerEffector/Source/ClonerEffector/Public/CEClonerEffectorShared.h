// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Components/SceneComponent.h"
#include "Engine/StaticMesh.h"
#include "Materials/MaterialInterface.h"
#include "CEClonerEffectorShared.generated.h"

class AActor;
class ADynamicMeshActor;
class AStaticMeshActor;
class UCEClonerComponent;
class UClass;
class UDynamicMesh;
class UNiagaraDataChannelReader;
class UNiagaraDataChannelWriter;
class UNiagaraDataInterfaceArrayMesh;
class UNiagaraSystem;
class USplineComponent;

/** Enumerate the axis available to use */
UENUM(BlueprintType)
enum class ECEClonerAxis : uint8
{
	X,
	Y,
	Z,
	Custom
};

/** Enumerate the planes available to use */
UENUM(BlueprintType)
enum class ECEClonerPlane : uint8
{
	XY,
	YZ,
	XZ,
	Custom
};

/** Enumerate the mesh render modes available to render a cloner instance */
UENUM(BlueprintType)
enum class ECEClonerMeshRenderMode : uint8
{
	/** Iterate through each attachment mesh available */
	Iterate,
	/** Pick randomly through each attachment mesh available, update cloner seed for variations */
	Random,
	/** Blend based on the total cloner instances and attachment mesh available */
	Blend
};

/** Enumerate the grid constraints available when cloner layout selected is grid */
UENUM(BlueprintType)
enum class ECEClonerGridConstraint : uint8
{
	None,
	Sphere,
	Cylinder,
	Texture
};

/* Enumerates all easing functions that are available to apply on weights
See https://easings.net/ for curves visualizations and web open domain implementations
Used as one enum to send the index to niagara as uint8 and apply easing directly in niagara */
UENUM(BlueprintType)
enum class ECEClonerEasing : uint8
{
	Linear,

	InSine,
	OutSine,
	InOutSine,

	InQuad,
	OutQuad,
	InOutQuad,

	InCubic,
	OutCubic,
	InOutCubic,

	InQuart,
	OutQuart,
	InOutQuart,

	InQuint,
	OutQuint,
	InOutQuint,

	InExpo,
	OutExpo,
	InOutExpo,

	InCirc,
	OutCirc,
	InOutCirc,

	InBack,
	OutBack,
	InOutBack,

	InElastic,
	OutElastic,
	InOutElastic,

	InBounce,
	OutBounce,
	InOutBounce,

	Random
};

/** Enumerate the mesh asset to look for when mesh layout is selected */
UENUM(BlueprintType)
enum class ECEClonerMeshAsset : uint8
{
	StaticMesh,
	SkeletalMesh
};

/** Enumerate the mesh sample dataset to pick from when mesh layout is selected */
UENUM(BlueprintType)
enum class ECEClonerMeshSampleData : uint8
{
	Vertices,
	Triangles,
	Sockets,
	Bones,
	Sections
};

/** Enumerates the effector shapes available */
UENUM(BlueprintType)
enum class ECEClonerEffectorType : uint8
{
	/** Clones inside the sphere radius will be affected by the effector */
	Sphere,
	/** Clones between two planes will be affected by the effector */
	Plane,
	/** Clones inside the box extent will be affected by the effector */
	Box,
	/** All clones will be affected by the effector with the same max weight */
	Unbound,
	/** All clones within the angle range will be affected */
	Radial,
	/** All clones inside the torus radius will be affected by the effector */
	Torus
};

/** Enumerates the effector mode available */
UENUM(BlueprintType)
enum class ECEClonerEffectorMode : uint8
{
	/** Control clones offset, rotation, scale manually */
	Default,
	/** Rotates clones towards a target actor */
	Target,
	/** Procedural pattern applied across the field zone */
	Procedural,
	/** Pushes clones apart based on a strength and direction */
	Push,
	/** Accumulate transform on clones based on their index */
	Step,
	/** Culls clones based on a zone */
	Cull,
	/** Cancels/reverses all effects */
	Cancel
};

/** Enumerates the effector procedural pattern available */
UENUM(BlueprintType)
enum class ECEClonerEffectorProceduralPattern : uint8
{
	/** Jacobian Simplex Noise */
	CurlNoise,
	/** Wave moving towards specific direction */
	DirectionalWave,
	/** Ripple wave movement */
	CircularWave
};

/** Enumerates the effector push direction available */
UENUM(BlueprintType)
enum class ECEClonerEffectorPushDirection : uint8
{
	/** Push based on the clone forward vector */
	Forward,
	/** Push based on the clone right vector */
	Right,
	/** Push based on the cloner up vector */
	Up,
	/** Push based on the clone position relative to the effector */
	Effector,
	/** Push based on a random unit vector based on the cloner seed */
	Random
};

/** Enumerate all texture channels to sample for constraint */
UENUM()
enum class ECEClonerTextureSampleChannel : uint8
{
	RGBLuminance,
	RGBAverage,
	RGBMax,
	R,
	G,
	B,
	A
};

/** Enumerate all operation compare mode for constraint */
UENUM()
enum class ECEClonerCompareMode : uint8
{
	Greater,
	GreaterEqual,
	Equal,
	NotEqual,
	Less,
	LessEqual
};

/** Enumerates all modes to handle cloner spawn mode */
UENUM()
enum class ECEClonerSpawnLoopMode : uint8
{
	/** Cloner spawns once and then enters idle mode */
	Once,
	/** Cloner spawns multiple times and then enters idle mode */
	Multiple,
	/** Cloner spawns infinitely and never enters idle mode */
	Infinite
};

/** Enumerates all modes for how clones are spawned */
UENUM()
enum class ECEClonerSpawnBehaviorMode : uint8
{
	/** Spawns instantly the number of clones needed for the layout at a specific interval */
	Instant,
	/** Spawns clones at a specific rate per second and fills the layout */
	Rate,
};

/** Enumerates all modes for how clones radius are calculed */
UENUM()
enum class ECEClonerCollisionRadiusMode : uint8
{
	/** Input collision radius manually */
	Manual,
	/** Collision radius will be calculated automatically based on the min extent value, mesh scale included */
	MinExtent,
	/** Collision radius will be calculated automatically based on the max extent value, mesh scale included */
	MaxExtent,
	/** Collision radius will be calculated automatically based on the extent length, mesh scale included */
	ExtentLength
};

/** Enumerates all conversion possible for cloner simulation */
enum class ECEClonerMeshConversion : uint8
{
	StaticMesh,
	StaticMeshes,
	DynamicMesh,
	DynamicMeshes,
	InstancedStaticMesh
};

/** Enumerates all states for extension/layout */
enum class ECEClonerSystemStatus : uint8
{
	/** Nothing to do */
	UpToDate = 0,
	/** Parameters needs an update */
	ParametersDirty = 1 << 0,
	/** Simulation needs an update */
	SimulationDirty = 1 << 1
};

/** Enumerates all visibility options for an actor */
enum class ECEClonerActorVisibility : uint8
{
	None = 0,
	Game = 1 << 0,
	Editor = 1 << 1,
	All = Game | Editor
};

USTRUCT()
struct FCEClonerEffectorChannelData
{
	GENERATED_BODY()

	friend class UCEEffectorSubsystem;

	/** General */
	static constexpr const TCHAR* MagnitudeName = TEXT("Magnitude");
	static constexpr const TCHAR* LocationName = TEXT("Location");
	static constexpr const TCHAR* RotationName = TEXT("Rotation");
	static constexpr const TCHAR* ScaleName = TEXT("Scale");
	static constexpr const TCHAR* ColorName = TEXT("Color");

	/** Type/Shape */
	static constexpr const TCHAR* TypeName = TEXT("Type");
	static constexpr const TCHAR* EasingName = TEXT("Easing");
	static constexpr const TCHAR* InnerExtentName = TEXT("InnerExtent");
	static constexpr const TCHAR* OuterExtentName = TEXT("OuterExtent");

	/** Mode */
	static constexpr const TCHAR* ModeName = TEXT("Mode");
	static constexpr const TCHAR* LocationDeltaName = TEXT("LocationDelta");
	static constexpr const TCHAR* RotationDeltaName = TEXT("RotationDelta");
	static constexpr const TCHAR* ScaleDeltaName = TEXT("ScaleDelta");
	static constexpr const TCHAR* FrequencyName = TEXT("Frequency");
	static constexpr const TCHAR* PanName = TEXT("Pan");
	static constexpr const TCHAR* PatternName = TEXT("Pattern");

	/** Delay */
	static constexpr const TCHAR* DelayInDurationName = TEXT("DelayInDuration");
	static constexpr const TCHAR* DelayOutDurationName = TEXT("DelayOutDuration");
	static constexpr const TCHAR* DelaySpringFrequencyName = TEXT("DelaySpringFrequency");
	static constexpr const TCHAR* DelaySpringFalloffName = TEXT("DelaySpringFalloff");

	/** Forces */
	static constexpr const TCHAR* OrientationForceRateName = TEXT("OrientationForceRate");
	static constexpr const TCHAR* OrientationForceMinName = TEXT("OrientationForceMin");
	static constexpr const TCHAR* OrientationForceMaxName = TEXT("OrientationForceMax");
	static constexpr const TCHAR* VortexForceAmountName = TEXT("VortexForceAmount");
	static constexpr const TCHAR* VortexForceAxisName = TEXT("VortexForceAxis");
	static constexpr const TCHAR* CurlNoiseForceStrengthName = TEXT("CurlNoiseForceStrength");
	static constexpr const TCHAR* CurlNoiseForceFrequencyName = TEXT("CurlNoiseForceFrequency");
	static constexpr const TCHAR* AttractionForceStrengthName = TEXT("AttractionForceStrength");
	static constexpr const TCHAR* AttractionForceFalloffName = TEXT("AttractionForceFalloff");
	static constexpr const TCHAR* GravityForceAccelerationName = TEXT("GravityForceAcceleration");
	static constexpr const TCHAR* DragForceLinearName = TEXT("DragForceLinear");
	static constexpr const TCHAR* DragForceRotationalName = TEXT("DragForceRotational");
	static constexpr const TCHAR* VectorNoiseForceAmountName = TEXT("VectorNoiseForceAmount");

	int32 GetIdentifier() const
	{
		return Identifier;
	}

	/** General */
	float Magnitude = 0.f;
	FVector Location = FVector::ZeroVector;
	FQuat Rotation = FQuat::Identity;
	FVector Scale = FVector::OneVector;
	FLinearColor Color = FLinearColor::Transparent;

	/** Type */
	ECEClonerEffectorType Type = ECEClonerEffectorType::Sphere;
	ECEClonerEasing Easing = ECEClonerEasing::Linear;
	FVector InnerExtent = FVector::ZeroVector;
	FVector OuterExtent = FVector::ZeroVector;

	/** Mode */
	ECEClonerEffectorMode Mode = ECEClonerEffectorMode::Default;
	FVector LocationDelta = FVector::ZeroVector;
	FVector RotationDelta = FVector::ZeroVector;
	FVector ScaleDelta = FVector::OneVector;
	float Frequency = 1.f;
	FVector Pan = FVector::ZeroVector;
	ECEClonerEffectorProceduralPattern Pattern = ECEClonerEffectorProceduralPattern::CurlNoise;

	/** Delay */
	float DelayInDuration = 0.f;
	float DelayOutDuration = 0.f;
	float DelaySpringFrequency = 0.f;
	float DelaySpringFalloff = 0.f;

	/** Forces */
	float OrientationForceRate = 0.f;
	FVector OrientationForceMin = FVector::ZeroVector;
	FVector OrientationForceMax = FVector::ZeroVector;
	float VortexForceAmount = 0.f;
	FVector VortexForceAxis = FVector::ZeroVector;
	float CurlNoiseForceStrength = 0.f;
	float CurlNoiseForceFrequency = 0.f;
	float AttractionForceStrength = 0.f;
	float AttractionForceFalloff = 0.f;
	FVector GravityForceAcceleration = FVector::ZeroVector;
	float DragForceLinear = 0.f;
	float DragForceRotational = 0.f;
	float VectorNoiseForceAmount = 0.f;

protected:
	/** Cached effector identifier to detect a change and update cloners DI */
	UPROPERTY(VisibleInstanceOnly, Category="Effector", meta=(NoResetToDefault))
	int32 Identifier = INDEX_NONE;

	void Write(UNiagaraDataChannelWriter* InWriter) const;
	void Read(const UNiagaraDataChannelReader* InReader);
};

USTRUCT(BlueprintType)
struct CLONEREFFECTOR_API FCEClonerGridConstraintSphere
{
	GENERATED_BODY()

	UPROPERTY(BlueprintReadWrite, EditInstanceOnly, Category="Cloner")
	float Radius = 400.f;

	UPROPERTY(BlueprintReadWrite, EditInstanceOnly, Category="Cloner")
	FVector Center = FVector(0);
};

USTRUCT(BlueprintType)
struct CLONEREFFECTOR_API FCEClonerGridConstraintCylinder
{
	GENERATED_BODY()

	UPROPERTY(BlueprintReadWrite, EditInstanceOnly, Category="Cloner")
	float Radius = 400.f;

	UPROPERTY(BlueprintReadWrite, EditInstanceOnly, Category="Cloner")
	float Height = 800.f;

	UPROPERTY(BlueprintReadWrite, EditInstanceOnly, Category="Cloner")
	FVector Center = FVector(0);
};

USTRUCT(BlueprintType)
struct CLONEREFFECTOR_API FCEClonerGridConstraintTexture
{
	GENERATED_BODY()

	UPROPERTY(BlueprintReadWrite, EditInstanceOnly, Category="Cloner")
	TWeakObjectPtr<UTexture> Texture;

	UPROPERTY(BlueprintReadWrite, EditInstanceOnly, Category="Cloner")
	ECEClonerTextureSampleChannel Channel = ECEClonerTextureSampleChannel::RGBLuminance;

	UPROPERTY(BlueprintReadWrite, EditInstanceOnly, Category="Cloner")
	ECEClonerPlane Plane = ECEClonerPlane::XY;

	UPROPERTY(BlueprintReadWrite, EditInstanceOnly, Category="Cloner")
	ECEClonerCompareMode CompareMode = ECEClonerCompareMode::Greater;

	UPROPERTY(BlueprintReadWrite, EditInstanceOnly, Category="Cloner")
	float Threshold = 0.f;
};

#if WITH_EDITOR
struct FCEExtensionSection
{
	FCEExtensionSection()
	{}

	explicit FCEExtensionSection(FName InSectionName, int32 InSectionOrder)
		: SectionName(InSectionName)
		, SectionOrder(InSectionOrder)
	{}

	/** Used for editor details UI */
	FName SectionName = NAME_None;

	/** Used to reorder categories in editor UI */
	int32 SectionOrder = 0;
};
#endif

namespace UE::ClonerEffector
{
	/** Name for the mesh array data interface to use */
	static const FLazyName MeshArrayName = TEXT("MeshArray"); 

	/** Finds the 'MeshArray' data interface in the given cloner */
	UNiagaraDataInterfaceArrayMesh* FindMeshArrayDataInterface(UCEClonerComponent* InClonerComponent);

#if WITH_EDITOR
	namespace EditorSection
	{
		/** Retrieves section metadata from class */
		FCEExtensionSection GetExtensionSectionFromClass(UClass* InClass);
	}
#endif

	namespace Conversion
	{
		/** Convert a cloner to a single merged static mesh actor */
		AStaticMeshActor* ConvertClonerToStaticMesh(UCEClonerComponent* InCloner);

		/** Convert a cloner to a single merged dynamic mesh actor */
		ADynamicMeshActor* ConvertClonerToDynamicMesh(UCEClonerComponent* InCloner);

		/** Convert a cloner to multiple static mesh actors */
		TArray<AStaticMeshActor*> ConvertClonerToStaticMeshes(UCEClonerComponent* InCloner);

		/** Convert a cloner to multiple dynamic mesh actors */
		TArray<ADynamicMeshActor*> ConvertClonerToDynamicMeshes(UCEClonerComponent* InCloner);

		/** Convert a cloner to multiple instanced static mesh actors */
		TArray<AActor*> ConvertClonerToInstancedStaticMeshes(UCEClonerComponent* InCloner);

		/** Creates the root component for an actor */
		UActorComponent* CreateRootComponent(AActor* InActor, TSubclassOf<USceneComponent> InComponentClass, const FTransform& InWorldTransform);

#if WITH_EDITOR
		/** Pick assets location */
		bool PickAssetPath(const FString& InDefaultPath, FString& OutPickedPath);

		/** Create a specific asset in a package */
		UObject* CreateAssetPackage(TSubclassOf<UObject> InAssetClass, const FString& InAssetPath);

		template<typename InClass
			UE_REQUIRES(TIsDerivedFrom<InClass, UObject>::Value)>
		InClass* CreateAssetPackage(const FString& InAssetPath)
		{
			return Cast<InClass>(CreateAssetPackage(InClass::StaticClass(), InAssetPath));
		}
#endif

	}
}