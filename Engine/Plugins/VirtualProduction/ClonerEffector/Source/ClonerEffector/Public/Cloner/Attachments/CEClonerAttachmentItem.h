// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "CEClonerAttachmentTypes.h"
#include "UObject/WeakObjectPtr.h"
#include "CEClonerAttachmentItem.generated.h"

class AActor;
class UDynamicMesh;
class UMaterialInterface;

USTRUCT()
struct FCEClonerAttachmentItem
{
	GENERATED_BODY()

	/** Children attached to this actor, order is not important here as they are combined into one */
	UPROPERTY()
	TSet<TWeakObjectPtr<AActor>> ChildrenActors;

	/** Current cloner attached actor represented by this item */
	UPROPERTY()
	TWeakObjectPtr<AActor> ItemActor;

	/** Parent of this item actor or null if below cloner */
	UPROPERTY()
	TWeakObjectPtr<AActor> ParentActor;

	/** Actual baked mesh for the current actor item,
		dynamic because it's easier for merging them together and avoid conversions again */
	UPROPERTY(Transient)
	TObjectPtr<UDynamicMesh> BakedMesh;

	/** Actual baked materials for the current actor item */
	UPROPERTY()
	TArray<TWeakObjectPtr<UMaterialInterface>> BakedMaterials;

	/** Status of the baked mesh, does it needs to be updated */
	UPROPERTY()
	ECEClonerAttachmentStatus MeshStatus = ECEClonerAttachmentStatus::Outdated;

	/** Last actor item transform, used to trigger an update if changed */
	UPROPERTY()
	FTransform ActorTransform = FTransform::Identity;

	/** Item is root cloner actor */
	UPROPERTY()
	bool bRootItem = false;

	/** Item setup is done (visibility, delegates) once attached/loaded */
	UPROPERTY(Transient)
	bool bSetupDone = false;

	/** Status of this attachment item */
	UPROPERTY(Transient)
	ECEClonerAttachmentStatus Status = ECEClonerAttachmentStatus::Outdated;

#if WITH_EDITORONLY_DATA
	/** Cached pivot to detect changes in geometry */
	UPROPERTY(Transient)
	FVector Origin = FVector::ZeroVector;

	/** Cached extent to detect changes in geometry */
	UPROPERTY(Transient)
	FVector Extent = FVector::ZeroVector;
#endif

	friend uint32 GetTypeHash(const FCEClonerAttachmentItem& InItem)
	{
		return GetTypeHash(InItem.ItemActor);
	}

	bool operator==(const FCEClonerAttachmentItem& InOther) const
	{
		return ItemActor.Get() == InOther.ItemActor.Get();
	}

#if WITH_EDITOR
	/** Check this item bounds without recursing */
	bool CheckBoundsChanged(bool bInUpdate);

	/** Calculate baked mesh bounds */
	FBox GetBakedMeshBounds() const;

	/** Get local attachment bounds */
	FBox GetAttachmentBounds() const;
#endif

	/** Check this item materials to ensure they are supported */
	bool CheckMaterialsChanged(bool bInUpdate, TArray<TWeakObjectPtr<UMaterialInterface>>* OutInvalidMaterials = nullptr);
};