// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "CEClonerAttachmentTypes.h"
#include "Misc/Optional.h"
#include "UObject/ObjectPtr.h"
#include "UObject/WeakObjectPtrTemplates.h"
#include "CEClonerAttachmentRootItem.generated.h"

class AActor;
class UStaticMesh;

USTRUCT()
struct FCEClonerAttachmentRootItem
{
	GENERATED_BODY()

	FCEClonerAttachmentRootItem()
	{}

	FCEClonerAttachmentRootItem(AActor* InActor);

	friend uint32 GetTypeHash(const FCEClonerAttachmentRootItem& InItem)
	{
		return GetTypeHash(InItem.RootActor);
	}

	bool operator==(const FCEClonerAttachmentRootItem& InOther) const
	{
		return RootActor.IsValid() && RootActor.Get() == InOther.RootActor.Get();
	}

	void Reset()
	{
		MergedBakedMesh = nullptr;
#if WITH_EDITORONLY_DATA
		MeshSize.Reset();
#endif
	}

	/** Actors directly attached to the cloner actor, order is important here */
	UPROPERTY()
	TWeakObjectPtr<AActor> RootActor;

	/**
	 * Merged static meshes corresponding to the root actors for niagara,
	 * result of merging dynamic meshes together
	 */
	UPROPERTY()
	TObjectPtr<UStaticMesh> MergedBakedMesh;

#if WITH_EDITORONLY_DATA
	/** Mesh size of the baked mesh / root actor tree, used to compare and detect a change */
	UPROPERTY()
	TOptional<FVector> MeshSize;
#endif
};