// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CEClonerAttachmentItem.h"
#include "CEClonerAttachmentRootItem.h"
#include "CEClonerAttachmentTypes.h"
#include "Components/SceneComponent.h"
#include "Containers/Array.h"
#include "Containers/Map.h"
#include "Containers/Set.h"
#include "Templates/SharedPointerFwd.h"
#include "UObject/ObjectPtr.h"
#include "CEClonerAttachmentTree.generated.h"

class AActor;
class ICEClonerAttachmentTreeBehavior;
class UObject;
class UStaticMesh;

/**
 * Represents a logical tree used by the cloner to render its attachments
 * Could differ from the actual physical tree used in the scene
 */
USTRUCT()
struct FCEClonerAttachmentTree
{
	GENERATED_BODY()

	DECLARE_DELEGATE_TwoParams(FOnAttachmentChanged, AActor*, FCEClonerAttachmentItem&)

	FOnAttachmentChanged::RegistrationType& OnItemAttached()
	{
		return OnItemAttachedDelegate;
	}

	FOnAttachmentChanged::RegistrationType& OnItemDetached()
	{
		return OnItemDetachedDelegate;
	}

	/** Sets tree attachment root to scan from */
	void SetAttachmentRoot(AActor* InActor);

	/** Gets tree attachment root */
	AActor* GetAttachmentRoot() const;

	/** Set the behavior implementation to use */
	void SetBehaviorImplementation(const TSharedRef<ICEClonerAttachmentTreeBehavior>& InImplementation);

	/** Performs cleanup before destruction */
	void Cleanup();

	/** Marks the tree as outdated to perform an update */
	void MarkAttachmentOutdated();

	/** Marks the actor cache as outdated for an update */
	void MarkCacheOutdated(AActor* InActor);
	
	/**
	* Triggers an update of the attachment tree to detect updated items
	* If reset is true, clears the attachment tree and rebuilds it otherwise diff update
	*/
	bool UpdateAttachments(bool bInReset = false);

	/** Finds root actor of an actor based on this tree hierarchy by climbing up */
	AActor* FindRootActor(AActor* InActor);

	/** Finds root item of an actor based on this tree hierarchy by climbing up */
	FCEClonerAttachmentRootItem* FindRootItem(AActor* InActor);

	/** Retrieves all root actors from this tree hierarchy */
	TArray<AActor*> GetRootActors() const;

	/** Traverses each attachment in tree, stops when false is returned */
	void ForEachAttachment(const TFunctionRef<bool(AActor*, FCEClonerAttachmentItem&)>& InFunctor);

	/** Detach an item from the tree */
	void DetachItem(AActor* InActor);

	/** True if all root meshes are valid and usable */
	bool IsCacheAvailable(bool bInAllowInvalid = true) const;
	
	/** Retrieves all attachments based on an actor */
	void GetAttachments(AActor* InActor, TArray<const FCEClonerAttachmentItem*>& OutAttachmentItems, bool bInRecurse) const;

	/** Get the attachment bounds based on tree hierarchy */
	FBox GetAttachmentBounds(AActor* InActor, bool bInIncludeChildren) const;

#if WITH_EDITOR
	/** Get cached size of root mesh, -1,-1,-1 when invalid */
	TOptional<FVector> GetRootCachedMeshSize(AActor* InActor) const;
#endif

	/** All cloner attached actor items */
	UPROPERTY()
	TMap<TWeakObjectPtr<AActor>, FCEClonerAttachmentItem> ItemAttachmentMap;

	/** Root items directly attached to the cloner actor, order is important here */
	UPROPERTY()
	TArray<FCEClonerAttachmentRootItem> RootItemAttachments;

	/** Status of the cloner tree */
	UPROPERTY(Transient)
	ECEClonerAttachmentStatus Status = ECEClonerAttachmentStatus::Outdated;

	/** Attachment items that are dirty and need an update */
	UPROPERTY(Transient)
	TSet<TWeakObjectPtr<AActor>> DirtyItemAttachments;

	/** Whether Added/Moved/Removed items requires an update */
	UPROPERTY(Transient)
	bool bItemAttachmentsDirty = false;

private:
	bool UpdateAttachmentsInternal();
	void UpdateAttachment(AActor* InActor, AActor* InParent, const FTransform& InActorTransform);
	void DetachItem(FCEClonerAttachmentItem& InItem);

	/** Clears the whole tree */
	void Reset();

	/** Called when tree item is attached */
	FOnAttachmentChanged OnItemAttachedDelegate;

	/** Called when tree item is detached */
	FOnAttachmentChanged OnItemDetachedDelegate;

	UPROPERTY(Transient)
	TObjectPtr<AActor> AttachmentRoot;

	TSharedPtr<ICEClonerAttachmentTreeBehavior> BehaviorImplementation;
};
