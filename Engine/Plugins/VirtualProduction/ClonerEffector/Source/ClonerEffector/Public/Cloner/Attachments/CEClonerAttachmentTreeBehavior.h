// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Components/SceneComponent.h"
#include "Containers/Array.h"
#include "Delegates/IDelegateInstance.h"
#include "Templates/SharedPointer.h"

class AActor;
struct FCEClonerAttachmentItem;
struct FCEClonerAttachmentTree;

/** Defines a custom implementation for the attachment tree */
class ICEClonerAttachmentTreeBehavior : public TSharedFromThis<ICEClonerAttachmentTreeBehavior>
{
public:
	virtual ~ICEClonerAttachmentTreeBehavior() = default;

	/** Called when this implementation is activated */
	virtual void OnActivation(FCEClonerAttachmentTree& InTree) = 0;

	/** Called when this implementation is deactivated */
	virtual void OnDeactivation(FCEClonerAttachmentTree& InTree) = 0;

	/** Retrieve ordered list of children actors */
	virtual void GetOrderedChildrenActors(FCEClonerAttachmentTree& InTree, AActor* InActor, TArray<AActor*>& OutChildren) const = 0;

	/** Attachment mesh status is out of date and needs an update */
	virtual void InvalidateMesh(FCEClonerAttachmentTree& InTree, AActor* InActor) = 0;

	/** Update attachment mesh (materials) */
	virtual void UpdateMesh(FCEClonerAttachmentTree& InTree, AActor* InActor) = 0;

	/** Item is attached in the tree */
	virtual void OnItemAttached(FCEClonerAttachmentTree& InTree, FCEClonerAttachmentItem& InItem) = 0;

	/** Item is detached from the tree */
	virtual void OnItemDetached(FCEClonerAttachmentTree& InTree, FCEClonerAttachmentItem& InItem) = 0;
};

/** Base hierarchy where children are grouped under a root cloner actor */
class FCEClonerAttachmentGroupBehavior : public ICEClonerAttachmentTreeBehavior
{
protected:
	//~ Begin ICEClonerAttachmentImplementation
	virtual void OnActivation(FCEClonerAttachmentTree& InTree) override;
	virtual void OnDeactivation(FCEClonerAttachmentTree& InTree) override;
	virtual void GetOrderedChildrenActors(FCEClonerAttachmentTree& InTree, AActor* InActor, TArray<AActor*>& OutChildren) const override;
	virtual void InvalidateMesh(FCEClonerAttachmentTree& InTree, AActor* InActor) override;
	virtual void OnItemAttached(FCEClonerAttachmentTree& InTree, FCEClonerAttachmentItem& InItem) override;
	virtual void OnItemDetached(FCEClonerAttachmentTree& InTree, FCEClonerAttachmentItem& InItem) override;
	virtual void UpdateMesh(FCEClonerAttachmentTree& InTree, AActor* InActor) override;
	//~ End ICEClonerAttachmentImplementation

	void BindActorDelegates(FCEClonerAttachmentTree& InTree, AActor* InActor);
	void UnbindActorDelegates(FCEClonerAttachmentTree& InTree, AActor* InActor);

	/** Called when the transform state of a cloned component changes */
	void OnComponentTransformed(FCEClonerAttachmentTree& InTree, USceneComponent* InComponent, EUpdateTransformFlags InFlags, ETeleportType InTeleport);

	/** Called when an actor in the world is destroyed */
	void OnWorldActorDestroyed(FCEClonerAttachmentTree& InTree, AActor* InActor);

#if WITH_EDITOR
	/** Called when a mesh has changed in any cloned actors */
	void OnMeshChanged(FCEClonerAttachmentTree& InTree, AActor* InActor);

	/** Called when the render state of a cloned component changes */
	void OnRenderStateDirty(FCEClonerAttachmentTree& InTree, UActorComponent& InComponent);

	/** Called when a level actor hierarchy changes */
	void OnLevelHierarchyChanged(FCEClonerAttachmentTree& InTree, AActor* InActor);
#endif

	/** Called when material is done compiling and when property is changed to test whether material have changed */
	void OnCheckMaterialChanged(FCEClonerAttachmentTree& InTree, UObject* InObject);

	/** Called to update all materials of a root actor baked mesh */
	void RefreshBakedMeshRootMaterials(FCEClonerAttachmentTree& InTree, AActor* InActor);

	FDelegateHandle WorldActorDestroyedDelegate;
};

/** Flat hierarchy where any grouped actor is unboxed as if it was under a root cloner actor */
class FCEClonerAttachmentFlatBehavior : public FCEClonerAttachmentGroupBehavior
{
	//~ Begin ICEClonerAttachmentImplementation
	virtual void GetOrderedChildrenActors(FCEClonerAttachmentTree& InTree, AActor* InActor, TArray<AActor*>& OutChildren) const override;
	//~ End ICEClonerAttachmentImplementation
};
