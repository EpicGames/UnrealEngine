// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CEClonerAttachmentTypes.generated.h"

UENUM()
enum class ECEClonerAttachmentStatus : uint8
{
	/** Item should be removed, no longer valid */
	Invalid,
	/** Item should be updated, changes detected */
	Outdated,
	/** Item is up to date, no changes needed */
	Updated,
	/** Item is being updated at the moment */
	Updating
};