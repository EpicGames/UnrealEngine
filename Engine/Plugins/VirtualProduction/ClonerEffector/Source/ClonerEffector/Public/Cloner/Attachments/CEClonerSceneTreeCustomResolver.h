// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "Delegates/IDelegateInstance.h"
#include "GameFramework/Actor.h"
#include "HAL/Platform.h"
#include "Templates/SharedPointer.h"

class ULevel;

/** Used to resolve underlying actors in the scene */
class ICEClonerSceneTreeCustomResolver : public TSharedFromThis<ICEClonerSceneTreeCustomResolver>
{
public:
	DECLARE_MULTICAST_DELEGATE_OneParam(FOnActorHierarchyChanged, AActor* /** InActor */)
	virtual ~ICEClonerSceneTreeCustomResolver() = default;
	virtual void Activate() = 0;
	virtual void Deactivate() = 0;
	virtual bool GetDirectChildrenActor(AActor* InActor, TArray<AActor*>& OutActors) const = 0;
	virtual FOnActorHierarchyChanged::RegistrationType& OnActorHierarchyChanged() = 0;
};