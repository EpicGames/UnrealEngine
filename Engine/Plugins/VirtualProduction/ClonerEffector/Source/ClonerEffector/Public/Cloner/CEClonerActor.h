// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CEClonerEffectorShared.h"
#include "Curves/RichCurve.h"
#include "GameFramework/Actor.h"
#include "Layouts/CEClonerLayoutBase.h"
#include "NiagaraMeshRendererProperties.h"
#include "CEClonerActor.generated.h"

class AActor;
class ACEEffectorActor;
class UCEClonerComponent;
class UCEClonerLayoutBase;
class UNiagaraDataInterfaceCurve;
class USceneComponent;

UCLASS(MinimalAPI
	, BlueprintType
	, AutoExpandCategories=(ClonerComponent)
	, HideCategories=(Rendering,Replication,Collision,HLOD,Physics,Networking,Input,Actor,Cooking,LevelInstance,Streaming,DataLayers,WorldPartition)
	, DisplayName = "Motion Design Cloner Actor")
class ACEClonerActor : public AActor
{
	GENERATED_BODY()

public:
	static inline const FString DefaultLabel = TEXT("Cloner");

	ACEClonerActor();

	UFUNCTION(BlueprintPure, Category="Cloner")
	UCEClonerComponent* GetClonerComponent() const
	{
		return ClonerComponent;
	}

#if WITH_EDITOR
	/** Sets whether to spawn and attach a default actor on creation. Only available in editor engine */
	CLONEREFFECTOR_API void SpawnDefaultActorAttached();
#endif

protected:
	//~ Begin UObject
	virtual void Serialize(FArchive& InArchive) override;
	virtual void PostLoad() override;
	//~ End UObject

	//~ Begin AActor
#if WITH_EDITOR
	virtual FString GetDefaultActorLabel() const override;
#endif
	//~ End AActor

private:
	void MigrateDeprecatedProperties();

#if WITH_EDITOR
	/** Hide outline selection when this cloner only is selected in viewport */
	void OnEditorSelectionChanged(UObject* InSelection);

	TOptional<bool> UseSelectionOutline;
#endif

	int32 MigrateToVersion = INDEX_NONE;

	UPROPERTY(VisibleInstanceOnly, Category="Cloner")
	TObjectPtr<UCEClonerComponent> ClonerComponent;

	UE_DEPRECATED(5.5, "Moved to component and extensions")
	UPROPERTY()
	bool bEnabled = true;

	UE_DEPRECATED(5.5, "Moved to component and extensions")
	UPROPERTY()
	float TreeUpdateInterval = 0.2f;

	UE_DEPRECATED(5.5, "Moved to component and extensions")
	UPROPERTY()
	int32 Seed = 0;

	UE_DEPRECATED(5.5, "Moved to component and extensions")
	UPROPERTY()
	FLinearColor Color = FLinearColor::White;

	UE_DEPRECATED(5.5, "Moved to component and extensions")
	UPROPERTY()
	bool bVisualizeEffectors;

	UE_DEPRECATED(5.5, "Moved to component and extensions")
	UPROPERTY()
	ECEClonerMeshRenderMode MeshRenderMode = ECEClonerMeshRenderMode::Iterate;

	UE_DEPRECATED(5.5, "Moved to component and extensions")
	UPROPERTY()
	ENiagaraMeshFacingMode MeshFacingMode = ENiagaraMeshFacingMode::Default;

	UE_DEPRECATED(5.5, "Moved to component and extensions")
	UPROPERTY()
	bool bMeshCastShadows = true;

	UE_DEPRECATED(5.5, "Moved to component and extensions")
	UPROPERTY()
	TArray<TObjectPtr<UStaticMesh>> DefaultMeshes;

	UE_DEPRECATED(5.5, "Moved to component and extensions")
	UPROPERTY()
	bool bUseOverrideMaterial = false;

	UE_DEPRECATED(5.5, "Moved to component and extensions")
	UPROPERTY()
	TObjectPtr<UMaterialInterface> OverrideMaterial;

	UE_DEPRECATED(5.5, "Moved to component and extensions")
	UPROPERTY()
	bool bSurfaceCollisionEnabled = false;

	UE_DEPRECATED(5.5, "Moved to component and extensions")
	UPROPERTY()
	bool bParticleCollisionEnabled = false;

	UE_DEPRECATED(5.5, "Moved to component and extensions")
	UPROPERTY()
	bool bCollisionVelocityEnabled = false;

	UE_DEPRECATED(5.5, "Moved to component and extensions")
	UPROPERTY()
	int32 CollisionIterations = 1;

	UE_DEPRECATED(5.5, "Moved to component and extensions")
	UPROPERTY()
	int32 CollisionGridResolution = 32;

	UE_DEPRECATED(5.5, "Moved to component and extensions")
	UPROPERTY()
	FVector CollisionGridSize = FVector(5000.f);

	UE_DEPRECATED(5.5, "Moved to component and extensions")
	UPROPERTY()
	ECEClonerCollisionRadiusMode CollisionRadiusMode = ECEClonerCollisionRadiusMode::ExtentLength;

	UE_DEPRECATED(5.5, "Moved to component and extensions")
	UPROPERTY()
	TArray<float> CollisionRadii;

	UE_DEPRECATED(5.5, "Moved to component and extensions")
	UPROPERTY()
	float MassMin = 1.f;

	UE_DEPRECATED(5.5, "Moved to component and extensions")
	UPROPERTY()
	float MassMax = 2.f;

	UE_DEPRECATED(5.5, "Moved to component and extensions")
	UPROPERTY()
	ECEClonerSpawnLoopMode SpawnLoopMode = ECEClonerSpawnLoopMode::Once;

	UE_DEPRECATED(5.5, "Moved to component and extensions")
	UPROPERTY()
	int32 SpawnLoopIterations = 1;

	UE_DEPRECATED(5.5, "Moved to component and extensions")
	UPROPERTY()
	float SpawnLoopInterval = 1.f;

	UE_DEPRECATED(5.5, "Moved to component and extensions")
	UPROPERTY()
	ECEClonerSpawnBehaviorMode SpawnBehaviorMode = ECEClonerSpawnBehaviorMode::Instant;

	UE_DEPRECATED(5.5, "Moved to component and extensions")
	UPROPERTY()
	float SpawnRate = 1.f;

	UE_DEPRECATED(5.5, "Moved to component and extensions")
	UPROPERTY()
	bool bLifetimeEnabled = false;

	UE_DEPRECATED(5.5, "Moved to component and extensions")
	UPROPERTY()
	float LifetimeMin = 0.25f;

	UE_DEPRECATED(5.5, "Moved to component and extensions")
	UPROPERTY()
	float LifetimeMax = 1.5f;

	UE_DEPRECATED(5.5, "Moved to component and extensions")
	UPROPERTY()
	bool bLifetimeScaleEnabled = false;

	UE_DEPRECATED(5.5, "Moved to component and extensions")
	UPROPERTY()
	FRichCurve LifetimeScaleCurve;

	UE_DEPRECATED(5.5, "Moved to component and extensions")
	UPROPERTY()
	FName LayoutName = NAME_None;

	UE_DEPRECATED(5.5, "Moved to component and extensions")
	UPROPERTY()
	bool bDeltaStepEnabled = false;

	UE_DEPRECATED(5.5, "Moved to component and extensions")
	UPROPERTY()
	FRotator DeltaStepRotation = FRotator(0.f);

	UE_DEPRECATED(5.5, "Moved to component and extensions")
	UPROPERTY()
	FVector DeltaStepScale = FVector(0.f);

	UE_DEPRECATED(5.5, "Moved to component and extensions")
	UPROPERTY()
	bool bRangeEnabled = false;

	UE_DEPRECATED(5.5, "Moved to component and extensions")
	UPROPERTY()
	FVector RangeOffsetMin = FVector::ZeroVector;

	UE_DEPRECATED(5.5, "Moved to component and extensions")
	UPROPERTY()
	FVector RangeOffsetMax = FVector::ZeroVector;

	UE_DEPRECATED(5.5, "Moved to component and extensions")
	UPROPERTY()
	FRotator RangeRotationMin = FRotator::ZeroRotator;

	UE_DEPRECATED(5.5, "Moved to component and extensions")
	UPROPERTY()
	FRotator RangeRotationMax = FRotator::ZeroRotator;

	UE_DEPRECATED(5.5, "Moved to component and extensions")
	UPROPERTY()
	bool bRangeScaleUniform = false;

	UE_DEPRECATED(5.5, "Moved to component and extensions")
	UPROPERTY()
	FVector RangeScaleMin = FVector::OneVector;

	UE_DEPRECATED(5.5, "Moved to component and extensions")
	UPROPERTY()
	FVector RangeScaleMax = FVector::OneVector;

	UE_DEPRECATED(5.5, "Moved to component and extensions")
	UPROPERTY()
	float RangeScaleUniformMin = 1.f;

	UE_DEPRECATED(5.5, "Moved to component and extensions")
	UPROPERTY()
	float RangeScaleUniformMax = 1.f;

	UE_DEPRECATED(5.5, "Moved to component and extensions")
	UPROPERTY()
	bool bInvertProgress = false;

	UE_DEPRECATED(5.5, "Moved to component and extensions")
	UPROPERTY()
	float Progress = 1.f;

	UE_DEPRECATED(5.5, "Moved to component and extensions")
	UPROPERTY()
	TArray<TWeakObjectPtr<ACEEffectorActor>> EffectorsWeak;

	UE_DEPRECATED(5.5, "Moved to component and extensions")
	UPROPERTY()
	TMap<FName, TObjectPtr<UCEClonerLayoutBase>> LayoutInstances;

#if WITH_EDITORONLY_DATA
	UE_DEPRECATED(5.5, "Moved to component and extensions")
	UPROPERTY()
	bool bVisualizerSpriteVisible = true;
#endif
};
