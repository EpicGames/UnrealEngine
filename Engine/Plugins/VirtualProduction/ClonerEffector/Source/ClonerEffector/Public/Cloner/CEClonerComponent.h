// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Attachments/CEClonerAttachmentTree.h"
#include "CEClonerEffectorShared.h"
#include "CEMeshBuilder.h"
#include "CEPropertyChangeDispatcher.h"
#include "Containers/Ticker.h"
#include "Layouts/CEClonerLayoutBase.h"
#include "NiagaraComponent.h"
#include "CEClonerComponent.generated.h"

class UCEClonerExtensionBase;
class UCEClonerLayoutBase;
class ULevel;
class UMaterialInterface;
struct FNiagaraMeshMaterialOverride;

UCLASS(MinimalAPI
	, BlueprintType
	, DisplayName = "Motion Design Cloner Component"
	, AutoExpandCategories=(Cloner, Layout)
	, HideCategories=(Transform, Niagara, Activation, Attachment, Randomness, Parameters)
	, meta = (BlueprintSpawnableComponent))
class UCEClonerComponent : public UNiagaraComponent
{
	GENERATED_BODY()

	friend class ACEClonerActor;

	DECLARE_MULTICAST_DELEGATE_OneParam(FOnClonerMeshUpdated, UCEClonerComponent* /** ClonerComponent */)
	DECLARE_MULTICAST_DELEGATE_TwoParams(FOnClonerLayoutLoaded, UCEClonerComponent* /** ClonerComponent */, UCEClonerLayoutBase* /** InLayout */)
	DECLARE_MULTICAST_DELEGATE_OneParam(FOnClonerInitialized, UCEClonerComponent* /** ClonerComponent */)
	DECLARE_MULTICAST_DELEGATE_TwoParams(FOnClonerAttachmentChanged, UCEClonerComponent* /** ClonerComponent */, AActor* /** InActor */)

public:
#if WITH_EDITOR
	static CLONEREFFECTOR_API FName GetActiveExtensionsPropertyName();

	static CLONEREFFECTOR_API FName GetActiveLayoutPropertyName();

	static CLONEREFFECTOR_API FName GetLayoutNamePropertyName();
#endif

	static FOnClonerMeshUpdated::RegistrationType& OnClonerMeshUpdated()
	{
		return OnClonerMeshUpdatedDelegate;
	}

	static FOnClonerLayoutLoaded::RegistrationType& OnClonerLayoutLoaded()
	{
		return OnClonerLayoutLoadedDelegate;
	}

	static FOnClonerInitialized::RegistrationType& OnClonerInitialized()
	{
		return OnClonerInitializedDelegate;
	}

	static FOnClonerAttachmentChanged::RegistrationType& OnClonerActorAttached()
	{
		return OnClonerActorAttachedDelegate;
	}

	static FOnClonerAttachmentChanged::RegistrationType& OnClonerActorDetached()
	{
		return OnClonerActorDetachedDelegate;
	}

	UCEClonerComponent();

	UFUNCTION(BlueprintCallable, Category="Cloner")
	CLONEREFFECTOR_API void SetEnabled(bool bInEnable);

	UFUNCTION(BlueprintPure, Category="Cloner")
	bool GetEnabled() const
	{
		return bEnabled;
	}

	UFUNCTION(BlueprintCallable, Category="Cloner")
	CLONEREFFECTOR_API void SetSeed(int32 InSeed);

	UFUNCTION(BlueprintPure, Category="Cloner")
	int32 GetSeed() const
	{
		return Seed;
	}

	UFUNCTION(BlueprintCallable, Category="Cloner")
	CLONEREFFECTOR_API void SetColor(const FLinearColor& InColor);

	UFUNCTION(BlueprintPure, Category="Cloner")
	const FLinearColor& GetColor() const
	{
		return Color;
	}

	UFUNCTION(BlueprintCallable, Category="Cloner")
	CLONEREFFECTOR_API void SetGlobalScale(const FVector& InScale);

	UFUNCTION(BlueprintPure, Category="Cloner")
	const FVector& GetGlobalScale() const
	{
		return GlobalScale;
	}

	UFUNCTION(BlueprintCallable, Category="Cloner")
	CLONEREFFECTOR_API void SetGlobalRotation(const FRotator& InRotation);

	UFUNCTION(BlueprintPure, Category="Cloner")
	const FRotator& GetGlobalRotation() const
	{
		return GlobalRotation;
	}

	UFUNCTION(BlueprintCallable, Category="Cloner")
	CLONEREFFECTOR_API void SetLayoutName(FName InLayoutName);

	UFUNCTION(BlueprintPure, Category="Cloner")
	FName GetLayoutName() const
	{
		return LayoutName;
	}

	UFUNCTION(BlueprintCallable, Category="Cloner")
	void SetLayoutClass(TSubclassOf<UCEClonerLayoutBase> InLayoutClass);

	UFUNCTION(BlueprintPure, Category="Cloner")
	TSubclassOf<UCEClonerLayoutBase> GetLayoutClass() const;

	UFUNCTION(BlueprintPure, Category="Cloner")
	UCEClonerLayoutBase* GetActiveLayout() const
	{
		return ActiveLayout;
	}

	template<
		typename InLayoutClass
		UE_REQUIRES(TIsDerivedFrom<InLayoutClass, UCEClonerLayoutBase>::Value)>
	bool IsActiveLayout() const
	{
		if (const UCEClonerLayoutBase* CurrentLayout = GetActiveLayout())
		{
			return CurrentLayout->GetClass() == InLayoutClass::StaticClass();
		}

		return false;
	}

	template<
		typename InLayoutClass
		UE_REQUIRES(TIsDerivedFrom<InLayoutClass, UCEClonerLayoutBase>::Value)>
	InLayoutClass* GetActiveLayout() const
	{
		return Cast<InLayoutClass>(GetActiveLayout());
	}

#if WITH_EDITOR
	UFUNCTION(BlueprintCallable, Category="Cloner")
	void SetTreeBehaviorName(FName InBehaviorName);

	UFUNCTION(BlueprintPure, Category="Cloner")
	FName GetTreeBehaviorName() const
	{
		return TreeBehaviorName;
	}

	UFUNCTION(BlueprintCallable, Category="Cloner")
	CLONEREFFECTOR_API void SetVisualizerSpriteVisible(bool bInVisible);

	UFUNCTION(BlueprintPure, Category="Cloner")
	bool GetVisualizerSpriteVisible() const
	{
		return bVisualizerSpriteVisible;
	}
#endif

	/** Returns the number of meshes this cloner currently handles */
	UFUNCTION(BlueprintPure, Category="Cloner")
	CLONEREFFECTOR_API int32 GetMeshCount() const;

	/** Returns the number of root attachment currently on this cloner */
	UFUNCTION(BlueprintPure, Category="Cloner")
	int32 GetAttachmentCount() const;

#if WITH_EDITOR
	/** This will force an update of the cloner attachment tree */
	UFUNCTION(CallInEditor, Category="Utilities")
	void ForceUpdateCloner();

	/** Open project settings for cloner */
	UFUNCTION(CallInEditor, Category="Utilities")
	void OpenClonerSettings();

	/** This will create a new default actor attached to this cloner if nothing is attached to this cloner */
	UFUNCTION(CallInEditor, Category="Utilities")
	void CreateDefaultActorAttached();

	/** Converts the cloner simulation into a single static mesh, this is a heavy operation */
	UFUNCTION(CallInEditor, Category="Utilities")
	void ConvertToStaticMesh();

	/** Converts the cloner simulation into a single dynamic mesh, this is a heavy operation */
	UFUNCTION(CallInEditor, Category="Utilities")
	void ConvertToDynamicMesh();

	/** Converts the cloner simulation into static meshes, this is a heavy operation */
	UFUNCTION(CallInEditor, Category="Utilities")
	void ConvertToStaticMeshes();

	/** Converts the cloner simulation into dynamic meshes, this is a heavy operation */
	UFUNCTION(CallInEditor, Category="Utilities")
	void ConvertToDynamicMeshes();

	/** Converts the cloner simulation into instanced static meshes, this is a heavy operation */
	UFUNCTION(CallInEditor, Category="Utilities")
	void ConvertToInstancedStaticMeshes();
#endif

	/** Will force a system update to refresh user parameters */
	void RequestClonerUpdate(bool bInImmediate = false);

	/** Forces a refresh of the meshes used */
	void RefreshClonerMeshes();

	template<typename InExtensionClass UE_REQUIRES(std::is_base_of_v<UCEClonerExtensionBase, InExtensionClass>)>
	InExtensionClass* GetExtension() const
	{
		return Cast<InExtensionClass>(GetExtension(InExtensionClass::StaticClass()));
	}

	UFUNCTION(BlueprintCallable, Category="Cloner", meta=(DeterminesOutputType="InExtensionClass"))
	CLONEREFFECTOR_API UCEClonerExtensionBase* GetExtension(TSubclassOf<UCEClonerExtensionBase> InExtensionClass) const;

	CLONEREFFECTOR_API UCEClonerExtensionBase* GetExtension(FName InExtensionName) const;

	TConstArrayView<TObjectPtr<UCEClonerExtensionBase>> GetActiveExtensions() const
	{
		return ActiveExtensions;
	}

	/**
	 * Retrieves all active extensions on this cloner
	 * @param OutExtensions [Out] Active extensions
	 */
	UFUNCTION(BlueprintCallable, Category="Cloner")
	void GetActiveExtensions(TArray<UCEClonerExtensionBase*>& OutExtensions) const
	{
		OutExtensions = ActiveExtensions;
	}

	/** Retrieve root actors in their respective order */
	TArray<AActor*> GetClonerRootActors() const;

protected:
	/** Called when meshes have been updated */
	CLONEREFFECTOR_API static FOnClonerMeshUpdated OnClonerMeshUpdatedDelegate;

	/** Called when new cloner layout is loaded */
	CLONEREFFECTOR_API static FOnClonerLayoutLoaded OnClonerLayoutLoadedDelegate;

	/** Called when cloner is initialized */
	CLONEREFFECTOR_API static FOnClonerInitialized OnClonerInitializedDelegate;

	/** Called when cloner actor is attached */
	CLONEREFFECTOR_API static FOnClonerAttachmentChanged OnClonerActorAttachedDelegate;

	/** Called when cloner actor is detached */
	CLONEREFFECTOR_API static FOnClonerAttachmentChanged OnClonerActorDetachedDelegate;

	//~ Begin UObject
	virtual void GetPreloadDependencies(TArray<UObject*>& OutDeps) override;
	virtual void PostInitProperties() override;
	virtual void PostLoad() override;
#if WITH_EDITOR
	virtual void PreSave(FObjectPreSaveContext InSaveContext) override;
	virtual void PreDuplicate(FObjectDuplicationParameters& InParams) override;
	virtual void PostDuplicate(bool bInPIE) override;
	virtual void PostEditImport() override;
	virtual void PostEditUndo() override;
	virtual void PostEditChangeProperty(FPropertyChangedEvent& InPropertyChangedEvent) override;
#endif
	//~ End UObject

	//~ Begin UActorComponent
	virtual void OnComponentCreated() override;
	virtual void OnComponentDestroyed(bool bInDestroyingHierarchy) override;
	//~ End UActorComponent

#if WITH_EDITOR
	/** Runs async update to rebuild dirty meshes */
	void UpdateDirtyMeshesAsync();

	/** Merges all primitive components from a single actor into a dynamic mesh */
	void UpdateActorDynamicMesh(AActor* InActor);

	/** Merges all actors dynamic meshes into a single static mesh */
	void UpdateActorStaticMesh(AActor* InRootActor);

	/** Called once static mesh is done building */
	void OnActorStaticMeshPostBuild(UStaticMesh* InMesh);
#endif

	void OnDirtyMeshesUpdated(bool bInSuccess);

	/** Update niagara asset static meshes */
	void UpdateClonerMeshes();

	/** Sets the layout to use for this cloner simulation */
	void SetClonerActiveLayout(UCEClonerLayoutBase* InLayout);

	void OnActiveLayoutLoaded(UCEClonerLayoutBase* InLayout, bool bInSuccess);

	void ActivateLayout(UCEClonerLayoutBase* InLayout);

	void OnActiveLayoutChanged();

	/** Is this cloner enabled/disabled */
	UPROPERTY(EditInstanceOnly, Setter="SetEnabled", Getter="GetEnabled", Category=General)
	bool bEnabled = true;

	/** Cloner instance seed for random deterministic patterns */
	UPROPERTY(EditInstanceOnly, Setter, Getter, Category=General)
	int32 Seed = 0;

	/** Cloner color when unaffected by effectors, color will be passed down to the material (ParticleColor) */
	UPROPERTY(EditInstanceOnly, Setter, Getter, Category=General)
	FLinearColor Color = FLinearColor::White;

	/** Global scale applied on all cloned items */
	UPROPERTY(EditInstanceOnly, Setter, Getter, Category=General, meta=(ClampMin="0", Delta="0.01", MotionDesignVectorWidget, AllowPreserveRatio="XYZ"))
	FVector GlobalScale = FVector::OneVector;

	/** Global rotation applied on all cloned items */
	UPROPERTY(EditInstanceOnly, Setter, Getter, Category=General)
	FRotator GlobalRotation = FRotator::ZeroRotator;

	/** Set the tree behavior with attachments */
	UPROPERTY(EditInstanceOnly, DisplayName="Tree Mode", Category=General, meta=(GetOptions="GetClonerTreeBehaviorNames"))
	FName TreeBehaviorName = NAME_None;

#if WITH_EDITORONLY_DATA
	/** Toggle the sprite to visualize and click on this cloner */
	UPROPERTY(EditInstanceOnly, AdvancedDisplay, Category=General, meta=(AllowPrivateAccess="true"))
	bool bVisualizerSpriteVisible = true;
#endif

	/** Name of the layout to use */
	UPROPERTY(EditInstanceOnly, Setter, Getter, DisplayName="Layout", Category="Layout", meta=(GetOptions="GetClonerLayoutNames"))
	FName LayoutName = NAME_None;

	/** Active layout used */
	UPROPERTY(Transient, DuplicateTransient, NonTransactional)
	TObjectPtr<UCEClonerLayoutBase> ActiveLayout;

	/** Active Extensions on this layout */
	UPROPERTY(Transient, DuplicateTransient, NonTransactional)
	TArray<TObjectPtr<UCEClonerExtensionBase>> ActiveExtensions;

	/** Layout instances cached */
	UPROPERTY()
	TArray<TObjectPtr<UCEClonerLayoutBase>> LayoutInstances;

	/** Layout extensions instances cached */
	UPROPERTY()
	TArray<TObjectPtr<UCEClonerExtensionBase>> ExtensionInstances;

private:
	static constexpr TCHAR SpriteTexturePath[] = TEXT("/Script/Engine.Texture2D'/ClonerEffector/Textures/T_ClonerIcon.T_ClonerIcon'");

	/** Initiate and perform operation */
	void InitializeCloner();
	void OnLevelLoaded(ULevel* InLevel, UWorld* InWorld);
	void PostInitializeCloner();
	void RegisterTicker();
	bool CheckResourcesReady();
	bool TickCloner(float InDelta);

	/** Called to trigger an update of cloner rendering state tree */
	void UpdateClonerRenderState();

	void OnEnabledChanged();
	void OnClonerEnabled();
	void OnClonerDisabled();
	void OnClonerSetEnabled(const UWorld* InWorld, bool bInEnabled, bool bInTransact);

	void OnSeedChanged();
	void OnColorChanged();
	void OnGlobalScaleChanged();
	void OnGlobalRotationChanged();
	void OnTreeBehaviorNameChanged();
	void OnLayoutNameChanged();

#if WITH_EDITOR
	void OnVisualizerSpriteVisibleChanged();
#endif

	void OnTreeItemAttached(AActor* InActor, FCEClonerAttachmentItem& InItem);
	void OnTreeItemDetached(AActor* InActor, FCEClonerAttachmentItem& InItem);

	template<
		typename InLayoutClass
		UE_REQUIRES(TIsDerivedFrom<InLayoutClass, UCEClonerLayoutBase>::Value)>
	InLayoutClass* FindOrAddLayout()
	{
		return Cast<InLayoutClass>(FindOrAddLayout(InLayoutClass::StaticClass()));
	}

	/** Find or add a layout by its class */
	UCEClonerLayoutBase* FindOrAddLayout(TSubclassOf<UCEClonerLayoutBase> InClass);

	/** Find or add a layout by its name */
	UCEClonerLayoutBase* FindOrAddLayout(FName InLayoutName);

	template<
		typename InExtensionClass
		UE_REQUIRES(TIsDerivedFrom<InExtensionClass, UCEClonerExtensionBase>::Value)>
	InExtensionClass* FindOrAddExtension()
	{
		return Cast<InExtensionClass>(FindOrAddExtension(InExtensionClass::StaticClass()));
	}

	/** Find or add an extension by its class */
	UCEClonerExtensionBase* FindOrAddExtension(TSubclassOf<UCEClonerExtensionBase> InClass);

	/** Find or add an extension by its name */
	UCEClonerExtensionBase* FindOrAddExtension(FName InExtensionName);

	/** Gets all layout names available */
	UFUNCTION()
	TArray<FName> GetClonerLayoutNames() const;

	/** Gets all tree implementation names available */
	UFUNCTION()
	TArray<FName> GetClonerTreeBehaviorNames() const;

	/** Attachment tree view */
	UPROPERTY(NonTransactional)
	FCEClonerAttachmentTree ClonerTree;

	/** Mesh builder to create final meshes that are sent to niagara */
	UPROPERTY(Transient, DuplicateTransient, NonTransactional)
	FCEMeshBuilder MeshBuilder;

	/** State of the baked dynamic and static mesh creation */
	std::atomic<bool> bClonerMeshesUpdating = false;

	bool bNeedsRefresh = false;

	bool bClonerInitialized = false;

	bool bClonerResourcesReady = false;

	FTSTicker::FDelegateHandle ClonerTickerHandle;
	FTSTicker::FDelegateHandle ConversionTickerHandle;

#if WITH_EDITOR
	/** Used for PECP */
	static const TCEPropertyChangeDispatcher<UCEClonerComponent> PropertyChangeDispatcher;
#endif
};