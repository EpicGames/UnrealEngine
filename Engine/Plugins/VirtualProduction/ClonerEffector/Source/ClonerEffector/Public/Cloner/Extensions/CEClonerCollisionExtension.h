// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CEClonerEffectorShared.h"
#include "CEClonerExtensionBase.h"
#include "CEPropertyChangeDispatcher.h"
#include "CEClonerCollisionExtension.generated.h"

class UMaterialInstanceDynamic;
class UStaticMeshComponent;

/** Extension dealing with collisions and physics related options */
UCLASS(MinimalAPI, BlueprintType, Within=CEClonerComponent, AutoExpandCategories=("Collisions"), meta=(Section="Physics", Priority=90))
class UCEClonerCollisionExtension : public UCEClonerExtensionBase
{
	GENERATED_BODY()

public:
	UCEClonerCollisionExtension();

	UFUNCTION(BlueprintCallable, Category="Cloner")
	CLONEREFFECTOR_API void SetSurfaceCollisionEnabled(bool bInSurfaceCollisionEnabled);

	UFUNCTION(BlueprintPure, Category="Cloner")
	bool GetSurfaceCollisionEnabled() const
	{
		return bSurfaceCollisionEnabled;
	}

	UFUNCTION(BlueprintCallable, Category="Cloner")
	CLONEREFFECTOR_API void SetParticleCollisionEnabled(bool bInParticleCollisionEnabled);

	UFUNCTION(BlueprintPure, Category="Cloner")
	bool GetParticleCollisionEnabled() const
	{
		return bParticleCollisionEnabled;
	}

	UFUNCTION(BlueprintCallable, Category="Cloner")
	CLONEREFFECTOR_API void SetCollisionVelocityEnabled(bool bInCollisionVelocityEnabled);

	UFUNCTION(BlueprintPure, Category="Cloner")
	bool GetCollisionVelocityEnabled() const
	{
		return bCollisionVelocityEnabled;
	}

	UFUNCTION(BlueprintCallable, Category="Cloner")
	CLONEREFFECTOR_API void SetCollisionIterations(int32 InCollisionIterations);

	UFUNCTION(BlueprintPure, Category="Cloner")
	int32 GetCollisionIterations() const
	{
		return CollisionIterations;
	}

	UFUNCTION(BlueprintCallable, Category="Cloner")
	CLONEREFFECTOR_API void SetCollisionGridResolution(int32 InCollisionGridResolution);

	UFUNCTION(BlueprintPure, Category="Cloner")
	int32 GetCollisionGridResolution() const
	{
		return CollisionGridResolution;
	}

	UFUNCTION(BlueprintCallable, Category="Cloner")
	CLONEREFFECTOR_API void SetCollisionGridSize(const FVector& InCollisionGridSize);

	UFUNCTION(BlueprintPure, Category="Cloner")
	FVector GetCollisionGridSize() const
	{
		return CollisionGridSize;
	}

	UFUNCTION(BlueprintCallable, Category="Cloner")
	CLONEREFFECTOR_API void SetCollisionRadiusMode(ECEClonerCollisionRadiusMode InMode);

	UFUNCTION(BlueprintPure, Category="Cloner")
	ECEClonerCollisionRadiusMode GetCollisionRadiusMode() const
	{
		return CollisionRadiusMode;
	}

	UFUNCTION(BlueprintPure, Category="Cloner")
	const TArray<float>& GetCollisionRadii() const
	{
		return CollisionRadii;
	}

	UFUNCTION(BlueprintCallable, Category="Cloner")
	CLONEREFFECTOR_API void SetMassMin(float InMassMin);

	UFUNCTION(BlueprintPure, Category="Cloner")
	float GetMassMin() const
	{
		return MassMin;
	}

	UFUNCTION(BlueprintCallable, Category="Cloner")
	CLONEREFFECTOR_API void SetMassMax(float InMassMax);

	UFUNCTION(BlueprintPure, Category="Cloner")
	float GetMassMax() const
	{
		return MassMax;
	}

protected:
	//~ Begin UObject
#if WITH_EDITOR
	virtual void PostEditChangeProperty(FPropertyChangedEvent& InPropertyChangedEvent) override;
#endif
	//~ End UObject

	//~ Begin UCEClonerExtensionBase
	virtual void OnExtensionDeactivated() override;
	virtual void OnExtensionParametersChanged(UCEClonerComponent* InComponent) override;
	virtual void OnClonerMeshesUpdated() override;
	//~ End UCEClonerExtensionBase

#if WITH_EDITOR
	bool ShouldInitVisualizerComponent(TNotNull<const UCEClonerComponent*> InComponent) const;
	bool InitVisualizerComponent();
	void DestroyVisualizerComponent();
#endif

	/**
	 * Allow particles to react to surface by using the distance field,
	 * ensure the mesh you want particle to collide with is tick enough or increase its distance field resolution scale in static mesh editor 
	 */
	UPROPERTY(EditInstanceOnly, Setter="SetSurfaceCollisionEnabled", Getter="GetSurfaceCollisionEnabled", Category="Collisions", meta=(RefreshPropertyView))
	bool bSurfaceCollisionEnabled = false;

	/** Allow particles to react to other emitter particles, uses a neighbor grid to detect collision */
	UPROPERTY(EditInstanceOnly, Setter="SetParticleCollisionEnabled", Getter="GetParticleCollisionEnabled", Category="Collisions", meta=(RefreshPropertyView))
	bool bParticleCollisionEnabled = false;

	/** Recalculate accurate velocity after collision is updated */
	UPROPERTY(EditInstanceOnly, Setter="SetCollisionVelocityEnabled", Getter="GetCollisionVelocityEnabled", Category="Collisions", meta=(EditCondition="bParticleCollisionEnabled", EditConditionHides))
	bool bCollisionVelocityEnabled = false;

	/** Amount of iterations to improve particle collision results but affects performance */
	UPROPERTY(EditInstanceOnly, Setter, Getter, Category="Collisions", meta=(ClampMin="1", EditCondition="bParticleCollisionEnabled", EditConditionHides))
	int32 CollisionIterations = 1;

	/** Resolution of the neighbor grid to detect collision between particles */
	UPROPERTY(EditInstanceOnly, Setter, Getter, Category="Collisions", meta=(ClampMin="1", EditCondition="bParticleCollisionEnabled", EditConditionHides))
	int32 CollisionGridResolution = 32;

	/** Size of the neighbor grid to detect collision between particles, particles outside this grid will not have collisions */
	UPROPERTY(EditInstanceOnly, Setter, Getter, Category="Collisions", meta=(ClampMin="0", MotionDesignVectorWidget, AllowPreserveRatio="XYZ", EditCondition="bParticleCollisionEnabled", EditConditionHides))
	FVector CollisionGridSize = FVector(5000.f);

	/** Collision radius calculation mode */
	UPROPERTY(EditInstanceOnly, Setter, Getter, Category="Collisions", meta=(EditCondition="bSurfaceCollisionEnabled || bParticleCollisionEnabled", EditConditionHides, RefreshPropertyView))
	ECEClonerCollisionRadiusMode CollisionRadiusMode = ECEClonerCollisionRadiusMode::ExtentLength;

	/** Radius expected around each particle for collision, order matches attachment index */
	UPROPERTY(EditInstanceOnly, Getter, Category="Collisions", EditFixedSize, meta=(ClampMin="0", EditFixedOrder, EditCondition="(bSurfaceCollisionEnabled || bParticleCollisionEnabled) && CollisionRadiusMode == ECEClonerCollisionRadiusMode::Manual", EditConditionHides))
	TArray<float> CollisionRadii;

	/** Minimum particle mass, used for collisions to push apart */
	UPROPERTY(EditInstanceOnly, Setter, Getter, Category="Collisions", meta=(ClampMin="1"))
	float MassMin = 1.f;

	/** Maximum particle mass, used for collisions to push apart */
	UPROPERTY(EditInstanceOnly, Setter, Getter, Category="Collisions", meta=(ClampMin="1"))
	float MassMax = 2.f;

#if WITH_EDITORONLY_DATA
	/** Preview 3D neighbor grid used for collision */
	UPROPERTY(EditInstanceOnly, Category="Collisions", meta=(EditCondition="bParticleCollisionEnabled", EditConditionHides))
	bool bPreviewCollisionGrid = false;
	
	/** Used to preview collision grid in editor */
	UPROPERTY(Transient, DuplicateTransient, TextExportTransient)
	TObjectPtr<UStaticMeshComponent> CollisionVisualizerComponent;

	UPROPERTY(Transient, DuplicateTransient, TextExportTransient)
	TObjectPtr<UMaterialInstanceDynamic> CollisionVisualizerMaterial;
#endif

private:
#if WITH_EDITOR
	/** Used for PECP */
	static const TCEPropertyChangeDispatcher<UCEClonerCollisionExtension> PropertyChangeDispatcher;
#endif
};