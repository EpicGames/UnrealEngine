// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CEClonerExtensionBase.h"
#include "CEPropertyChangeDispatcher.h"
#include "CEClonerConstraintExtension.generated.h"

/** Extension dealing with clone grid constraints */
UCLASS(MinimalAPI, BlueprintType, Within=CEClonerComponent, meta=(Section="Cloner", Priority=40))
class UCEClonerConstraintExtension : public UCEClonerExtensionBase
{
	GENERATED_BODY()

public:
	UCEClonerConstraintExtension();

	UFUNCTION(BlueprintCallable, Category="Cloner")
	CLONEREFFECTOR_API void SetConstraint(ECEClonerGridConstraint InConstraint);

	UFUNCTION(BlueprintPure, Category="Cloner")
	ECEClonerGridConstraint GetConstraint() const
	{
		return Constraint;
	}

	UFUNCTION(BlueprintCallable, Category="Cloner")
	CLONEREFFECTOR_API void SetInvertConstraint(bool bInInvertConstraint);

	UFUNCTION(BlueprintPure, Category="Cloner")
	bool GetInvertConstraint() const
	{
		return bInvertConstraint;
	}

	UFUNCTION(BlueprintCallable, Category="Cloner")
	CLONEREFFECTOR_API void SetSphereRadius(float InSphereRadius);

	UFUNCTION(BlueprintPure, Category="Cloner")
	float GetSphereRadius() const
	{
		return SphereRadius;
	}

	UFUNCTION(BlueprintCallable, Category="Cloner")
	CLONEREFFECTOR_API void SetSphereCenter(const FVector& InSphereCenter);

	UFUNCTION(BlueprintPure, Category="Cloner")
	FVector GetSphereCenter() const
	{
		return SphereCenter;
	}

	UFUNCTION(BlueprintCallable, Category="Cloner")
	CLONEREFFECTOR_API void SetCylinderRadius(float InCylinderRadius);

	UFUNCTION(BlueprintPure, Category="Cloner")
	float GetCylinderRadius() const
	{
		return CylinderRadius;
	}

	UFUNCTION(BlueprintCallable, Category="Cloner")
	CLONEREFFECTOR_API void SetCylinderHeight(float InCylinderHeight);

	UFUNCTION(BlueprintPure, Category="Cloner")
	float GetCylinderHeight() const
	{
		return CylinderHeight;
	}

	UFUNCTION(BlueprintCallable, Category="Cloner")
	CLONEREFFECTOR_API void SetCylinderCenter(const FVector& InCylinderCenter);

	UFUNCTION(BlueprintPure, Category="Cloner")
	FVector GetCylinderCenter() const
	{
		return CylinderCenter;
	}

	UFUNCTION(BlueprintCallable, Category="Cloner")
	CLONEREFFECTOR_API void SetTextureAsset(UTexture* InTexture);

	UFUNCTION(BlueprintPure, Category="Cloner")
	UTexture* GetTextureAsset() const
	{
		return TextureAsset;
	}

	UFUNCTION(BlueprintCallable, Category="Cloner")
	CLONEREFFECTOR_API void SetTextureSampleMode(ECEClonerTextureSampleChannel InSampleMode);

	UFUNCTION(BlueprintPure, Category="Cloner")
	ECEClonerTextureSampleChannel GetTextureSampleMode() const
	{
		return TextureSampleMode;
	}

	UFUNCTION(BlueprintCallable, Category="Cloner")
	CLONEREFFECTOR_API void SetTexturePlane(ECEClonerPlane InPlane);

	UFUNCTION(BlueprintPure, Category="Cloner")
	ECEClonerPlane GetTexturePlane() const
	{
		return TexturePlane;
	}

	UFUNCTION(BlueprintCallable, Category="Cloner")
	CLONEREFFECTOR_API void SetTextureCompareMode(ECEClonerCompareMode InMode);

	UFUNCTION(BlueprintPure, Category="Cloner")
	ECEClonerCompareMode GetTextureCompareMode() const
	{
		return TextureCompareMode;
	}

	UFUNCTION(BlueprintCallable, Category="Cloner")
	CLONEREFFECTOR_API void SetTextureThreshold(float InThreshold);

	UFUNCTION(BlueprintPure, Category="Cloner")
	float GetTextureThreshold() const
	{
		return TextureThreshold;
	}

	UFUNCTION(BlueprintCallable, Category="Cloner")
	CLONEREFFECTOR_API void SetTextureOffset(const FVector2D& InOffset);

	UFUNCTION(BlueprintPure, Category="Cloner")
	FVector2D GetTextureOffset() const
	{
		return TextureOffset;
	}

	UFUNCTION(BlueprintCallable, Category="Cloner")
	CLONEREFFECTOR_API void SetTextureRotation(float InRotation);

	UFUNCTION(BlueprintPure, Category="Cloner")
	float GetTextureRotation() const
	{
		return TextureRotation;
	}
	
	UFUNCTION(BlueprintCallable, Category="Cloner")
	CLONEREFFECTOR_API void SetTextureScale(const FVector2D& InScale);

	UFUNCTION(BlueprintPure, Category="Cloner")
	FVector2D GetTextureScale() const
	{
		return TextureScale;
	}
	
	UFUNCTION(BlueprintCallable, Category="Cloner")
	CLONEREFFECTOR_API void SetTextureClamp(bool bInClamp);

	UFUNCTION(BlueprintPure, Category="Cloner")
	bool GetTextureClamp() const
	{
		return bTextureClamp;
	}

protected:
	//~ Begin UObject
#if WITH_EDITOR
	virtual void PostEditChangeProperty(FPropertyChangedEvent& InPropertyChangedEvent) override;
#endif
	//~ End UObject

	//~ Begin UCEClonerExtensionBase
	virtual void OnExtensionParametersChanged(UCEClonerComponent* InComponent) override;
	virtual bool IsLayoutSupported(const UCEClonerLayoutBase* InLayout) const override;
	//~ End UCEClonerExtensionBase

	UPROPERTY(EditInstanceOnly, Setter, Getter, Category="Constraint", meta=(RefreshPropertyView))
	ECEClonerGridConstraint Constraint = ECEClonerGridConstraint::None;

	UPROPERTY(EditInstanceOnly, Setter="SetInvertConstraint", Getter="GetInvertConstraint", Category="Constraint", meta=(EditCondition="Constraint != ECEClonerGridConstraint::None", EditConditionHides))
	bool bInvertConstraint = false;

	/** Sphere constraint */

	UPROPERTY(EditInstanceOnly, Setter, Getter, Category="Constraint", meta=(EditCondition="Constraint == ECEClonerGridConstraint::Sphere", EditConditionHides))
	float SphereRadius = 400.f;

	UPROPERTY(EditInstanceOnly, Setter, Getter, Category="Constraint", meta=(EditCondition="Constraint == ECEClonerGridConstraint::Sphere", EditConditionHides))
	FVector SphereCenter = FVector::ZeroVector;

	/** Cylinder constraint */

	UPROPERTY(EditInstanceOnly, Setter, Getter, Category="Constraint", meta=(EditCondition="Constraint == ECEClonerGridConstraint::Cylinder", EditConditionHides))
	float CylinderRadius = 400.f;

	UPROPERTY(EditInstanceOnly, Setter, Getter, Category="Constraint", meta=(EditCondition="Constraint == ECEClonerGridConstraint::Cylinder", EditConditionHides))
	float CylinderHeight = 800.f;

	UPROPERTY(EditInstanceOnly, Setter, Getter, Category="Constraint", meta=(EditCondition="Constraint == ECEClonerGridConstraint::Cylinder", EditConditionHides))
	FVector CylinderCenter = FVector::ZeroVector;

	/** Texture constraint */

	UPROPERTY(EditInstanceOnly, Setter, Getter, Category="Constraint", meta=(EditCondition="Constraint == ECEClonerGridConstraint::Texture", EditConditionHides))
	TObjectPtr<UTexture> TextureAsset;

	UPROPERTY(EditInstanceOnly, Setter, Getter, Category="Constraint", meta=(InvalidEnumValues="Custom", EditCondition="Constraint == ECEClonerGridConstraint::Texture", EditConditionHides))
	ECEClonerPlane TexturePlane = ECEClonerPlane::XY;

	UPROPERTY(EditInstanceOnly, Setter, Getter, Category="Constraint", meta=(EditCondition="Constraint == ECEClonerGridConstraint::Texture", EditConditionHides))
	ECEClonerTextureSampleChannel TextureSampleMode = ECEClonerTextureSampleChannel::RGBLuminance;

	UPROPERTY(EditInstanceOnly, Setter, Getter, Category="Constraint", meta=(EditCondition="Constraint == ECEClonerGridConstraint::Texture", EditConditionHides))
	ECEClonerCompareMode TextureCompareMode = ECEClonerCompareMode::Greater;

	UPROPERTY(EditInstanceOnly, Setter, Getter, Category="Constraint", meta=(ClampMin="0", EditCondition="Constraint == ECEClonerGridConstraint::Texture", EditConditionHides))
	float TextureThreshold = 0.1f;

	UPROPERTY(EditInstanceOnly, Setter, Getter, Category="Constraint", meta=(EditCondition="Constraint == ECEClonerGridConstraint::Texture", EditConditionHides))
	FVector2D TextureOffset = FVector2D::ZeroVector;
	
	UPROPERTY(EditInstanceOnly, Setter, Getter, Category="Constraint", meta=(EditCondition="Constraint == ECEClonerGridConstraint::Texture", EditConditionHides))
	float TextureRotation = 0.f;
	
	UPROPERTY(EditInstanceOnly, Setter, Getter, Category="Constraint", meta=(Delta="0.01", MotionDesignVectorWidget, AllowPreserveRatio="XY", EditCondition="Constraint == ECEClonerGridConstraint::Texture", EditConditionHides))
	FVector2D TextureScale = FVector2D(1.0);

	UPROPERTY(EditInstanceOnly, Setter="SetTextureClamp", Getter="GetTextureClamp", Category="Constraint", meta=(EditCondition="Constraint == ECEClonerGridConstraint::Texture", EditConditionHides))
	bool bTextureClamp = false;

private:
#if WITH_EDITOR
	/** Used for PECP */
	static const TCEPropertyChangeDispatcher<UCEClonerConstraintExtension> PropertyChangeDispatcher;
#endif
};