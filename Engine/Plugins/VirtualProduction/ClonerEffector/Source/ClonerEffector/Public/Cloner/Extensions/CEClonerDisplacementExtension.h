// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CEClonerExtensionBase.h"
#include "CEPropertyChangeDispatcher.h"
#include "CEClonerDisplacementExtension.generated.h"

/** Extension dealing with clone grid displacements */
UCLASS(MinimalAPI, BlueprintType, Within=CEClonerComponent, meta=(Section="Cloner", Priority=50))
class UCEClonerDisplacementExtension : public UCEClonerExtensionBase
{
	GENERATED_BODY()

public:
	UCEClonerDisplacementExtension();

	UFUNCTION(BlueprintCallable, Category="Cloner")
	CLONEREFFECTOR_API void SetDisplacementEnabled(bool bInEnabled);

	UFUNCTION(BlueprintPure, Category="Cloner")
	bool GetDisplacementEnabled() const
	{
		return bDisplacementEnabled;
	}

	UFUNCTION(BlueprintCallable, Category="Cloner")
	CLONEREFFECTOR_API void SetDisplacementInvert(bool bInInvert);

	UFUNCTION(BlueprintPure, Category="Cloner")
	bool GetDisplacementInvert() const
	{
		return bDisplacementInvert;
	}
	
	UFUNCTION(BlueprintCallable, Category="Cloner")
	CLONEREFFECTOR_API void SetDisplacementOffsetMax(const FVector& InMax);

	UFUNCTION(BlueprintPure, Category="Cloner")
	const FVector& GetDisplacementOffsetMax() const
	{
		return DisplacementOffsetMax;
	}
	
	UFUNCTION(BlueprintCallable, Category="Cloner")
	CLONEREFFECTOR_API void SetDisplacementRotationMax(const FRotator& InMax);

	UFUNCTION(BlueprintPure, Category="Cloner")
	const FRotator& GetDisplacementRotationMax() const
	{
		return DisplacementRotationMax;
	}
	
	UFUNCTION(BlueprintCallable, Category="Cloner")
	CLONEREFFECTOR_API void SetDisplacementScaleMax(const FVector& InMax);

	UFUNCTION(BlueprintPure, Category="Cloner")
	const FVector& GetDisplacementScaleMax() const
	{
		return DisplacementScaleMax;
	}

	UFUNCTION(BlueprintCallable, Category="Cloner")
	CLONEREFFECTOR_API void SetDisplacementTextureAsset(UTexture* InTexture);

	UFUNCTION(BlueprintPure, Category="Cloner")
	UTexture* GetDisplacementTextureAsset() const
	{
		return DisplacementTextureAsset;
	}

	UFUNCTION(BlueprintCallable, Category="Cloner")
	CLONEREFFECTOR_API void SetDisplacementTextureSampleMode(ECEClonerTextureSampleChannel InSampleMode);

	UFUNCTION(BlueprintPure, Category="Cloner")
	ECEClonerTextureSampleChannel GetDisplacementTextureSampleMode() const
	{
		return DisplacementTextureSampleMode;
	}

	UFUNCTION(BlueprintCallable, Category="Cloner")
	CLONEREFFECTOR_API void SetDisplacementTexturePlane(ECEClonerPlane InPlane);

	UFUNCTION(BlueprintPure, Category="Cloner")
	ECEClonerPlane GetDisplacementTexturePlane() const
	{
		return DisplacementTexturePlane;
	}

	UFUNCTION(BlueprintCallable, Category="Cloner")
	CLONEREFFECTOR_API void SetDisplacementTextureOffset(const FVector2D& InOffset);

	UFUNCTION(BlueprintPure, Category="Cloner")
	FVector2D GetDisplacementTextureOffset() const
	{
		return DisplacementTextureOffset;
	}

	UFUNCTION(BlueprintCallable, Category="Cloner")
	CLONEREFFECTOR_API void SetDisplacementTextureRotation(float InRotation);

	UFUNCTION(BlueprintPure, Category="Cloner")
	float GetDisplacementTextureRotation() const
	{
		return DisplacementTextureRotation;
	}
	
	UFUNCTION(BlueprintCallable, Category="Cloner")
	CLONEREFFECTOR_API void SetDisplacementTextureScale(const FVector2D& InScale);

	UFUNCTION(BlueprintPure, Category="Cloner")
	FVector2D GetDisplacementTextureScale() const
	{
		return DisplacementTextureScale;
	}
	
	UFUNCTION(BlueprintCallable, Category="Cloner")
	CLONEREFFECTOR_API void SetDisplacementTextureClamp(bool bInClamp);

	UFUNCTION(BlueprintPure, Category="Cloner")
	bool GetDisplacementTextureClamp() const
	{
		return bDisplacementTextureClamp;
	}

protected:
	//~ Begin UObject
#if WITH_EDITOR
	virtual void PostEditChangeProperty(FPropertyChangedEvent& InPropertyChangedEvent) override;
#endif
	//~ End UObject

	//~ Begin UCEClonerExtensionBase
	virtual void OnExtensionParametersChanged(UCEClonerComponent* InComponent) override;
	virtual bool IsLayoutSupported(const UCEClonerLayoutBase* InLayout) const override;
	//~ End UCEClonerExtensionBase

	UPROPERTY(EditInstanceOnly, Setter="SetDisplacementEnabled", Getter="GetDisplacementEnabled", DisplayName="Enabled", Category="Displacement", meta=(RefreshPropertyView))
	bool bDisplacementEnabled = false;

	UPROPERTY(EditInstanceOnly, Setter="SetDisplacementInvert", Getter="GetDisplacementInvert", DisplayName="Invert", Category="Displacement", meta=(EditCondition="bDisplacementEnabled", EditConditionHides))
	bool bDisplacementInvert = false;

	UPROPERTY(EditInstanceOnly, Setter, Getter, DisplayName="Offset", Category="Displacement", meta=(EditCondition="bDisplacementEnabled", EditConditionHides))
	FVector DisplacementOffsetMax = FVector(0, 0, 100.f);

	UPROPERTY(EditInstanceOnly, Setter, Getter, DisplayName="Rotation", Category="Displacement", meta=(EditCondition="bDisplacementEnabled", EditConditionHides))
	FRotator DisplacementRotationMax = FRotator::ZeroRotator;

	UPROPERTY(EditInstanceOnly, Setter, Getter, DisplayName="Scale", Category="Displacement", meta=(ClampMin="0", Delta="0.01", MotionDesignVectorWidget, AllowPreserveRatio="XYZ", EditCondition="bDisplacementEnabled", EditConditionHides))
	FVector DisplacementScaleMax = FVector::OneVector;

	UPROPERTY(EditInstanceOnly, Setter, Getter, DisplayName="TextureAsset", Category="Displacement", meta=(EditCondition="bDisplacementEnabled", EditConditionHides))
	TObjectPtr<UTexture> DisplacementTextureAsset;

	UPROPERTY(EditInstanceOnly, Setter, Getter, DisplayName="TexturePlane", Category="Displacement", meta=(InvalidEnumValues="Custom", EditCondition="bDisplacementEnabled", EditConditionHides))
	ECEClonerPlane DisplacementTexturePlane = ECEClonerPlane::XY;

	UPROPERTY(EditInstanceOnly, Setter, Getter, DisplayName="TextureSampleMode", Category="Displacement", meta=(EditCondition="bDisplacementEnabled", EditConditionHides))
	ECEClonerTextureSampleChannel DisplacementTextureSampleMode = ECEClonerTextureSampleChannel::RGBLuminance;

	UPROPERTY(EditInstanceOnly, Setter, Getter, DisplayName="TextureOffset", Category="Displacement", meta=(EditCondition="bDisplacementEnabled", EditConditionHides))
	FVector2D DisplacementTextureOffset = FVector2D::ZeroVector;
	
	UPROPERTY(EditInstanceOnly, Setter, Getter, DisplayName="TextureRotation", Category="Displacement", meta=(EditCondition="bDisplacementEnabled", EditConditionHides))
	float DisplacementTextureRotation = 0.f;
	
	UPROPERTY(EditInstanceOnly, Setter, Getter, DisplayName="TextureScale", Category="Displacement", meta=(Delta="0.01", MotionDesignVectorWidget, AllowPreserveRatio="XY", EditCondition="bDisplacementEnabled", EditConditionHides))
	FVector2D DisplacementTextureScale = FVector2D(1.0);

	UPROPERTY(EditInstanceOnly, Setter="SetDisplacementTextureClamp", Getter="GetDisplacementTextureClamp", DisplayName="TextureClamp", Category="Displacement", meta=(EditCondition="bDisplacementEnabled", EditConditionHides))
	bool bDisplacementTextureClamp = false;

private:
#if WITH_EDITOR
	static const TCEPropertyChangeDispatcher<UCEClonerDisplacementExtension> PropertyChangeDispatcher;
#endif
};