// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CEClonerExtensionBase.h"
#include "CEPropertyChangeDispatcher.h"
#include "GameFramework/Actor.h"
#include "UObject/WeakObjectPtrTemplates.h"
#include "CEClonerEffectorExtension.generated.h"

class UCEEffectorComponent;
struct FCEClonerEffectorDataInterfaces;

/** Extension dealing with effectors options */
UCLASS(MinimalAPI, BlueprintType, Within=CEClonerComponent, meta=(Section="Effector", Priority=30))
class UCEClonerEffectorExtension : public UCEClonerExtensionBase
{
	GENERATED_BODY()

public:
#if WITH_EDITOR
	static CLONEREFFECTOR_API FName GetEffectorActorsWeakName();
#endif

	UCEClonerEffectorExtension();

	/** Gets the number of effectors applied on this cloner */
	UFUNCTION(BlueprintPure, Category="Cloner")
	CLONEREFFECTOR_API int32 GetEffectorCount() const;

	/** Links new actor effectors to apply transformation on clones */
	UFUNCTION(BlueprintCallable, Category="Cloner")
	CLONEREFFECTOR_API bool LinkEffector(AActor* InEffectorActor);

	/** Unlinks the effector actor and reset the cloner simulation */
	UFUNCTION(BlueprintCallable, Category="Cloner")
	CLONEREFFECTOR_API bool UnlinkEffector(AActor* InEffectorActor);

	/** Checks if an effector is linked with this cloner */
	UFUNCTION(BlueprintPure, Category="Cloner")
	CLONEREFFECTOR_API bool IsEffectorLinked(const AActor* InEffectorActor) const;

#if WITH_EDITOR
	/** This will create a new effector actor, link it to this cloner and select it */
	UFUNCTION(CallInEditor, Category="Effector")
	void CreateLinkedEffector();
#endif

protected:
	//~ Begin UObject
#if WITH_EDITOR
	virtual void PostEditChangeProperty(FPropertyChangedEvent& InPropertyChangedEvent) override;
	virtual void PostEditUndo() override;
#endif
	//~ End UObject

	//~ Begin UCEClonerExtensionBase
	virtual void OnExtensionActivated() override;
	virtual void OnExtensionDeactivated() override;
	virtual void OnExtensionParametersChanged(UCEClonerComponent* InComponent) override;
	//~ End UCEClonerExtensionBase

	void OnEffectorIdentifierChanged(UCEEffectorComponent* InEffector, int32 InOldIdentifier, int32 InNewIdentifier);
	void OnEffectorsChanged();
	void OnEffectorActorsChanged();

	/** Effectors actors linked to this cloner */
	UPROPERTY(EditInstanceOnly, Category="Effector", meta=(DisplayName="Effectors"))
	TArray<TWeakObjectPtr<AActor>> EffectorActorsWeak;

private:
	/** Copy of effectors linked to this cloner to compare diffs */
	UPROPERTY(Transient, DuplicateTransient, TextExportTransient, NonTransactional)
	TSet<TWeakObjectPtr<UCEEffectorComponent>> EffectorsInternalWeak;

#if WITH_EDITOR
	/** Used for PECP */
	static const TCEPropertyChangeDispatcher<UCEClonerEffectorExtension> PropertyChangeDispatcher;
#endif
};