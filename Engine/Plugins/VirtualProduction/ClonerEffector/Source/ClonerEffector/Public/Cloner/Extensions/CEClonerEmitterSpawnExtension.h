// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CEClonerEffectorShared.h"
#include "CEClonerExtensionBase.h"
#include "CEPropertyChangeDispatcher.h"
#include "CEClonerEmitterSpawnExtension.generated.h"

class UNiagaraDataInterfaceCurve;

/** Extension dealing with clones spawning options */
UCLASS(MinimalAPI, BlueprintType, Within=CEClonerComponent, meta=(Section="Emission", Priority=70))
class UCEClonerEmitterSpawnExtension : public UCEClonerExtensionBase
{
	GENERATED_BODY()

public:
	UCEClonerEmitterSpawnExtension();

	UFUNCTION(BlueprintCallable, Category="Cloner")
	CLONEREFFECTOR_API void SetSpawnLoopMode(ECEClonerSpawnLoopMode InMode);

	UFUNCTION(BlueprintPure, Category="Cloner")
	ECEClonerSpawnLoopMode GetSpawnLoopMode() const
	{
		return SpawnLoopMode;
	}

	UFUNCTION(BlueprintCallable, Category="Cloner")
	CLONEREFFECTOR_API void SetSpawnLoopIterations(int32 InIterations);

	UFUNCTION(BlueprintPure, Category="Cloner")
	int32 GetSpawnLoopIterations() const
	{
		return SpawnLoopIterations;
	}

	UFUNCTION(BlueprintCallable, Category="Cloner")
	CLONEREFFECTOR_API void SetSpawnLoopInterval(float InInterval);

	UFUNCTION(BlueprintPure, Category="Cloner")
	float GetSpawnLoopInterval() const
	{
		return SpawnLoopInterval;
	}

	UFUNCTION(BlueprintCallable, Category="Cloner")
	CLONEREFFECTOR_API void SetSpawnBehaviorMode(ECEClonerSpawnBehaviorMode InMode);

	UFUNCTION(BlueprintPure, Category="Cloner")
	ECEClonerSpawnBehaviorMode GetSpawnBehaviorMode() const
	{
		return SpawnBehaviorMode;
	}

	UFUNCTION(BlueprintCallable, Category="Cloner")
	CLONEREFFECTOR_API void SetSpawnRate(float InRate);

	UFUNCTION(BlueprintPure, Category="Cloner")
	float GetSpawnRate() const
	{
		return SpawnRate;
	}

	UFUNCTION(BlueprintCallable, Category="Cloner")
	CLONEREFFECTOR_API void SetSpawnMaxFrameCount(int32 InCount);

	UFUNCTION(BlueprintPure, Category="Cloner")
	int32 GetSpawnMaxFrameCount() const
	{
		return SpawnMaxFrameCount;
	}
	
	UFUNCTION(BlueprintCallable, Category="Cloner")
	CLONEREFFECTOR_API void SetSpawnMaxTotalCount(int32 InCount);

	UFUNCTION(BlueprintPure, Category="Cloner")
	int32 GetSpawnMaxTotalCount() const
	{
		return SpawnMaxTotalCount;
	}

#if WITH_EDITOR
	UFUNCTION(BlueprintCallable, Category="Cloner")
	CLONEREFFECTOR_API void SetUseLocalSpace(bool bInLocalSpace);

	UFUNCTION(BlueprintPure, Category="Cloner")
	bool GetUseLocalSpace() const
	{
		return bUseLocalSpace;
	}
#endif

protected:
	//~ Begin UObject
#if WITH_EDITOR
	virtual void PostEditChangeProperty(FPropertyChangedEvent& InPropertyChangedEvent) override;
#endif
	//~ End UObject

	//~ Begin UCEClonerExtensionBase
	virtual void OnExtensionParametersChanged(UCEClonerComponent* InComponent) override;
	//~ End UCEClonerExtensionBase

	/** How many times do we spawn clones */
	UPROPERTY(EditInstanceOnly, Setter, Getter, DisplayName="Emission Mode", Category="Spawn", meta=(RefreshPropertyView))
	ECEClonerSpawnLoopMode SpawnLoopMode = ECEClonerSpawnLoopMode::Once;

	/** Amount of spawn iterations for clones */
	UPROPERTY(EditInstanceOnly, Setter, Getter, DisplayName="Emission Count", Category="Spawn", meta=(ClampMin="1", EditCondition="SpawnLoopMode == ECEClonerSpawnLoopMode::Multiple", EditConditionHides))
	int32 SpawnLoopIterations = 1;

	/** How does spawn occurs */
	UPROPERTY(EditInstanceOnly, Setter, Getter, DisplayName="Emission Style", Category="Spawn", meta=(EditCondition="SpawnLoopMode != ECEClonerSpawnLoopMode::Once", EditConditionHides, RefreshPropertyView))
	ECEClonerSpawnBehaviorMode SpawnBehaviorMode = ECEClonerSpawnBehaviorMode::Instant;

	/** Interval/Duration of spawn for clones */
	UPROPERTY(EditInstanceOnly, Setter, Getter, DisplayName="Emission Interval", Category="Spawn", meta=(ClampMin="0", EditCondition="SpawnLoopMode != ECEClonerSpawnLoopMode::Once && SpawnBehaviorMode == ECEClonerSpawnBehaviorMode::Instant", EditConditionHides))
	float SpawnLoopInterval = 1.f;
	
	/** How many clones to spawn each seconds */
	UPROPERTY(EditInstanceOnly, Setter, Getter, DisplayName="Emission Rate", Category="Spawn", meta=(ClampMin="0", EditCondition="SpawnLoopMode != ECEClonerSpawnLoopMode::Once && SpawnBehaviorMode == ECEClonerSpawnBehaviorMode::Rate", EditConditionHides))
	float SpawnRate = 1.f;

	/** Simulation takes place in local space or world space */
	UPROPERTY(EditInstanceOnly, Category="Spawn", meta=(EditCondition="SpawnLoopMode != ECEClonerSpawnLoopMode::Once", EditConditionHides))
	bool bUseLocalSpace = true;

	/** Amount of particle allowed to spawn in a single frame */
	UPROPERTY(EditInstanceOnly, Setter, Getter, DisplayName="Max Frame Spawn Count", Category="Spawn", meta=(ClampMin="0"))
	int32 SpawnMaxFrameCount = 1000000;

	/** Amount of particle allowed to spawn by the emitter */
	UPROPERTY(EditInstanceOnly, Setter, Getter, DisplayName="Max Total Spawn Count", Category="Spawn", meta=(ClampMin="0"))
	int32 SpawnMaxTotalCount = 1000000;

private:
#if WITH_EDITOR
	/** Used for PECP */
	static const TCEPropertyChangeDispatcher<UCEClonerEmitterSpawnExtension> PropertyChangeDispatcher;

	void OnLocalSpaceChanged();
#endif
};