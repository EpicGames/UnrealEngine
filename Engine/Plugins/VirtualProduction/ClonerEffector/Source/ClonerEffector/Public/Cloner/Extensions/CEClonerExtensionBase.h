// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CEClonerEffectorShared.h"
#include "UObject/Object.h"
#include "CEClonerExtensionBase.generated.h"

class UCEClonerComponent;
class UCEClonerLayoutBase;
struct FNiagaraMeshRendererMeshPropertiesBase;

/** Reusable extension on cloner layout to group similar options */
UCLASS(MinimalAPI, Abstract, BlueprintType, Within=CEClonerComponent)
class UCEClonerExtensionBase : public UObject
{
	GENERATED_BODY()

public:
	UCEClonerExtensionBase();

	UCEClonerExtensionBase(FName InExtensionName, int32 InExtensionPriority);

	UFUNCTION(BlueprintPure, Category="Cloner")
	FName GetExtensionName() const
	{
		return ExtensionName;
	}

	int32 GetExtensionPriority() const
	{
		return ExtensionPriority;
	}

#if WITH_EDITOR
	CLONEREFFECTOR_API FCEExtensionSection GetExtensionSection() const;
#endif

	/** Get the cloner component using this extension */
	CLONEREFFECTOR_API UCEClonerComponent* GetClonerComponent() const;

	/** Get the cloner component using this extension checked version */
	UCEClonerComponent* GetClonerComponentChecked() const;

	/** Get the cloner active layout that uses this extension */
	UCEClonerLayoutBase* GetClonerLayout() const;

	/** Enable this extension */
	void ActivateExtension();

	/** Disable this extension */
	void DeactivateExtension();

	UFUNCTION(BlueprintPure, Category="Cloner")
	bool IsExtensionActive() const
	{
		return bExtensionActive;
	}

	/**
	 * Called when the meshes are being updated. With opportunity for extension to change the mesh data.
	 * @return true if the meshes were changed.
	 */
	virtual bool OnUpdateClonerMeshes(TArray<FNiagaraMeshRendererMeshPropertiesBase>& InOutMeshes)
	{
		return false;
	}

	/** Called when the meshes are updated */
	virtual void OnClonerMeshesUpdated() {}

	/** Updates all extensions parameters */
	void UpdateExtensionParameters();

	/** Request refresh extension next tick */
	void MarkExtensionDirty(bool bInUpdateCloner = true);

	/** Is this extension dirty */
	bool IsExtensionDirty() const;

	/** Filter supported layout for this extension */
	virtual bool IsLayoutSupported(const UCEClonerLayoutBase* InLayout) const
	{
		return true;
	}

	/** Called when an active extension is dirtied to allow other extensions to react accordingly */
	virtual void OnExtensionDirtied(const UCEClonerExtensionBase* InExtension) {}

protected:
	//~ Begin UObject
	virtual void PostEditImport() override;
#if WITH_EDITOR
	virtual void PostEditUndo() override;
#endif
	//~ End UObject

	/** Called when the extension becomes active */
	virtual void OnExtensionActivated() {}

	/** Called when the extension becomes inactive */
	virtual void OnExtensionDeactivated() {}

	/** Called to reapply extension parameters */
	virtual void OnExtensionParametersChanged(UCEClonerComponent* InComponent) {}

	/** Used by PECP to update parameters */
	void OnExtensionPropertyChanged();

private:
	/** Unique extension name for the cloner */
	UPROPERTY(Transient)
	FName ExtensionName = NAME_None;

	/** Some extensions need to be called before or after others */
	UPROPERTY(Transient)
	int32 ExtensionPriority = 0;

	bool bExtensionActive = false;

	ECEClonerSystemStatus ExtensionStatus = ECEClonerSystemStatus::UpToDate;
};