// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CEClonerExtensionBase.h"
#include "CEPropertyChangeDispatcher.h"
#include "Curves/RichCurve.h"
#include "CEClonerLifetimeExtension.generated.h"

class UCurveFloat;
class UNiagaraDataInterfaceCurve;

/** Extension dealing with clones lifetime options */
UCLASS(MinimalAPI, BlueprintType, Within=CEClonerComponent, meta=(Section="Emission", Priority=80))
class UCEClonerLifetimeExtension : public UCEClonerExtensionBase
{
	GENERATED_BODY()

public:
	UCEClonerLifetimeExtension();

	UFUNCTION(BlueprintCallable, Category="Cloner")
	CLONEREFFECTOR_API void SetLifetimeEnabled(bool bInEnabled);

	UFUNCTION(BlueprintPure, Category="Cloner")
	bool GetLifetimeEnabled() const
	{
		return bLifetimeEnabled;
	}

	UFUNCTION(BlueprintCallable, Category="Cloner")
	CLONEREFFECTOR_API void SetLifetimeMin(float InMin);

	UFUNCTION(BlueprintPure, Category="Cloner")
	float GetLifetimeMin() const
	{
		return LifetimeMin;
	}

	UFUNCTION(BlueprintCallable, Category="Cloner")
	CLONEREFFECTOR_API void SetLifetimeMax(float InMax);

	UFUNCTION(BlueprintPure, Category="Cloner")
	float GetLifetimeMax() const
	{
		return LifetimeMax;
	}

	UFUNCTION(BlueprintCallable, Category="Cloner")
	CLONEREFFECTOR_API void SetLifetimeScaleEnabled(bool bInEnabled);

	UFUNCTION(BlueprintPure, Category="Cloner")
	bool GetLifetimeScaleEnabled() const
	{
		return bLifetimeScaleEnabled;
	}

	UFUNCTION(BlueprintCallable, Category="Cloner")
	void SetLifetimeScaleCurve(UCurveFloat* InCurve);

	CLONEREFFECTOR_API void SetLifetimeScaleCurve(const FRichCurve& InCurve);

	const FRichCurve& GetLifetimeScaleCurve() const
	{
		return LifetimeScaleCurve;
	}

	UNiagaraDataInterfaceCurve* GetLifetimeScaleCurveDI() const
	{
		return LifetimeScaleCurveDIWeak.Get();
	}

protected:
	//~ Begin UObject
#if WITH_EDITOR
	virtual void PostEditChangeProperty(FPropertyChangedEvent& InPropertyChangedEvent) override;
#endif
	//~ End UObject

	//~ Begin UCEClonerExtensionBase
	virtual void OnExtensionParametersChanged(UCEClonerComponent* InComponent) override;
	//~ End UCEClonerExtensionBase

	void OnLifetimeScaleCurveChanged();

	/** Do we destroy the clones after a specific duration */
	UPROPERTY(EditInstanceOnly, Setter="SetLifetimeEnabled", Getter="GetLifetimeEnabled", DisplayName="Enabled", Category="Lifetime", meta=(RefreshPropertyView))
	bool bLifetimeEnabled = false;

	/** Minimum lifetime for a clone */
	UPROPERTY(EditInstanceOnly, Setter, Getter, DisplayName="Min", Category="Lifetime", meta=(ClampMin="0", EditCondition="bLifetimeEnabled", EditConditionHides))
	float LifetimeMin = 0.25f;

	/** Maximum lifetime for a clone */
	UPROPERTY(EditInstanceOnly, Setter, Getter, DisplayName="Max", Category="Lifetime", meta=(ClampMin="0", EditCondition="bLifetimeEnabled", EditConditionHides))
	float LifetimeMax = 1.5f;

	/** Enable scale by lifetime */
	UPROPERTY(EditInstanceOnly, Setter="SetLifetimeScaleEnabled", Getter="GetLifetimeScaleEnabled", DisplayName="ScaleEnabled", Category="Lifetime", meta=(EditCondition="bLifetimeEnabled", EditConditionHides, RefreshPropertyView))
	bool bLifetimeScaleEnabled = false;

	/** Used to expose the scale curve editor in details panel */
	UPROPERTY(Transient, DuplicateTransient, TextExportTransient)
	TWeakObjectPtr<UNiagaraDataInterfaceCurve> LifetimeScaleCurveDIWeak;

	/** Rich curve used in the data interface for the lifetime */
	UPROPERTY(Setter, Getter)
	FRichCurve LifetimeScaleCurve;

private:
#if WITH_EDITOR
	/** Used for PECP */
	static const TCEPropertyChangeDispatcher<UCEClonerLifetimeExtension> PropertyChangeDispatcher;
#endif
};