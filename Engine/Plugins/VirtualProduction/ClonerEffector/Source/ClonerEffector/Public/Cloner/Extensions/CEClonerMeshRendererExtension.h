// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CEClonerEffectorShared.h"
#include "CEClonerExtensionBase.h"
#include "CEPropertyChangeDispatcher.h"
#include "NiagaraMeshRendererProperties.h"
#include "CEClonerMeshRendererExtension.generated.h"

/** Extension dealing with mesh rendering options */
UCLASS(MinimalAPI, BlueprintType, Within=CEClonerComponent, meta=(Section="Rendering", Priority=110))
class UCEClonerMeshRendererExtension : public UCEClonerExtensionBase
{
	GENERATED_BODY()

public:
	UCEClonerMeshRendererExtension();

	UFUNCTION(BlueprintCallable, Category="Cloner")
	CLONEREFFECTOR_API void SetMeshRenderMode(ECEClonerMeshRenderMode InMode);

	UFUNCTION(BlueprintPure, Category="Cloner")
	ECEClonerMeshRenderMode GetMeshRenderMode() const
	{
		return MeshRenderMode;
	}

	UFUNCTION(BlueprintCallable, Category="Cloner")
	CLONEREFFECTOR_API void SetMeshFacingMode(ENiagaraMeshFacingMode InMode);

	UFUNCTION(BlueprintPure, Category="Cloner")
	ENiagaraMeshFacingMode GetMeshFacingMode() const
	{
		return MeshFacingMode;
	}

	UFUNCTION(BlueprintCallable, Category="Cloner")
	CLONEREFFECTOR_API void SetMeshCastShadows(bool InbCastShadows);

	UFUNCTION(BlueprintPure, Category="Cloner")
	bool GetMeshCastShadows() const
	{
		return bMeshCastShadows;
	}

	CLONEREFFECTOR_API void SetDefaultMeshes(const TArray<TObjectPtr<UStaticMesh>>& InMeshes);

	const TArray<TObjectPtr<UStaticMesh>>& GetDefaultMeshes() const
	{
		return DefaultMeshes;
	}

	UFUNCTION(BlueprintCallable, Category="Cloner", meta=(DisplayName="SetDefaultMeshes"))
	CLONEREFFECTOR_API void SetDefaultMeshes(const TArray<UStaticMesh*>& InMeshes);

	UFUNCTION(BlueprintPure, Category="Cloner", meta=(DisplayName="GetDefaultMeshes"))
	CLONEREFFECTOR_API void GetDefaultMeshes(TArray<UStaticMesh*>& OutMeshes) const;

	UFUNCTION(BlueprintCallable, Category="Cloner")
	CLONEREFFECTOR_API void SetVisualizeEffectors(bool bInVisualize);

	UFUNCTION(BlueprintPure, Category="Cloner")
	bool GetVisualizeEffectors() const
	{
		return bVisualizeEffectors;
	}

	UFUNCTION(BlueprintCallable, Category="Cloner")
	CLONEREFFECTOR_API void SetUseOverrideMaterial(bool bInOverride);

	UFUNCTION(BlueprintPure, Category="Cloner")
	bool GetUseOverrideMaterial() const
	{
		return bUseOverrideMaterial;
	}

	UFUNCTION(BlueprintCallable, Category="Cloner")
	CLONEREFFECTOR_API void SetOverrideMaterial(UMaterialInterface* InMaterial);

	UFUNCTION(BlueprintPure, Category="Cloner")
	UMaterialInterface* GetOverrideMaterial() const
	{
		return OverrideMaterial;
	}

	UFUNCTION(BlueprintCallable, Category="Cloner")
	CLONEREFFECTOR_API void SetSortTranslucentParticles(bool bInSort);

	UFUNCTION(BlueprintPure, Category="Cloner")
	bool GetSortTranslucentParticles() const
	{
		return bSortTranslucentParticles;
	}

protected:
	//~ Begin UObject
#if WITH_EDITOR
	virtual void PostTransacted(const FTransactionObjectEvent& InTransactionEvent) override;
	virtual void PostEditChangeProperty(FPropertyChangedEvent& InPropertyChangedEvent) override;
#endif
	//~ End UObject

	//~ Begin UCEClonerExtensionBase
	virtual bool OnUpdateClonerMeshes(TArray<FNiagaraMeshRendererMeshPropertiesBase>& InOutMeshes) override;
	virtual void OnExtensionParametersChanged(UCEClonerComponent* InComponent) override;
	//~ End UCEClonerExtensionBase

	/** Get the number of material slots available */
	int32 GetClonerMeshesMaterialCount() const;

	/** Get the niagara materials override if enabled */
	TArray<FNiagaraMeshMaterialOverride> GetOverrideMeshesMaterials() const;

	void OnOverrideMaterialOptionsChanged();

	/** Indicates how we select the mesh to render on each clones */
	UPROPERTY(EditInstanceOnly, Setter, Getter, Category="Renderer")
	ECEClonerMeshRenderMode MeshRenderMode = ECEClonerMeshRenderMode::Iterate;

	/** Mode to indicate how clones facing is determined */
	UPROPERTY(EditInstanceOnly, Setter, Getter, Category="Renderer")
	ENiagaraMeshFacingMode MeshFacingMode = ENiagaraMeshFacingMode::Default;

	/** Whether clones cast shadow, disabling will result in better performance */
	UPROPERTY(EditInstanceOnly, Setter="SetMeshCastShadows", Getter="GetMeshCastShadows", Category="Renderer")
	bool bMeshCastShadows = true;

	/** When nothing is attached to the cloner, these meshes are used as default */
	UPROPERTY(EditInstanceOnly, Setter, Getter, Category="Renderer")
	TArray<TObjectPtr<UStaticMesh>> DefaultMeshes;

	/** Override materials to show effectors applied on this cloner based on their color property */
	UPROPERTY(EditInstanceOnly, Setter="SetVisualizeEffectors", Getter="GetVisualizeEffectors", Category="Renderer")
	bool bVisualizeEffectors = false;

	/** Whether to override meshes materials with another material */
	UPROPERTY(EditInstanceOnly, Setter="SetUseOverrideMaterial", Getter="GetUseOverrideMaterial", Category="Renderer", meta=(RefreshPropertyView))
	bool bUseOverrideMaterial = false;

	/** The override materials that will be set instead of meshes materials, bVisualizeEffectors must be disabled */
	UPROPERTY(EditAnywhere, Setter, Getter, Category="Renderer", meta=(EditCondition="bUseOverrideMaterial", EditConditionHides))
	TObjectPtr<UMaterialInterface> OverrideMaterial;

	/** Sort particles by depth when it has a translucent material, this will avoid flickering artifacts from appearing */
	UPROPERTY(EditInstanceOnly, Setter="SetSortTranslucentParticles", Getter="GetSortTranslucentParticles", Category="Renderer")
	bool bSortTranslucentParticles = true;

private:
#if WITH_EDITOR
	/** Used for PECP */
	static const TCEPropertyChangeDispatcher<UCEClonerMeshRendererExtension> PropertyChangeDispatcher;
#endif
};
