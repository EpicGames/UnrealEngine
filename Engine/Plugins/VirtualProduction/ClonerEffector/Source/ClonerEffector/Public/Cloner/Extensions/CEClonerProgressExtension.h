// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CEClonerExtensionBase.h"
#include "CEPropertyChangeDispatcher.h"
#include "CEClonerProgressExtension.generated.h"

/** Extension dealing with clone progress options */
UCLASS(MinimalAPI, BlueprintType, Within=CEClonerComponent, meta=(Section="Emission", Priority=100))
class UCEClonerProgressExtension : public UCEClonerExtensionBase
{
	GENERATED_BODY()

public:
	UCEClonerProgressExtension();

	UFUNCTION(BlueprintCallable, Category="Cloner")
	CLONEREFFECTOR_API void SetInvertProgress(bool bInInvertProgress);

	UFUNCTION(BlueprintPure, Category="Cloner")
	bool GetInvertProgress() const
	{
		return bInvertProgress;
	}

	UFUNCTION(BlueprintCallable, Category="Cloner")
	CLONEREFFECTOR_API void SetProgress(float InProgress);

	UFUNCTION(BlueprintPure, Category="Cloner")
	float GetProgress() const
	{
		return Progress;
	}

protected:
	//~ Begin UObject
#if WITH_EDITOR
	virtual void PostEditChangeProperty(FPropertyChangedEvent& InPropertyChangedEvent) override;
#endif
	//~ End UObject

	//~ Begin UCEClonerExtensionBase
	virtual void OnExtensionParametersChanged(UCEClonerComponent* InComponent) override;
	//~ End UCEClonerExtensionBase

	/** Invert progress behaviour */
	UPROPERTY(EditInstanceOnly, Setter="SetInvertProgress", Getter="GetInvertProgress", Category="Progress")
	bool bInvertProgress = false;

	/** Changes visibility of instances based on the total count, 1.f = 100% = all instances visible */
	UPROPERTY(EditInstanceOnly, Setter, Getter, Category="Progress", meta=(ClampMin="0", ClampMax="1"))
	float Progress = 1.f;

private:
#if WITH_EDITOR
	/** Used for PECP */
	static const TCEPropertyChangeDispatcher<UCEClonerProgressExtension> PropertyChangeDispatcher;
#endif
};