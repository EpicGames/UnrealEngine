// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CEClonerExtensionBase.h"
#include "CEPropertyChangeDispatcher.h"
#include "CEClonerRangeExtension.generated.h"

/** Extension dealing with range options */
UCLASS(MinimalAPI, BlueprintType, Within=CEClonerComponent, meta=(Section="Cloner", Priority=10))
class UCEClonerRangeExtension : public UCEClonerExtensionBase
{
	GENERATED_BODY()

public:
	UCEClonerRangeExtension();

	UFUNCTION(BlueprintCallable, Category="Cloner")
	CLONEREFFECTOR_API void SetRangeEnabled(bool bInRangeEnabled);

	UFUNCTION(BlueprintPure, Category="Cloner")
	bool GetRangeEnabled() const
	{
		return bRangeEnabled;
	}

	UFUNCTION(BlueprintCallable, Category="Cloner")
	CLONEREFFECTOR_API void SetRangeMirrored(bool bInMirrored);

	UFUNCTION(BlueprintPure, Category="Cloner")
	bool GetRangeMirrored() const
	{
		return bRangeMirrored;
	}

	UFUNCTION(BlueprintCallable, Category="Cloner")
	CLONEREFFECTOR_API void SetRangeOffsetMin(const FVector& InRangeOffsetMin);

	UFUNCTION(BlueprintPure, Category="Cloner")
	const FVector& GetRangeOffsetMin() const
	{
		return RangeOffsetMin;
	}

	UFUNCTION(BlueprintCallable, Category="Cloner")
	CLONEREFFECTOR_API void SetRangeOffsetMax(const FVector& InRangeOffsetMax);

	UFUNCTION(BlueprintPure, Category="Cloner")
	const FVector& GetRangeOffsetMax() const
	{
		return RangeOffsetMax;
	}

	UFUNCTION(BlueprintCallable, Category="Cloner")
	CLONEREFFECTOR_API void SetRangeRotationMin(const FRotator& InRangeRotationMin);

	UFUNCTION(BlueprintPure, Category="Cloner")
	const FRotator& GetRangeRotationMin() const
	{
		return RangeRotationMin;
	}

	UFUNCTION(BlueprintCallable, Category="Cloner")
	CLONEREFFECTOR_API void SetRangeRotationMax(const FRotator& InRangeRotationMax);

	UFUNCTION(BlueprintPure, Category="Cloner")
	const FRotator& GetRangeRotationMax() const
	{
		return RangeRotationMax;
	}

	UFUNCTION(BlueprintCallable, Category="Cloner")
	CLONEREFFECTOR_API void SetRangeScaleUniform(bool bInRangeScaleUniform);

	UFUNCTION(BlueprintPure, Category="Cloner")
	bool GetRangeScaleUniform() const
	{
		return bRangeScaleUniform;
	}

	UFUNCTION(BlueprintCallable, Category="Cloner")
	CLONEREFFECTOR_API void SetRangeScaleMin(const FVector& InRangeScaleMin);

	UFUNCTION(BlueprintPure, Category="Cloner")
	const FVector& GetRangeScaleMin() const
	{
		return RangeScaleMin;
	}

	UFUNCTION(BlueprintCallable, Category="Cloner")
	CLONEREFFECTOR_API void SetRangeScaleMax(const FVector& InRangeScaleMax);

	UFUNCTION(BlueprintPure, Category="Cloner")
	const FVector& GetRangeScaleMax() const
	{
		return RangeScaleMax;
	}

	UFUNCTION(BlueprintCallable, Category="Cloner")
	CLONEREFFECTOR_API void SetRangeScaleUniformMin(float InRangeScaleUniformMin);

	UFUNCTION(BlueprintPure, Category="Cloner")
	float GetRangeScaleUniformMin() const
	{
		return RangeScaleUniformMin;
	}

	UFUNCTION(BlueprintCallable, Category="Cloner")
	CLONEREFFECTOR_API void SetRangeScaleUniformMax(float InRangeScaleUniformMax);

	UFUNCTION(BlueprintPure, Category="Cloner")
	float GetRangeScaleUniformMax() const
	{
		return RangeScaleUniformMax;
	}

protected:
	//~ Begin UObject
#if WITH_EDITOR
	virtual void PostEditChangeProperty(FPropertyChangedEvent& InPropertyChangedEvent) override;
#endif
	//~ End UObject

	//~ Begin UCEClonerExtensionBase
	virtual void OnExtensionParametersChanged(UCEClonerComponent* InComponent) override;
	//~ End UCEClonerExtensionBase

	/** Use random range transforms for each clones */
	UPROPERTY(EditInstanceOnly, Setter="SetRangeEnabled", Getter="GetRangeEnabled", DisplayName="Enabled", Category="Range", meta=(RefreshPropertyView))
	bool bRangeEnabled = false;

	/** Mirrors max offset and rotation values in min offset and rotation */
	UPROPERTY(EditInstanceOnly, Setter="SetRangeMirrored", Getter="GetRangeMirrored", DisplayName="Mirror Min & Max", Category="Range", meta=(EditCondition="bRangeEnabled", EditConditionHides, RefreshPropertyView))
	bool bRangeMirrored = true;

	UPROPERTY(EditInstanceOnly, Setter, Getter, DisplayName="OffsetMin", Category="Range", meta=(EditCondition="bRangeEnabled && !bRangeMirrored", EditConditionHides))
	FVector RangeOffsetMin = FVector::ZeroVector;

	UPROPERTY(EditInstanceOnly, Setter, Getter, DisplayName="OffsetMax", Category="Range", meta=(EditCondition="bRangeEnabled", EditConditionHides))
	FVector RangeOffsetMax = FVector::ZeroVector;

	UPROPERTY(EditInstanceOnly, Setter, Getter, DisplayName="RotationMin", Category="Range", meta=(EditCondition="bRangeEnabled && !bRangeMirrored", EditConditionHides))
	FRotator RangeRotationMin = FRotator::ZeroRotator;

	UPROPERTY(EditInstanceOnly, Setter, Getter, DisplayName="RotationMax", Category="Range", meta=(EditCondition="bRangeEnabled", EditConditionHides))
	FRotator RangeRotationMax = FRotator::ZeroRotator;

	UPROPERTY(EditInstanceOnly, Setter="SetRangeScaleUniform", Getter="GetRangeScaleUniform", DisplayName="ScaleUniformEnabled", Category="Range", meta=(EditCondition="bRangeEnabled", EditConditionHides, RefreshPropertyView))
	bool bRangeScaleUniform = true;

	UPROPERTY(EditInstanceOnly, Setter, Getter, DisplayName="ScaleMin", Category="Range", meta=(MotionDesignVectorWidget, AllowPreserveRatio="XYZ", Delta="0.0001", ClampMin="0", EditCondition="bRangeEnabled && !bRangeScaleUniform", EditConditionHides))
	FVector RangeScaleMin = FVector::OneVector;

	UPROPERTY(EditInstanceOnly, Setter, Getter, DisplayName="ScaleMax", Category="Range", meta=(MotionDesignVectorWidget, AllowPreserveRatio="XYZ", Delta="0.0001", ClampMin="0", EditCondition="bRangeEnabled && !bRangeScaleUniform", EditConditionHides))
	FVector RangeScaleMax = FVector::OneVector;

	UPROPERTY(EditInstanceOnly, Setter, Getter, DisplayName="ScaleMin", Category="Range", meta=(Delta="0.0001", ClampMin="0", EditCondition="bRangeEnabled && bRangeScaleUniform", EditConditionHides))
	float RangeScaleUniformMin = 1.f;

	UPROPERTY(EditInstanceOnly, Setter, Getter, DisplayName="ScaleMax", Category="Range", meta=(Delta="0.0001", ClampMin="0", EditCondition="bRangeEnabled && bRangeScaleUniform", EditConditionHides))
	float RangeScaleUniformMax = 1.f;

private:
#if WITH_EDITOR
	/** Used for PECP */
	static const TCEPropertyChangeDispatcher<UCEClonerRangeExtension> PropertyChangeDispatcher;
#endif
};
