// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CEClonerExtensionBase.h"
#include "CEPropertyChangeDispatcher.h"
#include "CEClonerStepExtension.generated.h"

/** Extension dealing with delta step accumulated options */
UCLASS(MinimalAPI, BlueprintType, Within=CEClonerComponent, meta=(Section="Cloner", Priority=20))
class UCEClonerStepExtension : public UCEClonerExtensionBase
{
	GENERATED_BODY()

public:
	UCEClonerStepExtension();

	UFUNCTION(BlueprintCallable, Category="Cloner")
	CLONEREFFECTOR_API void SetDeltaStepEnabled(bool bInEnabled);

	UFUNCTION(BlueprintPure, Category="Cloner")
	bool GetDeltaStepEnabled() const
	{
		return bDeltaStepEnabled;
	}

	UFUNCTION(BlueprintCallable, Category="Cloner")
	CLONEREFFECTOR_API void SetDeltaStepPosition(const FVector& InPosition);

	UFUNCTION(BlueprintPure, Category="Cloner")
	FVector GetDeltaStepPosition() const
	{
		return DeltaStepPosition;
	}

	UFUNCTION(BlueprintCallable, Category="Cloner")
	CLONEREFFECTOR_API void SetDeltaStepRotation(const FRotator& InRotation);

	UFUNCTION(BlueprintPure, Category="Cloner")
	FRotator GetDeltaStepRotation() const
	{
		return DeltaStepRotation;
	}

	UFUNCTION(BlueprintCallable, Category="Cloner")
	CLONEREFFECTOR_API void SetDeltaStepScale(const FVector& InScale);

	UFUNCTION(BlueprintPure, Category="Cloner")
	FVector GetDeltaStepScale() const
	{
		return DeltaStepScale;
	}

protected:
	//~ Begin UObject
#if WITH_EDITOR
	virtual void PostEditChangeProperty(FPropertyChangedEvent& InPropertyChangedEvent) override;
#endif
	//~ End UObject

	//~ Begin UCEClonerExtensionBase
	virtual void OnExtensionParametersChanged(UCEClonerComponent* InComponent) override;
	//~ End UCEClonerExtensionBase

	/** Enable steps to add delta variation on each clone instance */
	UPROPERTY(EditInstanceOnly, Setter="SetDeltaStepEnabled", Getter="GetDeltaStepEnabled", DisplayName="Enabled", Category="Step", meta=(RefreshPropertyView))
	bool bDeltaStepEnabled = false;

	/** Amount of position difference between one step and the next one */
	UPROPERTY(EditInstanceOnly, Setter, Getter, DisplayName="Position", Category="Step", meta=(EditCondition="bDeltaStepEnabled", EditConditionHides))
	FVector DeltaStepPosition = FVector::ZeroVector;

	/** Amount of rotation difference between one step and the next one */
	UPROPERTY(EditInstanceOnly, Setter, Getter, DisplayName="Rotation", Category="Step", meta=(EditCondition="bDeltaStepEnabled", EditConditionHides))
	FRotator DeltaStepRotation = FRotator::ZeroRotator;

	/** Amount of scale difference between one step and the next one */
	UPROPERTY(EditInstanceOnly, Setter, Getter, DisplayName="Scale", Category="Step", meta=(MotionDesignVectorWidget, AllowPreserveRatio="XYZ", Delta="0.0001", EditCondition="bDeltaStepEnabled", EditConditionHides))
	FVector DeltaStepScale = FVector::ZeroVector;

private:
#if WITH_EDITOR
	/** Used for PECP */
	static const TCEPropertyChangeDispatcher<UCEClonerStepExtension> PropertyChangeDispatcher;
#endif
};