// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CEClonerExtensionBase.h"
#include "CEPropertyChangeDispatcher.h"
#include "CEClonerTextureExtension.generated.h"

UENUM()
enum class ECEClonerTextureProvider : uint8
{
	/** Use texture from constraint extension */
	Constraint,
	/** Use texture from displacement extension */
	Displacement,
	/** Use custom texture */
	Custom
};

/** Extension dealing with clone grid texture and UV for material */
UCLASS(MinimalAPI, BlueprintType, Within=CEClonerComponent, meta=(Section="Cloner", Priority=60))
class UCEClonerTextureExtension : public UCEClonerExtensionBase
{
	GENERATED_BODY()

public:
	UCEClonerTextureExtension();

	UFUNCTION(BlueprintCallable, Category="Cloner")
	void SetTextureEnabled(bool bInEnabled);
	
	UFUNCTION(BlueprintPure, Category="Cloner")
	bool GetTextureEnabled() const
	{
		return bTextureEnabled;
	}

	UFUNCTION(BlueprintCallable, Category="Cloner")
	void SetTextureProvider(ECEClonerTextureProvider InProvider);

	UFUNCTION(BlueprintPure, Category="Cloner")
	ECEClonerTextureProvider GetTextureProvider() const
	{
		return TextureProvider;
	}

	UFUNCTION(BlueprintCallable, Category="Cloner")
	void SetCustomTextureAsset(UTexture* InTexture);
	
	UFUNCTION(BlueprintPure, Category="Cloner")
	UTexture* GetCustomTextureAsset() const
	{
		return CustomTextureAsset;
	}

	UFUNCTION(BlueprintCallable, Category="Cloner")
	void SetTextureUVProvider(ECEClonerTextureProvider InProvider);

	UFUNCTION(BlueprintPure, Category="Cloner")
	ECEClonerTextureProvider GetTextureUVProvider() const
	{
		return TextureUVProvider;
	}
	
	UFUNCTION(BlueprintCallable, Category="Cloner")
	void SetCustomTextureUVPlane(ECEClonerPlane InPlane);

	UFUNCTION(BlueprintPure, Category="Cloner")
	ECEClonerPlane GetCustomTextureUVPlane() const
	{
		return CustomTextureUVPlane;
	}

	UFUNCTION(BlueprintCallable, Category="Cloner")
	CLONEREFFECTOR_API void SetCustomTextureUVOffset(const FVector2D& InOffset);

	UFUNCTION(BlueprintPure, Category="Cloner")
	FVector2D GetCustomTextureUVOffset() const
	{
		return CustomTextureUVOffset;
	}

	UFUNCTION(BlueprintCallable, Category="Cloner")
	CLONEREFFECTOR_API void SetCustomTextureUVRotation(float InRotation);

	UFUNCTION(BlueprintPure, Category="Cloner")
	float GetCustomTextureUVRotation() const
	{
		return CustomTextureUVRotation;
	}
	
	UFUNCTION(BlueprintCallable, Category="Cloner")
	CLONEREFFECTOR_API void SetCustomTextureUVScale(const FVector2D& InScale);

	UFUNCTION(BlueprintPure, Category="Cloner")
	FVector2D GetCustomTextureUVScale() const
	{
		return CustomTextureUVScale;
	}
	
	UFUNCTION(BlueprintCallable, Category="Cloner")
	CLONEREFFECTOR_API void SetCustomTextureUVClamp(bool bInClamp);

	UFUNCTION(BlueprintPure, Category="Cloner")
	bool GetCustomTextureUVClamp() const
	{
		return bCustomTextureUVClamp;
	}
	
protected:
	//~ Begin UObject
#if WITH_EDITOR
	virtual void PostEditChangeProperty(FPropertyChangedEvent& InPropertyChangedEvent) override;
#endif
	//~ End UObject

	//~ Begin UCEClonerExtensionBase
	virtual void OnExtensionParametersChanged(UCEClonerComponent* InComponent) override;
	virtual bool IsLayoutSupported(const UCEClonerLayoutBase* InLayout) const override;
	virtual void OnExtensionDirtied(const UCEClonerExtensionBase* InExtension) override;
	//~ End UCEClonerExtensionBase

	UPROPERTY(EditInstanceOnly, Setter="SetTextureEnabled", Getter="GetTextureEnabled", DisplayName="Enabled", Category="Texture", meta=(RefreshPropertyView))
	bool bTextureEnabled = false;

	UPROPERTY(EditInstanceOnly, Setter, Getter, DisplayName="Texture Provider", Category="Texture", meta=(EditCondition="bTextureEnabled", EditConditionHides, RefreshPropertyView))
	ECEClonerTextureProvider TextureProvider = ECEClonerTextureProvider::Constraint;

	UPROPERTY(EditInstanceOnly, Setter, Getter, DisplayName="Texture Asset", Category="Texture", meta=(EditCondition="bTextureEnabled && TextureProvider == ECEClonerTextureProvider::Custom", EditConditionHides))
	TObjectPtr<UTexture> CustomTextureAsset;
	
	UPROPERTY(EditInstanceOnly, Setter, Getter, DisplayName="Texture UV Provider", Category="Texture", meta=(EditCondition="bTextureEnabled", EditConditionHides))
	ECEClonerTextureProvider TextureUVProvider = ECEClonerTextureProvider::Constraint;

	UPROPERTY(EditInstanceOnly, Setter, Getter, DisplayName="Texture UV Plane", Category="Texture", meta=(InvalidEnumValues="Custom", EditCondition="bTextureEnabled && TextureUVProvider == ECEClonerTextureProvider::Custom", EditConditionHides))
	ECEClonerPlane CustomTextureUVPlane = ECEClonerPlane::XY;

	UPROPERTY(EditInstanceOnly, Setter, Getter, DisplayName="Texture UV Offset", Category="Texture", meta=(EditCondition="bTextureEnabled && TextureUVProvider == ECEClonerTextureProvider::Custom", EditConditionHides))
	FVector2D CustomTextureUVOffset = FVector2D::ZeroVector;

	UPROPERTY(EditInstanceOnly, Setter, Getter, DisplayName="Texture UV Rotation", Category="Texture", meta=(EditCondition="bTextureEnabled && TextureUVProvider == ECEClonerTextureProvider::Custom", EditConditionHides))
	float CustomTextureUVRotation = 0.f;

	UPROPERTY(EditInstanceOnly, Setter, Getter, DisplayName="Texture UV Scale", Category="Texture", meta=(Delta="0.01", MotionDesignVectorWidget, AllowPreserveRatio="XY", EditCondition="bTextureEnabled && TextureUVProvider == ECEClonerTextureProvider::Custom", EditConditionHides))
	FVector2D CustomTextureUVScale = FVector2D(1.0);

	UPROPERTY(EditInstanceOnly, Setter="SetCustomTextureUVClamp", Getter="GetCustomTextureUVClamp", DisplayName="Texture UV Clamp", Category="Texture", meta=(EditCondition="bTextureEnabled && TextureUVProvider == ECEClonerTextureProvider::Custom", EditConditionHides))
	bool bCustomTextureUVClamp = false;

private:
#if WITH_EDITOR
	static const TCEPropertyChangeDispatcher<UCEClonerTextureExtension> PropertyChangeDispatcher;
#endif
};