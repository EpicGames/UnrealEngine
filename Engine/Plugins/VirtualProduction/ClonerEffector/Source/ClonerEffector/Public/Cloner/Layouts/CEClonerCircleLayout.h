// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CEClonerEffectorShared.h"
#include "CEClonerLayoutBase.h"
#include "CEPropertyChangeDispatcher.h"
#include "CEClonerCircleLayout.generated.h"

UCLASS(MinimalAPI, BlueprintType)
class UCEClonerCircleLayout : public UCEClonerLayoutBase
{
	GENERATED_BODY()

	friend class FAvaClonerActorVisualizer;

public:
	UCEClonerCircleLayout()
		: UCEClonerLayoutBase(
			TEXT("Circle")
			, TEXT("/Script/Niagara.NiagaraSystem'/ClonerEffector/Systems/NS_ClonerCircle.NS_ClonerCircle'")
		)
	{}

	UFUNCTION(BlueprintCallable, Category="Cloner|Layout|Circle")
	CLONEREFFECTOR_API void SetCount(int32 InCount);

	UFUNCTION(BlueprintPure, Category="Cloner|Layout|Circle")
	int32 GetCount() const
	{
		return Count;
	}

	UFUNCTION(BlueprintCallable, Category="Cloner|Layout|Circle")
	CLONEREFFECTOR_API void SetRingCount(int32 InCount);

	UFUNCTION(BlueprintPure, Category="Cloner|Layout|Circle")
	int32 GetRingCount() const
	{
		return RingCount;
	}

	UFUNCTION(BlueprintCallable, Category="Cloner|Layout|Circle")
	CLONEREFFECTOR_API void SetRadius(float InRadius);

	UFUNCTION(BlueprintPure, Category="Cloner|Layout|Circle")
	float GetRadius() const
	{
		return Radius;
	}

	UFUNCTION(BlueprintCallable, Category="Cloner|Layout|Circle")
	CLONEREFFECTOR_API void SetAngleStart(float InAngleStart);

	UFUNCTION(BlueprintPure, Category="Cloner|Layout|Circle")
	float GetAngleStart() const
	{
		return AngleStart;
	}

	UFUNCTION(BlueprintCallable, Category="Cloner|Layout|Circle")
	CLONEREFFECTOR_API void SetAngleRatio(float InAngleRatio);

	UFUNCTION(BlueprintPure, Category="Cloner|Layout|Circle")
	float GetAngleRatio() const
	{
		return AngleRatio;
	}

	UFUNCTION(BlueprintCallable, Category="Cloner|Layout|Circle")
	CLONEREFFECTOR_API void SetOrientMesh(bool bInOrientMesh);

	UFUNCTION(BlueprintPure, Category="Cloner|Layout|Circle")
	bool GetOrientMesh() const
	{
		return bOrientMesh;
	}

	UFUNCTION(BlueprintCallable, Category="Cloner|Layout|Circle")
	CLONEREFFECTOR_API void SetPlane(ECEClonerPlane InPlane);

	UFUNCTION(BlueprintPure, Category="Cloner|Layout|Circle")
	ECEClonerPlane GetPlane() const
	{
		return Plane;
	}

	UFUNCTION(BlueprintCallable, Category="Cloner|Layout|Circle")
	CLONEREFFECTOR_API void SetRotation(const FRotator& InRotation);

	UFUNCTION(BlueprintPure, Category="Cloner|Layout|Circle")
	const FRotator& GetRotation() const
	{
		return Rotation;
	}

	UFUNCTION(BlueprintCallable, Category="Cloner|Layout|Circle")
	CLONEREFFECTOR_API void SetScale(const FVector& InScale);

	UFUNCTION(BlueprintPure, Category="Cloner|Layout|Circle")
	const FVector& GetScale() const
	{
		return Scale;
	}

protected:
	//~ Begin UObject
#if WITH_EDITOR
	virtual void PostEditChangeProperty(FPropertyChangedEvent& InPropertyChangedEvent) override;
#endif
	//~ End UObject

	//~ Begin UCEClonerLayoutBase
	virtual void OnLayoutParametersChanged(UCEClonerComponent* InComponent) override;
	//~ End UCEClonerLayoutBase

	UPROPERTY(EditInstanceOnly, Setter, Getter, Category="Layout")
	int32 Count = 3 * 3;

	UPROPERTY(EditInstanceOnly, Setter, Getter, Category="Layout", meta=(ClampMin="1"))
	int32 RingCount = 1;

	UPROPERTY(EditInstanceOnly, Setter, Getter, Category="Layout")
	float Radius = 200.f;

	UPROPERTY(EditInstanceOnly, Setter, Getter, Category="Layout")
	float AngleStart = 0.f;

	UPROPERTY(EditInstanceOnly, Setter, Getter, Category="Layout")
	float AngleRatio = 1.f;

	UPROPERTY(EditInstanceOnly, Setter="SetOrientMesh", Getter="GetOrientMesh", Category="Layout")
	bool bOrientMesh = true;

	UPROPERTY(EditInstanceOnly, Setter, Getter, Category="Layout", meta=(RefreshPropertyView))
	ECEClonerPlane Plane = ECEClonerPlane::XY;

	UPROPERTY(EditInstanceOnly, Setter, Getter, Category="Layout", meta=(EditCondition="Plane == ECEClonerPlane::Custom", EditConditionHides))
	FRotator Rotation = FRotator(0.f, 0.f, 0.f);

	UPROPERTY(EditInstanceOnly, Setter, Getter, Category="Layout", meta=(ClampMin="0", MotionDesignVectorWidget, AllowPreserveRatio="XYZ", Delta="0.01"))
	FVector Scale = FVector(1.f, 1.f, 1.f);

private:
#if WITH_EDITOR
	/** Used for PECP */
	static const TCEPropertyChangeDispatcher<UCEClonerCircleLayout> PropertyChangeDispatcher;
#endif
};
