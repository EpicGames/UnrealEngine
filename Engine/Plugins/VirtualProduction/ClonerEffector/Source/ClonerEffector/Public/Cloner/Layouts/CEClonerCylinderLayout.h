// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CEClonerEffectorShared.h"
#include "CEClonerLayoutBase.h"
#include "CEPropertyChangeDispatcher.h"
#include "CEClonerCylinderLayout.generated.h"

UCLASS(MinimalAPI, BlueprintType)
class UCEClonerCylinderLayout : public UCEClonerLayoutBase
{
	GENERATED_BODY()

	friend class FAvaClonerActorVisualizer;

public:
	UCEClonerCylinderLayout()
		: UCEClonerLayoutBase(
			TEXT("Cylinder")
			, TEXT("/Script/Niagara.NiagaraSystem'/ClonerEffector/Systems/NS_ClonerCylinder.NS_ClonerCylinder'")
		)
	{}

	UFUNCTION(BlueprintCallable, Category="Cloner|Layout|Cylinder")
	CLONEREFFECTOR_API void SetBaseCount(int32 InBaseCount);

	UFUNCTION(BlueprintPure, Category="Cloner|Layout|Cylinder")
	int32 GetBaseCount() const
	{
		return BaseCount;
	}

	UFUNCTION(BlueprintCallable, Category="Cloner|Layout|Cylinder")
	CLONEREFFECTOR_API void SetHeightCount(int32 InHeightCount);

	UFUNCTION(BlueprintPure, Category="Cloner|Layout|Cylinder")
	int32 GetHeightCount() const
	{
		return HeightCount;
	}

	UFUNCTION(BlueprintCallable, Category="Cloner|Layout|Cylinder")
	CLONEREFFECTOR_API void SetHeight(float InHeight);

	UFUNCTION(BlueprintPure, Category="Cloner|Layout|Cylinder")
	float GetHeight() const
	{
		return Height;
	}

	UFUNCTION(BlueprintCallable, Category="Cloner|Layout|Cylinder")
	CLONEREFFECTOR_API void SetRadius(float InRadius);

	UFUNCTION(BlueprintPure, Category="Cloner|Layout|Cylinder")
	float GetRadius() const
	{
		return Radius;
	}

	UFUNCTION(BlueprintCallable, Category="Cloner|Layout|Cylinder")
	CLONEREFFECTOR_API void SetAngleStart(float InAngleStart);

	UFUNCTION(BlueprintPure, Category="Cloner|Layout|Cylinder")
	float GetAngleStart() const
	{
		return AngleStart;
	}

	UFUNCTION(BlueprintCallable, Category="Cloner|Layout|Cylinder")
	CLONEREFFECTOR_API void SetAngleRatio(float InAngleRatio);

	UFUNCTION(BlueprintPure, Category="Cloner|Layout|Cylinder")
	float GetAngleRatio() const
	{
		return AngleRatio;
	}

	UFUNCTION(BlueprintCallable, Category="Cloner|Layout|Cylinder")
	CLONEREFFECTOR_API void SetOrientMesh(bool bInOrientMesh);

	UFUNCTION(BlueprintPure, Category="Cloner|Layout|Cylinder")
	bool GetOrientMesh() const
	{
		return bOrientMesh;
	}

	UFUNCTION(BlueprintCallable, Category="Cloner|Layout|Cylinder")
	CLONEREFFECTOR_API void SetPlane(ECEClonerPlane InPlane);

	UFUNCTION(BlueprintPure, Category="Cloner|Layout|Cylinder")
	ECEClonerPlane GetPlane() const
	{
		return Plane;
	}

	UFUNCTION(BlueprintCallable, Category="Cloner|Layout|Cylinder")
	CLONEREFFECTOR_API void SetRotation(const FRotator& InRotation);

	UFUNCTION(BlueprintPure, Category="Cloner|Layout|Cylinder")
	const FRotator& GetRotation() const
	{
		return Rotation;
	}

	UFUNCTION(BlueprintCallable, Category="Cloner|Layout|Cylinder")
	CLONEREFFECTOR_API void SetScale(const FVector& InScale);

	UFUNCTION(BlueprintPure, Category="Cloner|Layout|Cylinder")
	const FVector& GetScale() const
	{
		return Scale;
	}

protected:
	//~ Begin UObject
#if WITH_EDITOR
	virtual void PostEditChangeProperty(FPropertyChangedEvent& InPropertyChangedEvent) override;
#endif
	//~ End UObject

	//~ Begin UCEClonerLayoutBase
	virtual void OnLayoutParametersChanged(UCEClonerComponent* InComponent) override;
	//~ End UCEClonerLayoutBase

	UPROPERTY(EditInstanceOnly, Setter, Getter, Category="Layout")
	int32 BaseCount = 3 * 3;

	UPROPERTY(EditInstanceOnly, Setter, Getter, Category="Layout")
	int32 HeightCount = 3;

	UPROPERTY(EditInstanceOnly, Setter, Getter, Category="Layout")
	float Height = 400.f;

	UPROPERTY(EditInstanceOnly, Setter, Getter, Category="Layout")
	float Radius = 200.f;

	UPROPERTY(EditInstanceOnly, Setter, Getter, Category="Layout")
	float AngleStart = 0.f;

	UPROPERTY(EditInstanceOnly, Setter, Getter, Category="Layout")
	float AngleRatio = 1.f;

	UPROPERTY(EditInstanceOnly, Setter="SetOrientMesh", Getter="GetOrientMesh", Category="Layout")
	bool bOrientMesh = true;

	UPROPERTY(EditInstanceOnly, Setter, Getter, Category="Layout", meta=(RefreshPropertyView))
	ECEClonerPlane Plane = ECEClonerPlane::XY;

	UPROPERTY(EditInstanceOnly, Setter, Getter, Category="Layout", meta=(EditCondition="Plane == ECEClonerPlane::Custom", EditConditionHides))
	FRotator Rotation = FRotator(0.f, 0.f, 0.f);

	UPROPERTY(EditInstanceOnly, Setter, Getter, Category="Layout", meta=(ClampMin="0", MotionDesignVectorWidget, AllowPreserveRatio="XYZ", Delta="0.01"))
	FVector Scale = FVector(1.f, 1.f, 1.f);

private:
#if WITH_EDITOR
	/** Used for PECP */
	static const TCEPropertyChangeDispatcher<UCEClonerCylinderLayout> PropertyChangeDispatcher;
#endif
};