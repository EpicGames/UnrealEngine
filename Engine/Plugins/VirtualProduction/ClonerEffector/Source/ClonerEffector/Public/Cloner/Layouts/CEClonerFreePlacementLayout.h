// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CEClonerEffectorShared.h"
#include "CEClonerLayoutBase.h"
#include "CEPropertyChangeDispatcher.h"
#include "CEClonerFreePlacementLayout.generated.h"

UCLASS(MinimalAPI, BlueprintType)
class UCEClonerFreePlacementLayout : public UCEClonerLayoutBase
{
	GENERATED_BODY()

public:
	UCEClonerFreePlacementLayout()
		: UCEClonerLayoutBase(
			TEXT("FreePlacement")
			, TEXT("/Script/Niagara.NiagaraSystem'/ClonerEffector/Systems/NS_ClonerFreePlacement.NS_ClonerFreePlacement'")
		)
	{}

protected:
	//~ Begin UObject
#if WITH_EDITOR
	virtual void PostEditChangeProperty(FPropertyChangedEvent& InPropertyChangedEvent) override;
#endif
	//~ End UObject

	//~ Begin UCEClonerLayoutBase
	virtual void OnLayoutActive() override;
	virtual void OnLayoutInactive() override;
	virtual void OnLayoutParametersChanged(UCEClonerComponent* InComponent) override;
	//~ End UCEClonerLayoutBase
	
	void OnClonerMeshUpdated(UCEClonerComponent* InClonerComponent);

#if WITH_EDITOR
	void OnClonerActorAttached(UCEClonerComponent* InClonerComponent, AActor* InActor);
	void OnClonerActorDetached(UCEClonerComponent* InClonerComponent, AActor* InActor);
	void ApplyComponentsSettings();
	void RestoreComponentsSettings();
	void ApplyComponentsSettings(AActor* InActor);
	void RestoreComponentsSettings(AActor* InActor);
#endif
	
private:
#if WITH_EDITOR
	/** Used for PECP */
	static const TCEPropertyChangeDispatcher<UCEClonerFreePlacementLayout> PropertyChangeDispatcher;
#endif
};