// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CEClonerEffectorShared.h"
#include "CEClonerLayoutBase.h"
#include "CEPropertyChangeDispatcher.h"
#include "CEClonerGridLayout.generated.h"

UCLASS(MinimalAPI, BlueprintType)
class UCEClonerGridLayout : public UCEClonerLayoutBase
{
	GENERATED_BODY()

	friend class FAvaClonerActorVisualizer;

public:
	UCEClonerGridLayout()
		: UCEClonerLayoutBase(
			TEXT("Grid")
			, TEXT("/Script/Niagara.NiagaraSystem'/ClonerEffector/Systems/NS_ClonerGrid.NS_ClonerGrid'")
		)
	{}

	UFUNCTION(BlueprintCallable, Category="Cloner|Layout|Grid")
	CLONEREFFECTOR_API void SetCountX(int32 InCountX);

	UFUNCTION(BlueprintPure, Category="Cloner|Layout|Grid")
	int32 GetCountX() const
	{
		return CountX;
	}

	UFUNCTION(BlueprintCallable, Category="Cloner|Layout|Grid")
	CLONEREFFECTOR_API void SetCountY(int32 InCountY);

	UFUNCTION(BlueprintPure, Category="Cloner|Layout|Grid")
	int32 GetCountY() const
	{
		return CountY;
	}

	UFUNCTION(BlueprintCallable, Category="Cloner|Layout|Grid")
	CLONEREFFECTOR_API void SetCountZ(int32 InCountZ);

	UFUNCTION(BlueprintPure, Category="Cloner|Layout|Grid")
	int32 GetCountZ() const
	{
		return CountZ;
	}

	UFUNCTION(BlueprintCallable, Category="Cloner|Layout|Grid")
	CLONEREFFECTOR_API void SetSpacingX(float InSpacingX);

	UFUNCTION(BlueprintPure, Category="Cloner|Layout|Grid")
	float GetSpacingX() const
	{
		return SpacingX;
	}

	UFUNCTION(BlueprintCallable, Category="Cloner|Layout|Grid")
	CLONEREFFECTOR_API void SetSpacingY(float InSpacingY);

	UFUNCTION(BlueprintPure, Category="Cloner|Layout|Grid")
	float GetSpacingY() const
	{
		return SpacingY;
	}

	UFUNCTION(BlueprintCallable, Category="Cloner|Layout|Grid")
	CLONEREFFECTOR_API void SetSpacingZ(float InSpacingZ);

	UFUNCTION(BlueprintPure, Category="Cloner|Layout|Grid")
	float GetSpacingZ() const
	{
		return SpacingZ;
	}

	UFUNCTION(BlueprintCallable, Category="Cloner|Layout|Grid")
	CLONEREFFECTOR_API void SetTwistFactor(float InFactor);

	UFUNCTION(BlueprintPure, Category="Cloner|Layout|Grid")
	float GetTwistFactor() const
	{
		return TwistFactor;
	}

	UFUNCTION(BlueprintCallable, Category="Cloner|Layout|Grid")
	CLONEREFFECTOR_API void SetTwistAxis(ECEClonerAxis InAxis);

	UFUNCTION(BlueprintPure, Category="Cloner|Layout|Grid")
	ECEClonerAxis GetTwistAxis() const
	{
		return TwistAxis;
	}

	PRAGMA_DISABLE_DEPRECATION_WARNINGS

	ECEClonerGridConstraint GetConstraint() const
	{
		return Constraint;
	}

	bool GetInvertConstraint() const
	{
		return bInvertConstraint;
	}

	const FCEClonerGridConstraintSphere& GetSphereConstraint() const
	{
		return SphereConstraint;
	}

	const FCEClonerGridConstraintCylinder& GetCylinderConstraint() const
	{
		return CylinderConstraint;
	}

	const FCEClonerGridConstraintTexture& GetTextureConstraint() const
	{
		return TextureConstraint;
	}

	PRAGMA_ENABLE_DEPRECATION_WARNINGS

protected:
	//~ Begin UObject
#if WITH_EDITOR
	virtual void PostEditChangeProperty(FPropertyChangedEvent& InPropertyChangedEvent) override;
#endif
	//~ End UObject

	//~ Begin UCEClonerLayoutBase
	virtual void OnLayoutParametersChanged(UCEClonerComponent* InComponent) override;
	//~ End UCEClonerLayoutBase

	UPROPERTY(EditInstanceOnly, Setter, Getter, Category="Layout")
	int32 CountX = 3;

	UPROPERTY(EditInstanceOnly, Setter, Getter, Category="Layout")
	int32 CountY = 3;

	UPROPERTY(EditInstanceOnly, Setter, Getter, Category="Layout")
	int32 CountZ = 3;

	UPROPERTY(EditInstanceOnly, Setter, Getter, Category="Layout")
	float SpacingX = 105.f;

	UPROPERTY(EditInstanceOnly, Setter, Getter, Category="Layout")
	float SpacingY = 105.f;

	UPROPERTY(EditInstanceOnly, Setter, Getter, Category="Layout")
	float SpacingZ = 105.f;

	UPROPERTY(EditInstanceOnly, Setter, Getter, Category="Layout", meta=(UIMin="0", UIMax="100"))
	float TwistFactor = 0.f;

	UPROPERTY(EditInstanceOnly, Setter, Getter, Category="Layout", meta=(InvalidEnumValues="Custom"))
	ECEClonerAxis TwistAxis = ECEClonerAxis::Y;

private:
	UE_DEPRECATED(5.5, "Moved to constraint extension")
	UPROPERTY()
	ECEClonerGridConstraint Constraint = ECEClonerGridConstraint::None;

	UE_DEPRECATED(5.5, "Moved to constraint extension")
	UPROPERTY()
	bool bInvertConstraint = false;

	UE_DEPRECATED(5.5, "Moved to constraint extension")
	UPROPERTY()
	FCEClonerGridConstraintSphere SphereConstraint;

	UE_DEPRECATED(5.5, "Moved to constraint extension")
	UPROPERTY()
	FCEClonerGridConstraintCylinder CylinderConstraint;

	UE_DEPRECATED(5.5, "Moved to constraint extension")
	UPROPERTY()
	FCEClonerGridConstraintTexture TextureConstraint;

#if WITH_EDITOR
	/** Used for PECP */
	static const TCEPropertyChangeDispatcher<UCEClonerGridLayout> PropertyChangeDispatcher;
#endif
};