// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CEClonerEffectorShared.h"
#include "CEClonerLayoutBase.h"
#include "CEPropertyChangeDispatcher.h"
#include "CEClonerHoneycombLayout.generated.h"

UCLASS(MinimalAPI, BlueprintType)
class UCEClonerHoneycombLayout : public UCEClonerLayoutBase
{
	GENERATED_BODY()

	friend class FAvaClonerActorVisualizer;

public:
	UCEClonerHoneycombLayout()
		: UCEClonerLayoutBase(
			TEXT("Honeycomb")
			, TEXT("/Script/Niagara.NiagaraSystem'/ClonerEffector/Systems/NS_ClonerHoneycomb.NS_ClonerHoneycomb'")
		)
	{}

	UFUNCTION(BlueprintCallable, Category="Cloner|Layout|Honeycomb")
	CLONEREFFECTOR_API void SetPlane(ECEClonerPlane InPlane);

	UFUNCTION(BlueprintPure, Category="Cloner|Layout|Honeycomb")
	ECEClonerPlane GetPlane() const
	{
		return Plane;
	}

	UFUNCTION(BlueprintCallable, Category="Cloner|Layout|Honeycomb")
	CLONEREFFECTOR_API void SetWidthCount(int32 InWidthCount);

	UFUNCTION(BlueprintPure, Category="Cloner|Layout|Honeycomb")
	int32 GetWidthCount() const
	{
		return WidthCount;
	}

	UFUNCTION(BlueprintCallable, Category="Cloner|Layout|Honeycomb")
	CLONEREFFECTOR_API void SetHeightCount(int32 InHeightCount);

	UFUNCTION(BlueprintPure, Category="Cloner|Layout|Honeycomb")
	int32 GetHeightCount() const
	{
		return HeightCount;
	}

	UFUNCTION(BlueprintCallable, Category="Cloner|Layout|Honeycomb")
	CLONEREFFECTOR_API void SetWidthOffset(float InWidthOffset);

	UFUNCTION(BlueprintPure, Category="Cloner|Layout|Honeycomb")
	float GetWidthOffset() const
	{
		return WidthOffset;
	}

	UFUNCTION(BlueprintCallable, Category="Cloner|Layout|Honeycomb")
	CLONEREFFECTOR_API void SetHeightOffset(float InHeightOffset);

	UFUNCTION(BlueprintPure, Category="Cloner|Layout|Honeycomb")
	float GetHeightOffset() const
	{
		return HeightOffset;
	}

	UFUNCTION(BlueprintCallable, Category="Cloner|Layout|Honeycomb")
	CLONEREFFECTOR_API void SetHeightSpacing(float InHeightSpacing);

	UFUNCTION(BlueprintPure, Category="Cloner|Layout|Honeycomb")
	float GetHeightSpacing() const
	{
		return HeightSpacing;
	}

	UFUNCTION(BlueprintCallable, Category="Cloner|Layout|Honeycomb")
	CLONEREFFECTOR_API void SetWidthSpacing(float InWidthSpacing);

	UFUNCTION(BlueprintPure, Category="Cloner|Layout|Honeycomb")
	float GetWidthSpacing() const
	{
		return WidthSpacing;
	}

	UFUNCTION(BlueprintCallable, Category="Cloner|Layout|Honeycomb")
	CLONEREFFECTOR_API void SetTwistFactor(float InFactor);

	UFUNCTION(BlueprintPure, Category="Cloner|Layout|Honeycomb")
	float GetTwistFactor() const
	{
		return TwistFactor;
	}

	UFUNCTION(BlueprintCallable, Category="Cloner|Layout|Honeycomb")
	CLONEREFFECTOR_API void SetTwistAxis(ECEClonerAxis InAxis);

	UFUNCTION(BlueprintPure, Category="Cloner|Layout|Honeycomb")
	ECEClonerAxis GetTwistAxis() const
	{
		return TwistAxis;
	}

protected:
	//~ Begin UObject
#if WITH_EDITOR
	virtual void PostEditChangeProperty(FPropertyChangedEvent& InPropertyChangedEvent) override;
#endif
	//~ End UObject

	void OnTwistAxisChanged();

	//~ Begin UCEClonerLayoutBase
	virtual void OnLayoutParametersChanged(UCEClonerComponent* InComponent) override;
	//~ End UCEClonerLayoutBase

	UPROPERTY(EditInstanceOnly, Setter, Getter, Category="Layout", meta=(InvalidEnumValues="Custom"))
	ECEClonerPlane Plane = ECEClonerPlane::XY;

	UPROPERTY(EditInstanceOnly, Setter, Getter, Category="Layout")
	int32 WidthCount = 3;

	UPROPERTY(EditInstanceOnly, Setter, Getter, Category="Layout")
	int32 HeightCount = 3;

	UPROPERTY(EditInstanceOnly, Setter, Getter, Category="Layout")
	float WidthOffset = 0.f;

	UPROPERTY(EditInstanceOnly, Setter, Getter, Category="Layout")
	float HeightOffset = 0.5f;

	UPROPERTY(EditInstanceOnly, Setter, Getter, Category="Layout")
	float WidthSpacing = 105.f;

	UPROPERTY(EditInstanceOnly, Setter, Getter, Category="Layout")
	float HeightSpacing = 105.f;

	UPROPERTY(EditInstanceOnly, Setter, Getter, Category="Layout", meta=(UIMin="0", UIMax="100"))
	float TwistFactor = 0.f;

	UPROPERTY(EditInstanceOnly, Setter, Getter, Category="Layout", meta=(InvalidEnumValues="Custom"))
	ECEClonerAxis TwistAxis = ECEClonerAxis::Y;

private:
#if WITH_EDITOR
	/** Used for PECP */
	static const TCEPropertyChangeDispatcher<UCEClonerHoneycombLayout> PropertyChangeDispatcher;
#endif
};