// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CEClonerEffectorShared.h"
#include "Templates/SubclassOf.h"
#include "UObject/Object.h"
#include "CEClonerLayoutBase.generated.h"

class UCEClonerComponent;
class UCEClonerExtensionBase;
class UNiagaraMeshRendererProperties;
class UNiagaraSystem;

/**
 * Base class for layouts available in the cloner actor
 * Steps to add a new layout :
 * 1. Create a new system that extends from NS_ClonerBase and expose all the parent parameters (examples can be found in Content)
 * 2. Extend this layout class and give it a unique name with the newly created system path
 * 3. Expose all new system specific parameters in the layout extended class and update them when required
 * Your new layout is ready and will be available in the cloner in the layout dropdown
 */
UCLASS(MinimalAPI, Abstract, BlueprintType, AutoExpandCategories=("Layout"))
class UCEClonerLayoutBase : public UObject
{
	GENERATED_BODY()

	DECLARE_MULTICAST_DELEGATE_TwoParams(FOnClonerLayoutLoaded, UCEClonerLayoutBase* /** InLayout */, bool /** bInSuccess */)

public:
	static constexpr TCHAR LayoutBaseAssetPath[] = TEXT("/Script/Niagara.NiagaraSystem'/ClonerEffector/Systems/NS_ClonerBase.NS_ClonerBase'");

	UCEClonerLayoutBase()
		: UCEClonerLayoutBase(NAME_None, FString())
	{}

	UCEClonerLayoutBase(const FName& InLayoutName, const FString& InLayoutAssetPath)
		: LayoutName(InLayoutName)
		, LayoutAssetPath(InLayoutAssetPath)
	{}

	UFUNCTION(BlueprintPure, Category="Cloner")
	FName GetLayoutName() const
	{
		return LayoutName;
	}

	const FString& GetLayoutAssetPath() const
	{
		return LayoutAssetPath;
	}

	UNiagaraSystem* GetSystem() const
	{
		return NiagaraSystem;
	}

	UNiagaraMeshRendererProperties* GetMeshRenderer() const
	{
		return MeshRenderer;
	}

	FOnClonerLayoutLoaded::RegistrationType& OnLayoutLoadedDelegate()
	{
		return OnClonerLayoutLoadedDelegate;
	}

	/** Get the cloner component using this layout */
	UCEClonerComponent* GetClonerComponent() const;

	/** Get the actor using this layout */
	AActor* GetClonerActor() const;

	/** Updates all parameters handled by this layout */
	void UpdateLayoutParameters();

	/** Checks if the niagara system asset is valid and usable with the cloner */
	bool IsLayoutValid() const;

	/** Is this layout system cached and ready to be used */
	bool IsLayoutLoaded() const;

	/** Load this layout system if not already loaded */
	void LoadLayout();

	/** Free the loaded system and return to idle state */
	bool UnloadLayout();

	/** Is this layout system in use within the cloner */
	UFUNCTION(BlueprintPure, Category="Cloner|Layout")
	CLONEREFFECTOR_API bool IsLayoutActive() const;

	/** Activate this layout system on the cloner, must be loaded first */
	bool ActivateLayout();

	/** Deactivate this layout system if active */
	bool DeactivateLayout();

	/** Gets the cloner extensions supported by this layout */
	TSet<TSubclassOf<UCEClonerExtensionBase>> GetSupportedExtensions() const;

	/** Filter supported extension for this layout */
	virtual bool IsExtensionSupported(const UCEClonerExtensionBase* InExtension) const
	{
		return true;
	}

	/** Request refresh layout next tick */
	void MarkLayoutDirty(bool bInUpdateCloner = true);

	/** Is the cloner not up to date with layout parameters */
	bool IsLayoutDirty() const;

protected:
	//~ Begin UObject
	virtual void PostEditImport() override;
	virtual void PostLoad() override;
#if WITH_EDITOR
	virtual void PostEditUndo() override;
#endif
	//~ End UObject

	/** Called once after layout is loaded */
	virtual void OnLayoutLoaded() {}

	/** Called once after layout is unloaded */
	virtual void OnLayoutUnloaded() {}

	/** Called after layout becomes active */
	virtual void OnLayoutActive() {}

	/** Called after layout becomes inactive */
	virtual void OnLayoutInactive() {}

	/** Called to reapply layout parameters */
	virtual void OnLayoutParametersChanged(UCEClonerComponent* InComponent) {}

	void OnLayoutPropertyChanged();

private:
	FOnClonerLayoutLoaded OnClonerLayoutLoadedDelegate;

	/** Called when the system package was async loaded */
	void OnSystemPackageLoaded(const FName& InName, UPackage* InPackage, EAsyncLoadingResult::Type InResult);
	void OnSystemLoaded();

	/** Finds and cache first mesh renderer in emitter */
	void CacheMeshRenderer();

	/** Bind delegates to clear resources during level or world cleanup */
	void BindCleanupDelegates();
	void UnbindCleanupDelegates() const;

	/** When level is unloaded or world cleaned up, deactivate and unload layout */
	void OnWorldCleanup(UWorld* InWorld, bool bInSessionEnded, bool bInCleanupResources);
	void OnLevelCleanup();

	/** Marks niagara system owned by this layout garbage to avoid GC leak */
	void CleanOwnedSystem() const;

#if WITH_EDITOR
	/** Get the niagara system package hash */
	FString GetLayoutHash() const;
#endif

	/** Checks if cached hash matches package hash */
	bool IsSystemHashMatching() const;

	/** Layout name to display in layout options */
	UPROPERTY(Transient)
	FName LayoutName;

	/** Niagara System asset path for this layout */
	UPROPERTY(Transient)
	FString LayoutAssetPath;

	/** Niagara system used for this layout, cached to save some time */
	UPROPERTY(DuplicateTransient, TextExportTransient)
	TObjectPtr<UNiagaraSystem> NiagaraSystem;

	/** Mesh renderer in this niagara system */
	UPROPERTY(Transient, DuplicateTransient, TextExportTransient)
	TObjectPtr<UNiagaraMeshRendererProperties> MeshRenderer;

	/** Hash of the cached system for diffs */
	UPROPERTY(DuplicateTransient, TextExportTransient)
	FString CachedSystemHash;

	/** Id for the load request initiated */
	int32 LoadRequestIdentifier = INDEX_NONE;

	/** Status for this layout */
	ECEClonerSystemStatus LayoutStatus = ECEClonerSystemStatus::UpToDate;
};