// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CEClonerEffectorShared.h"
#include "CEClonerLayoutBase.h"
#include "CEPropertyChangeDispatcher.h"
#include "CEClonerLineLayout.generated.h"

UCLASS(MinimalAPI, BlueprintType)
class UCEClonerLineLayout : public UCEClonerLayoutBase
{
	GENERATED_BODY()

	friend class FAvaClonerActorVisualizer;

public:
	UCEClonerLineLayout()
		: UCEClonerLayoutBase(
			TEXT("Line")
			, TEXT("/Script/Niagara.NiagaraSystem'/ClonerEffector/Systems/NS_ClonerLine.NS_ClonerLine'")
		)
	{}

	UFUNCTION(BlueprintCallable, Category="Cloner|Layout|Line")
	CLONEREFFECTOR_API void SetCount(int32 InCount);

	UFUNCTION(BlueprintPure, Category="Cloner|Layout|Line")
	int32 GetCount() const
	{
		return Count;
	}

	UFUNCTION(BlueprintCallable, Category="Cloner|Layout|Line")
	CLONEREFFECTOR_API void SetSpacing(float InSpacing);

	UFUNCTION(BlueprintPure, Category="Cloner|Layout|Line")
	float GetSpacing() const
	{
		return Spacing;
	}

	UFUNCTION(BlueprintCallable, Category="Cloner|Layout|Line")
	CLONEREFFECTOR_API void SetAxis(ECEClonerAxis InAxis);

	UFUNCTION(BlueprintPure, Category="Cloner|Layout|Line")
	ECEClonerAxis GetAxis() const
	{
		return Axis;
	}

	UFUNCTION(BlueprintCallable, Category="Cloner|Layout|Line")
	CLONEREFFECTOR_API void SetDirection(const FVector& InDirection);

	UFUNCTION(BlueprintPure, Category="Cloner|Layout|Line")
	const FVector& GetDirection() const
	{
		return Direction;
	}

	UFUNCTION(BlueprintCallable, Category="Cloner|Layout|Line")
	CLONEREFFECTOR_API void SetRotation(const FRotator& InRotation);

	UFUNCTION(BlueprintPure, Category="Cloner|Layout|Line")
	const FRotator& GetRotation() const
	{
		return Rotation;
	}

protected:
	//~ Begin UObject
#if WITH_EDITOR
	virtual void PostEditChangeProperty(FPropertyChangedEvent& InPropertyChangedEvent) override;
#endif
	//~ End UObject

	//~ Begin UCEClonerLayoutBase
	virtual void OnLayoutParametersChanged(UCEClonerComponent* InComponent) override;
	//~ End UCEClonerLayoutBase

	UPROPERTY(EditInstanceOnly, Setter, Getter, Category="Layout")
    int32 Count = 10;

	UPROPERTY(EditInstanceOnly, Setter, Getter, Category="Layout")
	float Spacing = 105.f;

	UPROPERTY(EditInstanceOnly, Setter, Getter, Category="Layout", meta=(RefreshPropertyView))
	ECEClonerAxis Axis = ECEClonerAxis::Y;

	UPROPERTY(EditInstanceOnly, Setter, Getter, Category="Layout", meta=(ClampMin="0", ClampMax="1", EditCondition="Axis == ECEClonerAxis::Custom", EditConditionHides))
	FVector Direction = FVector::YAxisVector;

	UPROPERTY(EditInstanceOnly, Setter, Getter, Category="Layout")
	FRotator Rotation = FRotator(0.f);

private:
#if WITH_EDITOR
	/** Used for PECP */
	static const TCEPropertyChangeDispatcher<UCEClonerLineLayout> PropertyChangeDispatcher;
#endif
};