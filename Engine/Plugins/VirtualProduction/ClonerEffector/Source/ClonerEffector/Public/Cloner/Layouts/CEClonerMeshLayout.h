// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CEClonerEffectorShared.h"
#include "CEClonerLayoutBase.h"
#include "CEPropertyChangeDispatcher.h"
#include "UObject/WeakObjectPtrTemplates.h"
#include "CEClonerMeshLayout.generated.h"

class AActor;
class USceneComponent;

UCLASS(MinimalAPI, BlueprintType)
class UCEClonerMeshLayout : public UCEClonerLayoutBase
{
	GENERATED_BODY()

	friend class FAvaClonerActorVisualizer;

public:
#if WITH_EDITOR
	static CLONEREFFECTOR_API FName GetSampleActorWeakName();
	static CLONEREFFECTOR_API FName GetAssetName();
#endif

	UCEClonerMeshLayout()
		: UCEClonerLayoutBase(
			TEXT("Mesh")
			, TEXT("/Script/Niagara.NiagaraSystem'/ClonerEffector/Systems/NS_ClonerSampleMesh.NS_ClonerSampleMesh'")
		)
	{}

	UFUNCTION(BlueprintCallable, Category="Cloner|Layout|Mesh")
	CLONEREFFECTOR_API void SetCount(int32 InCount);

	UFUNCTION(BlueprintPure, Category="Cloner|Layout|Mesh")
	int32 GetCount() const
	{
		return Count;
	}

	UFUNCTION(BlueprintCallable, Category="Cloner|Layout|Mesh")
	CLONEREFFECTOR_API void SetAsset(ECEClonerMeshAsset InAsset);

	UFUNCTION(BlueprintPure, Category="Cloner|Layout|Mesh")
	ECEClonerMeshAsset GetAsset() const
	{
		return Asset;
	}

	UFUNCTION(BlueprintCallable, Category="Cloner|Layout|Mesh")
	CLONEREFFECTOR_API void SetSampleData(ECEClonerMeshSampleData InSampleData);

	UFUNCTION(BlueprintPure, Category="Cloner|Layout|Mesh")
	ECEClonerMeshSampleData GetSampleData() const
	{
		return SampleData;
	}

	UFUNCTION()
	CLONEREFFECTOR_API void SetSampleActorWeak(const TWeakObjectPtr<AActor>& InSampleActor);

	TWeakObjectPtr<AActor> GetSampleActorWeak() const
	{
		return SampleActorWeak;
	}

	UFUNCTION(BlueprintCallable, Category="Cloner|Layout|Mesh")
	CLONEREFFECTOR_API void SetSampleActor(AActor* InActor);

	UFUNCTION(BlueprintPure, Category="Cloner|Layout|Mesh")
	AActor* GetSampleActor() const
	{
		return SampleActorWeak.Get();
	}

protected:
	//~ Begin UObject
#if WITH_EDITOR
	virtual void PostEditChangeProperty(FPropertyChangedEvent& InPropertyChangedEvent) override;
#endif
	//~ End UObject

	//~ Begin UCEClonerLayoutBase
	virtual void OnLayoutParametersChanged(UCEClonerComponent* InComponent) override;
	//~ End UCEClonerLayoutBase
	
	UFUNCTION()
	void OnSampleActorDestroyed(AActor* InDestroyedActor);

	UPROPERTY(EditInstanceOnly, Setter, Getter, Category="Layout")
	ECEClonerMeshAsset Asset = ECEClonerMeshAsset::StaticMesh;

	UPROPERTY(EditInstanceOnly, Setter, Getter, DisplayName="Sample Actor", Category="Layout", meta=(DisplayAfter="Asset"))
	TWeakObjectPtr<AActor> SampleActorWeak;

	UPROPERTY(EditInstanceOnly, Setter, Getter, Category="Layout")
	int32 Count = 3 * 3 * 3;

	UPROPERTY(EditInstanceOnly, Setter, Getter, Category="Layout")
	ECEClonerMeshSampleData SampleData = ECEClonerMeshSampleData::Vertices;

private:
	UPROPERTY(Transient)
	TWeakObjectPtr<USceneComponent> SceneComponentWeak;

#if WITH_EDITOR
	/** Used for PECP */
	static const TCEPropertyChangeDispatcher<UCEClonerMeshLayout> PropertyChangeDispatcher;
#endif
};