// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CEClonerLayoutBase.h"
#include "CEPropertyChangeDispatcher.h"
#include "CEClonerSphereRandomLayout.generated.h"

UCLASS(MinimalAPI, BlueprintType)
class UCEClonerSphereRandomLayout : public UCEClonerLayoutBase
{
	GENERATED_BODY()

	friend class FAvaClonerActorVisualizer;

public:
	UCEClonerSphereRandomLayout()
		: UCEClonerLayoutBase(
			TEXT("SphereRandom")
			, TEXT("/Script/Niagara.NiagaraSystem'/ClonerEffector/Systems/NS_ClonerSphereRandom.NS_ClonerSphereRandom'")
		)
	{}

	UFUNCTION(BlueprintCallable, Category="Cloner|Layout|SphereRandom")
	CLONEREFFECTOR_API void SetCount(int32 InCount);

	UFUNCTION(BlueprintPure, Category="Cloner|Layout|SphereRandom")
	int32 GetCount() const
	{
		return Count;
	}

	UFUNCTION(BlueprintCallable, Category="Cloner|Layout|SphereRandom")
	CLONEREFFECTOR_API void SetRadius(float InRadius);

	UFUNCTION(BlueprintPure, Category="Cloner|Layout|SphereRandom")
	float GetRadius() const
	{
		return Radius;
	}

	UFUNCTION(BlueprintCallable, Category="Cloner|Layout|SphereRandom")
	CLONEREFFECTOR_API void SetDistribution(float InDistribution);

	UFUNCTION(BlueprintPure, Category="Cloner|Layout|SphereRandom")
	float GetDistribution() const
	{
		return Distribution;
	}

	UFUNCTION(BlueprintCallable, Category="Cloner|Layout|SphereRandom")
	CLONEREFFECTOR_API void SetLongitude(float InLongitude);

	UFUNCTION(BlueprintPure, Category="Cloner|Layout|SphereRandom")
	float GetLongitude() const
	{
		return Longitude;
	}

	UFUNCTION(BlueprintCallable, Category="Cloner|Layout|SphereRandom")
	CLONEREFFECTOR_API void SetLatitude(float InLatitude);

	UFUNCTION(BlueprintPure, Category="Cloner|Layout|SphereRandom")
	float GetLatitude() const
	{
		return Latitude;
	}

	UFUNCTION(BlueprintCallable, Category="Cloner|Layout|SphereRandom")
	CLONEREFFECTOR_API void SetOrientMesh(bool InbOrientMesh);

	UFUNCTION(BlueprintPure, Category="Cloner|Layout|SphereRandom")
	bool GetOrientMesh() const
	{
		return bOrientMesh;
	}

	UFUNCTION(BlueprintCallable, Category="Cloner|Layout|SphereRandom")
	CLONEREFFECTOR_API void SetRotation(const FRotator& InRotation);

	UFUNCTION(BlueprintPure, Category="Cloner|Layout|SphereRandom")
	FRotator GetRotation() const
	{
		return Rotation;
	}

	UFUNCTION(BlueprintCallable, Category="Cloner|Layout|SphereRandom")
	CLONEREFFECTOR_API void SetScale(const FVector& InScale);

	UFUNCTION(BlueprintPure, Category="Cloner|Layout|SphereRandom")
	FVector GetScale() const
	{
		return Scale;
	}

protected:
	//~ Begin UObject
#if WITH_EDITOR
	virtual void PostEditChangeProperty(FPropertyChangedEvent& InPropertyChangedEvent) override;
#endif
	//~ End UObject

	//~ Begin UCEClonerLayoutBase
	virtual void OnLayoutParametersChanged(UCEClonerComponent* InComponent) override;
	//~ End UCEClonerLayoutBase

	UPROPERTY(EditInstanceOnly, Setter, Getter, Category="Layout")
	int32 Count = 30;

	UPROPERTY(EditInstanceOnly, Setter, Getter, Category="Layout")
	float Radius = 200.f;

	UPROPERTY(EditInstanceOnly, Setter, Getter, Category="Layout", meta=(ClampMin="0", ClampMax="1"))
	float Distribution = 1.f;

	UPROPERTY(EditInstanceOnly, Setter, Getter, Category="Layout", meta=(ClampMin="0", ClampMax="1"))
	float Longitude = 1.f;

	UPROPERTY(EditInstanceOnly, Setter, Getter, Category="Layout", meta=(ClampMin="0", ClampMax="1"))
	float Latitude = 1.f;

	UPROPERTY(EditInstanceOnly, Setter="SetOrientMesh", Getter="GetOrientMesh", Category="Layout")
	bool bOrientMesh = false;

	UPROPERTY(EditInstanceOnly, Setter, Getter, Category="Layout")
	FRotator Rotation = FRotator(0.f, 0.f, 0.f);

	UPROPERTY(EditInstanceOnly, Setter, Getter, Category="Layout", meta=(ClampMin="0", MotionDesignVectorWidget, AllowPreserveRatio="XYZ", Delta="0.01"))
	FVector Scale = FVector(1.f, 1.f, 1.f);

private:
#if WITH_EDITOR
	/** Used for PECP */
	static const TCEPropertyChangeDispatcher<UCEClonerSphereRandomLayout> PropertyChangeDispatcher;
#endif
};