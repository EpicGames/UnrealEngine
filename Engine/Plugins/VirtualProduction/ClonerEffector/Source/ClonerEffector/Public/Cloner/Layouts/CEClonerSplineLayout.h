// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CEClonerLayoutBase.h"
#include "CEPropertyChangeDispatcher.h"
#include "UObject/WeakObjectPtrTemplates.h"
#include "CEClonerSplineLayout.generated.h"

class AActor;
class USplineComponent;

UCLASS(MinimalAPI, BlueprintType)
class UCEClonerSplineLayout : public UCEClonerLayoutBase
{
	GENERATED_BODY()

	friend class FAvaClonerActorVisualizer;

public:
#if WITH_EDITOR
	static CLONEREFFECTOR_API FName GetSplineActorWeakName();
#endif

	UCEClonerSplineLayout()
		: UCEClonerLayoutBase(
			TEXT("Spline")
			, TEXT("/Script/Niagara.NiagaraSystem'/ClonerEffector/Systems/NS_ClonerSampleSpline.NS_ClonerSampleSpline'")
		)
	{}

	UFUNCTION(BlueprintCallable, Category="Cloner|Layout|Spline")
	CLONEREFFECTOR_API void SetCount(int32 InCount);

	UFUNCTION(BlueprintPure, Category="Cloner|Layout|Spline")
	int32 GetCount() const
	{
		return Count;
	}

	UFUNCTION()
	CLONEREFFECTOR_API void SetSplineActorWeak(const TWeakObjectPtr<AActor>& InSplineActor);

	TWeakObjectPtr<AActor> GetSplineActorWeak() const
	{
		return SplineActorWeak;
	}

	UFUNCTION(BlueprintCallable, Category="Cloner|Layout|Spline")
	CLONEREFFECTOR_API void SetSplineActor(AActor* InSplineActor);

	UFUNCTION(BlueprintPure, Category="Cloner|Layout|Spline")
	AActor* GetSplineActor() const
	{
		return SplineActorWeak.Get();
	}

	UFUNCTION(BlueprintCallable, Category="Cloner|Layout|Spline")
	CLONEREFFECTOR_API void SetOrientMesh(bool bInOrientMesh);

	UFUNCTION(BlueprintPure, Category="Cloner|Layout|Spline")
	bool GetOrientMesh() const
	{
		return bOrientMesh;
	}

#if WITH_EDITOR
	/** Spawns an actor with a spline component and linked it to this cloner layout */
	UFUNCTION(CallInEditor, Category="Layout")
	CLONEREFFECTOR_API void SpawnLinkedSplineActor();
#endif

protected:
	//~ Begin UObject
#if WITH_EDITOR
	virtual void PostEditChangeProperty(FPropertyChangedEvent& InPropertyChangedEvent) override;
#endif
	//~ End UObject

	//~ Begin UCEClonerLayoutBase
	virtual void OnLayoutActive() override;
	virtual void OnLayoutInactive() override;
	virtual void OnLayoutParametersChanged(UCEClonerComponent* InComponent) override;
	//~ End UCEClonerLayoutBase

	void OnSampleSplineRenderStateUpdated(UActorComponent& InComponent);

	UPROPERTY(EditInstanceOnly, Setter, Getter, Category="Layout")
	int32 Count = 3 * 3;

	UPROPERTY(EditInstanceOnly, Setter, Getter, DisplayName="SampleActor", Category="Layout")
	TWeakObjectPtr<AActor> SplineActorWeak;

	UPROPERTY(EditInstanceOnly, Setter="SetOrientMesh", Getter="GetOrientMesh", Category="Layout")
    bool bOrientMesh = false;

private:
	UFUNCTION()
	void OnSampleSplineDestroyed(AActor* InDestroyedActor);
	
#if WITH_EDITOR
	/** Used for PECP */
	static const TCEPropertyChangeDispatcher<UCEClonerSplineLayout> PropertyChangeDispatcher;
#endif
};
