// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CEClonerEffectorShared.h"
#include "GameFramework/Actor.h"
#include "CEEffectorActor.generated.h"

class ACEClonerActor;
class UCEEffectorComponent;
class UDynamicMeshComponent;

UCLASS(
	MinimalAPI
	, BlueprintType
	, HideCategories=(Rendering,Replication,Collision,HLOD,Physics,Networking,Input,Actor,Cooking,LevelInstance,DataLayers,WorldPartition)
	, DisplayName = "Motion Design Effector Actor")
class ACEEffectorActor : public AActor
{
	GENERATED_BODY()

	friend class ACEClonerActor;
	friend class FCEEditorEffectorDetailCustomization;
	friend class FAvaEffectorActorVisualizer;
	friend class UCEEffectorSubsystem;

public:
	static inline const FString DefaultLabel = TEXT("Effector");

	ACEEffectorActor();

	UFUNCTION(BlueprintPure, Category="Effector")
	UCEEffectorComponent* GetEffectorComponent() const
	{
		return EffectorComponent;
	}

	//~ Begin AActor
#if WITH_EDITOR
	virtual FString GetDefaultActorLabel() const override;
#endif
	//~ End AActor

protected:
	//~ Begin UObject
	virtual void Serialize(FArchive& InArchive) override;
	virtual void PostLoad() override;
	//~ End UObject

private:
	void MigrateDeprecatedProperties();
	void RegisterToChannel() const;

	UPROPERTY(VisibleInstanceOnly, Category="Effector")
	TObjectPtr<UCEEffectorComponent> EffectorComponent;

	int32 MigrateToVersion = INDEX_NONE;

	UE_DEPRECATED(5.5, "Moved to component and extensions")
	UPROPERTY()
	bool bEnabled = true;

	UE_DEPRECATED(5.5, "Moved to component and extensions")
	UPROPERTY()
	float Magnitude = 1.f;

	UE_DEPRECATED(5.5, "Moved to component and extensions")
	UPROPERTY()
	FLinearColor Color = FLinearColor::Red;

	UE_DEPRECATED(5.5, "Moved to component and extensions")
	UPROPERTY()
	ECEClonerEffectorType Type = ECEClonerEffectorType::Sphere;

	UE_DEPRECATED(5.5, "Moved to component and extensions")
	UPROPERTY()
	bool bInvertType = false;

	UE_DEPRECATED(5.5, "Moved to component and extensions")
	UPROPERTY()
	ECEClonerEasing Easing = ECEClonerEasing::Linear;

	UE_DEPRECATED(5.5, "Moved to component and extensions")
	UPROPERTY()
	float InnerRadius = 50.f;

	UE_DEPRECATED(5.5, "Moved to component and extensions")
	UPROPERTY()
	float OuterRadius = 200.f;

	UE_DEPRECATED(5.5, "Moved to component and extensions")
	UPROPERTY()
	FVector InnerExtent = FVector(50.f);

	UE_DEPRECATED(5.5, "Moved to component and extensions")
	UPROPERTY()
	FVector OuterExtent = FVector(200.f);

	UE_DEPRECATED(5.5, "Moved to component and extensions")
	UPROPERTY()
	float PlaneSpacing = 200.f;

	UE_DEPRECATED(5.5, "Moved to component and extensions")
	UPROPERTY()
	float RadialAngle = 180.f;

	UE_DEPRECATED(5.5, "Moved to component and extensions")
	UPROPERTY()
	float RadialMinRadius = 0.f;

	UE_DEPRECATED(5.5, "Moved to component and extensions")
	UPROPERTY()
	float RadialMaxRadius = 1000.f;

	UE_DEPRECATED(5.5, "Moved to component and extensions")
	UPROPERTY()
	float TorusRadius = 250.f;

	UE_DEPRECATED(5.5, "Moved to component and extensions")
	UPROPERTY()
	float TorusInnerRadius = 50.f;

	UE_DEPRECATED(5.5, "Moved to component and extensions")
	UPROPERTY()
	float TorusOuterRadius = 200.f;

	UE_DEPRECATED(5.5, "Moved to component and extensions")
	UPROPERTY()
	ECEClonerEffectorMode Mode = ECEClonerEffectorMode::Default;

	UE_DEPRECATED(5.5, "Moved to component and extensions")
	UPROPERTY()
	FVector Offset = FVector::ZeroVector;

	UE_DEPRECATED(5.5, "Moved to component and extensions")
	UPROPERTY()
	FRotator Rotation = FRotator::ZeroRotator;

	UE_DEPRECATED(5.5, "Moved to component and extensions")
	UPROPERTY()
	FVector Scale = FVector::OneVector;

	UE_DEPRECATED(5.5, "Moved to component and extensions")
	UPROPERTY()
	TWeakObjectPtr<AActor> TargetActorWeak = nullptr;

	UE_DEPRECATED(5.5, "Moved to component and extensions")
	UPROPERTY()
	TWeakObjectPtr<AActor> InternalTargetActorWeak = nullptr;

	UE_DEPRECATED(5.5, "Moved to component and extensions")
	UPROPERTY()
	FVector LocationStrength = FVector::ZeroVector;

	UE_DEPRECATED(5.5, "Moved to component and extensions")
	UPROPERTY()
	FRotator RotationStrength = FRotator::ZeroRotator;

	UE_DEPRECATED(5.5, "Moved to component and extensions")
	UPROPERTY()
	FVector ScaleStrength = FVector::OneVector;

	UE_DEPRECATED(5.5, "Moved to component and extensions")
	UPROPERTY()
	FVector Pan = FVector::ZeroVector;

	UE_DEPRECATED(5.5, "Moved to component and extensions")
	UPROPERTY()
	float Frequency = 0.5f;

	UE_DEPRECATED(5.5, "Moved to component and extensions")
	UPROPERTY()
	FVector PushStrength = FVector(100.f);

	UE_DEPRECATED(5.5, "Moved to component and extensions")
	UPROPERTY()
	ECEClonerEffectorPushDirection PushDirection = ECEClonerEffectorPushDirection::Forward;

	UE_DEPRECATED(5.5, "Moved to component and extensions")
	UPROPERTY()
	bool bOrientationForceEnabled = false;

	UE_DEPRECATED(5.5, "Moved to component and extensions")
	UPROPERTY()
	float OrientationForceRate = 1.f;

	UE_DEPRECATED(5.5, "Moved to component and extensions")
	UPROPERTY()
	FVector OrientationForceMin = FVector(-0.1f);

	UE_DEPRECATED(5.5, "Moved to component and extensions")
	UPROPERTY()
	FVector OrientationForceMax = FVector(0.1f);

	UE_DEPRECATED(5.5, "Moved to component and extensions")
	UPROPERTY()
	bool bVortexForceEnabled = false;

	UE_DEPRECATED(5.5, "Moved to component and extensions")
	UPROPERTY()
	float VortexForceAmount = 10000.f;

	UE_DEPRECATED(5.5, "Moved to component and extensions")
	UPROPERTY()
	FVector VortexForceAxis = FVector::ZAxisVector;

	UE_DEPRECATED(5.5, "Moved to component and extensions")
	UPROPERTY()
	bool bCurlNoiseForceEnabled = false;

	UE_DEPRECATED(5.5, "Moved to component and extensions")
	UPROPERTY()
	float CurlNoiseForceStrength = 1000.f;

	UE_DEPRECATED(5.5, "Moved to component and extensions")
	UPROPERTY()
	float CurlNoiseForceFrequency = 10.f;

	UE_DEPRECATED(5.5, "Moved to component and extensions")
	UPROPERTY()
	bool bAttractionForceEnabled = false;

	UE_DEPRECATED(5.5, "Moved to component and extensions")
	UPROPERTY()
	float AttractionForceStrength = 1000.f;

	UE_DEPRECATED(5.5, "Moved to component and extensions")
	UPROPERTY()
	float AttractionForceFalloff = 0.1f;

	UE_DEPRECATED(5.5, "Moved to component and extensions")
	UPROPERTY()
	bool bGravityForceEnabled = false;

	UE_DEPRECATED(5.5, "Moved to component and extensions")
	UPROPERTY()
	FVector GravityForceAcceleration = FVector(0, 0, -980.f);

#if WITH_EDITORONLY_DATA
	UE_DEPRECATED(5.5, "Moved to component and extensions")
	UPROPERTY()
	bool bVisualizerComponentVisible = true;

	UE_DEPRECATED(5.5, "Moved to component and extensions")
	UPROPERTY()
	bool bVisualizerSpriteVisible = true;
#endif
};
