// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CEClonerEffectorShared.h"
#include "CEPropertyChangeDispatcher.h"
#include "Components/SceneComponent.h"
#include "UObject/WeakObjectPtrTemplates.h"
#include "CEEffectorComponent.generated.h"

class UCEClonerEffectorExtension;
class UCEEffectorEffectBase;
class UCEEffectorExtensionBase;
class UCEEffectorModeBase;
class UCEEffectorTypeBase;
class UDynamicMesh;
class UDynamicMeshComponent;

/** Class used to define an effector */
UCLASS(
	MinimalAPI
	, BlueprintType
	, HideCategories=(Tags,Activation,LOD,AssetUserData,Navigation,Rendering,Cooking,Physics,WorldPartition)
	, DisplayName = "Motion Design Effector Component"
	, meta=(BlueprintSpawnableComponent))
class UCEEffectorComponent : public USceneComponent
{
	GENERATED_BODY()

	friend class ACEEffectorActor;
	friend class FCEEditorEffectorComponentDetailCustomization;
	friend class UCEClonerEffectorExtension;

public:
#if WITH_EDITOR
	CLONEREFFECTOR_API static FName GetModeNamePropertyName();

	CLONEREFFECTOR_API static FName GetTypeNamePropertyName();
#endif

	UCEEffectorComponent();

	UFUNCTION(BlueprintCallable, Category="Effector")
	CLONEREFFECTOR_API void SetEnabled(bool bInEnable);

	UFUNCTION(BlueprintPure, Category="Effector")
	bool GetEnabled() const
	{
		return bEnabled;
	}

	UFUNCTION(BlueprintCallable, Category="Effector")
	CLONEREFFECTOR_API void SetColor(const FLinearColor& InColor);

	UFUNCTION(BlueprintPure, Category="Effector")
	const FLinearColor& GetColor() const
	{
		return Color;
	}

	UFUNCTION(BlueprintCallable, Category="Effector")
	CLONEREFFECTOR_API void SetMagnitude(float InMagnitude);

	UFUNCTION(BlueprintPure, Category="Effector")
	float GetMagnitude() const
	{
		return Magnitude;
	}

	UFUNCTION(BlueprintCallable, Category="Effector")
	CLONEREFFECTOR_API void SetTypeName(FName InTypeName);

	UFUNCTION(BlueprintPure, Category="Effector")
	FName GetTypeName() const
	{
		return TypeName;
	}

	UFUNCTION(BlueprintCallable, Category="Effector")
	CLONEREFFECTOR_API void SetTypeClass(TSubclassOf<UCEEffectorTypeBase> InTypeClass);

	UFUNCTION(BlueprintPure, Category="Effector")
	CLONEREFFECTOR_API TSubclassOf<UCEEffectorTypeBase> GetTypeClass() const;

	UFUNCTION(BlueprintCallable, Category="Effector")
	CLONEREFFECTOR_API void SetModeName(FName InModeName);

	UFUNCTION(BlueprintPure, Category="Effector")
	FName GetModeName() const
	{
		return ModeName;
	}

	UFUNCTION(BlueprintCallable, Category="Effector")
	CLONEREFFECTOR_API void SetModeClass(TSubclassOf<UCEEffectorModeBase> InModeClass);

	UFUNCTION(BlueprintPure, Category="Effector")
	CLONEREFFECTOR_API TSubclassOf<UCEEffectorModeBase> GetModeClass() const;

#if WITH_EDITOR
	UFUNCTION(BlueprintCallable, Category="Effector")
	CLONEREFFECTOR_API void SetVisualizerComponentVisible(bool bInVisible);

	UFUNCTION(BlueprintPure, Category="Effector")
	bool GetVisualizerComponentVisible() const
	{
		return bVisualizerComponentVisible;
	}

	UFUNCTION(BlueprintCallable, Category="Effector")
	CLONEREFFECTOR_API void SetVisualizerSpriteVisible(bool bInVisible);

	UFUNCTION(BlueprintPure, Category="Effector")
	bool GetVisualizerSpriteVisible() const
	{
		return bVisualizerSpriteVisible;
	}

	int32 AddVisualizerComponent(UDynamicMeshComponent* InComponent);

	/** Updates a specific mesh visualizers based on its index */
	void UpdateVisualizer(int32 InVisualizerIndex, TFunctionRef<void(UDynamicMesh*)> InMeshFunction) const;

	void SetVisualizerColor(int32 InVisualizerIndex, const FLinearColor& InColor);
#endif

	/** Get linked cloners */
	TConstArrayView<TWeakObjectPtr<UCEClonerEffectorExtension>> GetClonerExtensionsWeak() const;

	/** Get the effector channel data */
	FCEClonerEffectorChannelData& GetChannelData();

	/** Get the effector channel identifier */
	UFUNCTION(BlueprintPure, Category="Effector")
	int32 GetChannelIdentifier() const;

	void OnClonerLinked(UCEClonerEffectorExtension* InClonerExtension);
	void OnClonerUnlinked(UCEClonerEffectorExtension* InClonerExtension);

	template<
		typename InTypeClass
		UE_REQUIRES(TIsDerivedFrom<InTypeClass, UCEEffectorTypeBase>::Value)>
	InTypeClass* GetActiveType() const
	{
		return Cast<InTypeClass>(GetActiveType());
	}

	UFUNCTION(BlueprintPure, Category="Effector")
	UCEEffectorTypeBase* GetActiveType() const
	{
		return ActiveType;
	}

	template<
		typename InModeClass
		UE_REQUIRES(TIsDerivedFrom<InModeClass, UCEEffectorModeBase>::Value)>
	InModeClass* GetActiveType() const
	{
		return Cast<InModeClass>(GetActiveMode());
	}

	UFUNCTION(BlueprintPure, Category="Effector")
	UCEEffectorModeBase* GetActiveMode() const
	{
		return ActiveMode;
	}

	TConstArrayView<TObjectPtr<UCEEffectorEffectBase>> GetActiveEffects() const
	{
		return ActiveEffects;
	}

	UFUNCTION(BlueprintPure, Category="Effector")
	void GetActiveEffects(TArray<UCEEffectorEffectBase*>& OutEffects) const;

	UFUNCTION(BlueprintCallable, Category="Effector", meta=(DeterminesOutputType="InExtensionClass"))
	UCEEffectorExtensionBase* GetExtension(TSubclassOf<UCEEffectorExtensionBase> InExtensionClass) const;

	template<typename InExtensionClass UE_REQUIRES(std::is_base_of_v<UCEEffectorExtensionBase, InExtensionClass>)>
	InExtensionClass* GetExtension() const
	{
		return Cast<InExtensionClass>(GetExtension(InExtensionClass::StaticClass()));
	}

	UCEEffectorExtensionBase* GetExtension(FName InExtensionName) const;

	void RequestClonerUpdate(bool bInImmediate);

protected:
	//~ Begin UActorComponent
	virtual void OnComponentCreated() override;
	virtual void OnComponentDestroyed(bool bDestroyingHierarchy) override;
	//~ End UActorComponent

	//~ Begin UObject
#if WITH_EDITOR
	virtual void PostEditChangeProperty(FPropertyChangedEvent& InPropertyChangedEvent) override;
	virtual void PostEditUndo() override;
#endif
	virtual void PostEditImport() override;
	virtual void PostLoad() override;
	virtual void PostDuplicate(EDuplicateMode::Type InDuplicateMode) override;
	//~ End UObject

#if WITH_EDITOR
	UFUNCTION(CallInEditor, Category="Effector")
	void ForceRefreshLinkedCloners();
#endif

	/** Is this effector enabled/disabled on linked cloners */
	UPROPERTY(EditInstanceOnly, Setter="SetEnabled", Getter="GetEnabled", Category="Effector")
	bool bEnabled = true;

	/** The ratio effect of the effector on clones */
	UPROPERTY(EditInstanceOnly, Setter, Getter, Category="Effector", meta=(ClampMin="0", ClampMax="1"))
	float Magnitude = 1.f;

	/** Affected clones color passed over to material */
	UPROPERTY(EditInstanceOnly, Setter, Getter, Category="Effector")
	FLinearColor Color = FLinearColor::Red;

	/** Name of the shape type to use */
	UPROPERTY(EditInstanceOnly, Setter, Getter, DisplayName="Shape", Category="Shape", meta=(GetOptions="GetEffectorTypeNames"))
	FName TypeName = NAME_None;

	/** Cached active type used for faster access */
	UPROPERTY(Transient, DuplicateTransient, NonTransactional)
	TObjectPtr<UCEEffectorTypeBase> ActiveType;

	/** Name of the shape type to use */
	UPROPERTY(EditInstanceOnly, Setter, Getter, DisplayName="Mode", Category="Mode", meta=(GetOptions="GetEffectorModeNames"))
	FName ModeName = NAME_None;

	/** Cached active mode used for faster access */
	UPROPERTY(Transient, DuplicateTransient, NonTransactional)
	TObjectPtr<UCEEffectorModeBase> ActiveMode;

	/** Active Effects on this effector */
	UPROPERTY(Transient, DuplicateTransient, NonTransactional)
	TArray<TObjectPtr<UCEEffectorEffectBase>> ActiveEffects;

#if WITH_EDITORONLY_DATA
	/** Visibility of the components visualizer */
	UPROPERTY(EditInstanceOnly, AdvancedDisplay, Category="Effector")
	bool bVisualizerComponentVisible = true;

	/** Toggle the sprite to visualize and click on this effector */
	UPROPERTY(EditInstanceOnly, AdvancedDisplay, Category="Effector")
	bool bVisualizerSpriteVisible = true;
#endif

private:
	static constexpr TCHAR VisualizerMaterialPath[] = TEXT("/Script/Engine.Material'/ClonerEffector/Materials/M_EffectorVisualizer.M_EffectorVisualizer'");
	static constexpr TCHAR VisualizerColorName[] = TEXT("VisualizerColor");
	static constexpr TCHAR SpriteTexturePath[] = TEXT("/Script/Engine.Texture2D'/ClonerEffector/Textures/T_EffectorIcon.T_EffectorIcon'");

	void RegisterToChannel();
	void UnregisterFromChannel();
	void OnTransformUpdated(USceneComponent* InComponent, EUpdateTransformFlags InFlags, ETeleportType InTeleportType);

	void OnEnabledChanged();
	void OnEffectorEnabled();
	void OnEffectorDisabled();
	void OnEffectorSetEnabled(const UWorld* InWorld, bool bInEnabled, bool bInTransact);

	void OnEffectorOptionsChanged();
	void OnTypeNameChanged();
	void OnModeNameChanged();
	void OnEffectsChanged();

#if WITH_EDITOR
	void OnVisualizerOptionsChanged();
#endif

	template<
		typename InExtensionClass>
	InExtensionClass* FindOrAddExtension()
	{
		return Cast<InExtensionClass>(FindOrAddExtension(InExtensionClass::StaticClass()));
	}

	UCEEffectorExtensionBase* FindOrAddExtension(FName InExtensionName);
	UCEEffectorExtensionBase* FindOrAddExtension(TSubclassOf<UCEEffectorExtensionBase> InExtensionClass);

	/** Gets all type names available */
	UFUNCTION()
	TArray<FName> GetEffectorTypeNames() const;

	/** Gets all mode names available */
	UFUNCTION()
	TArray<FName> GetEffectorModeNames() const;

	/** Transient effector channel data */
	UPROPERTY(VisibleInstanceOnly, Transient, DuplicateTransient, TextExportTransient, NonTransactional, AdvancedDisplay, Category="Effector", meta=(NoResetToDefault))
	FCEClonerEffectorChannelData ChannelData;

	/** Cloners linked to this effector, used to refresh or relink on duplicate */
	UPROPERTY(SkipSerialization, NonTransactional)
	TArray<TWeakObjectPtr<UCEClonerEffectorExtension>> ClonerExtensionsWeak;

	/** Cached extensions instances */
	UPROPERTY()
	TArray<TObjectPtr<UCEEffectorExtensionBase>> ExtensionInstances;

#if WITH_EDITORONLY_DATA
	UPROPERTY(Transient, DuplicateTransient)
	TArray<TWeakObjectPtr<UDynamicMeshComponent>> VisualizerComponentsWeak;

	UPROPERTY(Transient, DuplicateTransient)
	TArray<TObjectPtr<UMaterialInstanceDynamic>> VisualizerMaterialsWeak;
#endif

#if WITH_EDITOR
	static const TCEPropertyChangeDispatcher<UCEEffectorComponent> PropertyChangeDispatcher;
#endif
};