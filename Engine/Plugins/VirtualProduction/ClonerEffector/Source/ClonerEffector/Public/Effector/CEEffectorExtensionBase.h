// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CEClonerEffectorShared.h"
#include "UObject/Object.h"
#include "CEEffectorExtensionBase.generated.h"

class UCEEffectorComponent;

/** Represents an extension for an effector to apply a custom behavior on cloner */
UCLASS(MinimalAPI, Abstract, BlueprintType, Within=CEEffectorComponent)
class UCEEffectorExtensionBase : public UObject
{
	GENERATED_BODY()

public:
	UCEEffectorExtensionBase()
		: UCEEffectorExtensionBase(
			NAME_None
		)
	{}

	UCEEffectorExtensionBase(FName InExtensionName)
		: ExtensionName(InExtensionName)
	{}

	UFUNCTION(BlueprintPure, Category="Effector")
	FName GetExtensionName() const
	{
		return ExtensionName;
	}

#if WITH_EDITOR
	CLONEREFFECTOR_API FCEExtensionSection GetExtensionSection() const;
#endif

	/** Get the effector component using this extension */
	UCEEffectorComponent* GetEffectorComponent() const;

	/** Request refresh extension next tick */
	void UpdateExtensionParameters(bool bInUpdateLinkedCloners = false);

	/** Enable this extension */
	void ActivateExtension();

	/** Disable this extension */
	void DeactivateExtension();

	UFUNCTION(BlueprintPure, Category="Effector")
	bool IsExtensionActive() const
	{
		return bExtensionActive;
	}

	/** Called after load when extension name is obsolete to handle redirection to this extension */
	virtual bool RedirectExtensionName(FName InOldExtensionName) const
	{
		return false;
	}

protected:
	//~ Begin UObject
	virtual void PostEditImport() override;
	//~ End UObject

	/** Called when extension becomes active */
	virtual void OnExtensionActivated() {}

	/** Called when extension becomes inactive */
	virtual void OnExtensionDeactivated() {}

	/** Called to reapply type parameters */
	virtual void OnExtensionParametersChanged(UCEEffectorComponent* InComponent) {}

	/** Used by PECP to update parameters */
	void OnExtensionPropertyChanged();

private:
	/** Unique extension name used for dropdown and selection */
	UPROPERTY(Transient)
	FName ExtensionName = NAME_None;

	bool bExtensionActive = false;
};