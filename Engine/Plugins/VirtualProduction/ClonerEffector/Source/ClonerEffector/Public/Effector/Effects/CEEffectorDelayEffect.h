// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CEPropertyChangeDispatcher.h"
#include "Effector/Effects/CEEffectorEffectBase.h"
#include "CEEffectorDelayEffect.generated.h"

class UCEEffectorComponent;

/** Effector delay extension to affect clones after a period of time with a spring effect */
UCLASS(MinimalAPI, BlueprintType, Within=CEEffectorComponent, meta=(Section="Delay", Priority=3))
class UCEEffectorDelayEffect : public UCEEffectorEffectBase
{
	GENERATED_BODY()

public:
	UCEEffectorDelayEffect()
		: UCEEffectorEffectBase(
			TEXT("Delay")
		)
	{}

	UFUNCTION(BlueprintCallable, Category="Effector")
	CLONEREFFECTOR_API void SetDelayEnabled(bool bInEnabled);

	UFUNCTION(BlueprintPure, Category="Effector")
	bool GetDelayEnabled() const
	{
		return bDelayEnabled;
	}

	UFUNCTION(BlueprintCallable, Category="Effector")
	CLONEREFFECTOR_API void SetDelayInDuration(float InDuration);

	UFUNCTION(BlueprintPure, Category="Effector")
	float GetDelayInDuration() const
	{
		return DelayInDuration;
	}

	UFUNCTION(BlueprintCallable, Category="Effector")
	CLONEREFFECTOR_API void SetDelayOutDuration(float InDuration);

	UFUNCTION(BlueprintPure, Category="Effector")
	float GetDelayOutDuration() const
	{
		return DelayOutDuration;
	}

	UFUNCTION(BlueprintCallable, Category="Effector")
	CLONEREFFECTOR_API void SetDelaySpringFrequency(float InFrequency);

	UFUNCTION(BlueprintPure, Category="Effector")
	float GetDelaySpringFrequency() const
	{
		return DelaySpringFrequency;
	}

	UFUNCTION(BlueprintCallable, Category="Effector")
	CLONEREFFECTOR_API void SetDelaySpringFalloff(float InFalloff);

	UFUNCTION(BlueprintPure, Category="Effector")
	float GetDelaySpringFalloff() const
	{
		return DelaySpringFalloff;
	}

protected:
	//~ Begin UObject
#if WITH_EDITOR
	virtual void PostEditChangeProperty(FPropertyChangedEvent& InPropertyChangedEvent) override;
#endif
	//~ End UObject

	//~ Begin UCEEffectorEffectBase
	virtual void UpdateEffectChannelData(FCEClonerEffectorChannelData& InChannelData, bool bInEnabled) override;
	//~ End UCEEffectorEffectBase

	/** Enable time delay for the mode effect */
	UPROPERTY(EditInstanceOnly, Setter="SetDelayEnabled", Getter="GetDelayEnabled", DisplayName="Enabled", Category="Delay", meta=(RefreshPropertyView))
	bool bDelayEnabled = false;
	
	/** Time delay for the effect to reach its peak */
	UPROPERTY(EditInstanceOnly, Setter, Getter, DisplayName="Offset Duration", Category="Delay", meta=(Units=Seconds, ClampMin="0", EditCondition="bDelayEnabled", EditConditionHides))
	float DelayInDuration = 0.f;

	/** Time delay for the effect to go back to its rest state */
	UPROPERTY(EditInstanceOnly, Setter, Getter, DisplayName="Trail Duration", Category="Delay", meta=(Units=Seconds, ClampMin="0", EditCondition="bDelayEnabled", EditConditionHides))
	float DelayOutDuration = 1.f;

	/** Frequency of the delayed spring effect */
	UPROPERTY(EditInstanceOnly, Setter, Getter, DisplayName="Spring Frequency", Category="Delay", meta=(ClampMin="1", EditCondition="bDelayEnabled", EditConditionHides))
	float DelaySpringFrequency = 3.f;

	/** Damping or decay of the delayed spring effect */
	UPROPERTY(EditInstanceOnly, Setter, Getter, DisplayName="Spring Falloff", Category="Delay", meta=(ClampMin="1", EditCondition="bDelayEnabled", EditConditionHides))
	float DelaySpringFalloff = 1.f;

private:
#if WITH_EDITOR
	/** Used for PECP */
	static const TCEPropertyChangeDispatcher<UCEEffectorDelayEffect> PropertyChangeDispatcher;
#endif
};