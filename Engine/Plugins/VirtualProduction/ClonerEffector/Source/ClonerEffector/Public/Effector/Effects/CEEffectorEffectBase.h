// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Effector/CEEffectorExtensionBase.h"
#include "CEEffectorEffectBase.generated.h"

class UCEEffectorModeBase;

/** Represents an effect for an effector to affect clones in a specific way, works with modes and types */
UCLASS(MinimalAPI, Abstract, BlueprintType, Within=CEEffectorComponent, meta=(Section="Effect", Priority=3))
class UCEEffectorEffectBase : public UCEEffectorExtensionBase
{
	GENERATED_BODY()

public:
	UCEEffectorEffectBase()
		: UCEEffectorExtensionBase(NAME_None)
	{}

	UCEEffectorEffectBase(FName InModeName)
		: UCEEffectorExtensionBase(
			InModeName
		)
	{}

	/** Filter supported mode */
	virtual bool IsModeSupported(TSubclassOf<UCEEffectorModeBase> InModeClass) const
	{
		return true;
	}

protected:
	//~ Begin UCEEffectorExtensionBase
	virtual void OnExtensionParametersChanged(UCEEffectorComponent* InComponent) override;
	virtual void OnExtensionDeactivated() override;
	//~ End UCEEffectorExtensionBase

	virtual void UpdateEffectChannelData(FCEClonerEffectorChannelData& InChannelData, bool bInEnabled) {}
};