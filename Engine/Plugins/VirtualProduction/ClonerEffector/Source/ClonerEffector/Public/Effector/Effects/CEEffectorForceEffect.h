// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CEPropertyChangeDispatcher.h"
#include "Effector/Effects/CEEffectorEffectBase.h"
#include "CEEffectorForceEffect.generated.h"

class UCEEffectorComponent;

/** Effector supported forces to affect clones */
UCLASS(MinimalAPI, BlueprintType, Within=CEEffectorComponent, meta=(Section="Forces", Priority=3))
class UCEEffectorForceEffect : public UCEEffectorEffectBase
{
	GENERATED_BODY()

public:
	UCEEffectorForceEffect()
		: UCEEffectorEffectBase(
			TEXT("Forces")
		)
	{}

	UFUNCTION(BlueprintCallable, Category="Effector")
	CLONEREFFECTOR_API void SetForcesEnabled(bool bInForcesEnabled);

	UFUNCTION(BlueprintPure, Category="Effector")
	bool GetForcesEnabled() const
	{
		return bForcesEnabled;
	}

	UFUNCTION(BlueprintCallable, Category="Effector")
	CLONEREFFECTOR_API void SetOrientationForceEnabled(bool bInForceEnabled);

	UFUNCTION(BlueprintPure, Category="Effector")
	bool GetOrientationForceEnabled() const
	{
		return bOrientationForceEnabled;
	}

	UFUNCTION(BlueprintCallable, Category="Effector")
	CLONEREFFECTOR_API void SetOrientationForceRate(float InForceOrientationRate);

	UFUNCTION(BlueprintPure, Category="Effector")
	float GetOrientationForceRate() const
	{
		return OrientationForceRate;
	}

	UFUNCTION(BlueprintCallable, Category="Effector")
	CLONEREFFECTOR_API void SetOrientationForceMin(const FVector& InForceOrientationMin);

	UFUNCTION(BlueprintPure, Category="Effector")
	const FVector& GetOrientationForceMin() const
	{
		return OrientationForceMin;
	}

	UFUNCTION(BlueprintCallable, Category="Effector")
	CLONEREFFECTOR_API void SetOrientationForceMax(const FVector& InForceOrientationMax);

	UFUNCTION(BlueprintPure, Category="Effector")
	const FVector& GetOrientationForceMax() const
	{
		return OrientationForceMax;
	}

	UFUNCTION(BlueprintCallable, Category="Effector")
	CLONEREFFECTOR_API void SetVortexForceEnabled(bool bInForceEnabled);

	UFUNCTION(BlueprintPure, Category="Effector")
	bool GetVortexForceEnabled() const
	{
		return bVortexForceEnabled;
	}

	UFUNCTION(BlueprintCallable, Category="Effector")
	CLONEREFFECTOR_API void SetVortexForceAmount(float InForceVortexAmount);

	UFUNCTION(BlueprintPure, Category="Effector")
	float GetVortexForceAmount() const
	{
		return VortexForceAmount;
	}

	UFUNCTION(BlueprintCallable, Category="Effector")
	CLONEREFFECTOR_API void SetVortexForceAxis(const FVector& InForceVortexAxis);

	UFUNCTION(BlueprintPure, Category="Effector")
	const FVector& GetVortexForceAxis() const
	{
		return VortexForceAxis;
	}

	UFUNCTION(BlueprintCallable, Category="Effector")
	CLONEREFFECTOR_API void SetCurlNoiseForceEnabled(bool bInForceEnabled);

	UFUNCTION(BlueprintPure, Category="Effector")
	bool GetCurlNoiseForceEnabled() const
	{
		return bCurlNoiseForceEnabled;
	}

	UFUNCTION(BlueprintCallable, Category="Effector")
	CLONEREFFECTOR_API void SetCurlNoiseForceStrength(float InForceCurlNoiseStrength);

	UFUNCTION(BlueprintPure, Category="Effector")
	float GetCurlNoiseForceStrength() const
	{
		return CurlNoiseForceStrength;
	}

	UFUNCTION(BlueprintCallable, Category="Effector")
	CLONEREFFECTOR_API void SetCurlNoiseForceFrequency(float InForceCurlNoiseFrequency);

	UFUNCTION(BlueprintPure, Category="Effector")
	float GetCurlNoiseForceFrequency() const
	{
		return CurlNoiseForceFrequency;
	}

	UFUNCTION(BlueprintCallable, Category="Effector")
	CLONEREFFECTOR_API void SetAttractionForceEnabled(bool bInForceEnabled);

	UFUNCTION(BlueprintPure, Category="Effector")
	bool GetAttractionForceEnabled() const
	{
		return bAttractionForceEnabled;
	}

	UFUNCTION(BlueprintCallable, Category="Effector")
	CLONEREFFECTOR_API void SetAttractionForceStrength(float InForceStrength);

	UFUNCTION(BlueprintPure, Category="Effector")
	float GetAttractionForceStrength() const
	{
		return AttractionForceStrength;
	}

	UFUNCTION(BlueprintCallable, Category="Effector")
	CLONEREFFECTOR_API void SetAttractionForceFalloff(float InForceFalloff);

	UFUNCTION(BlueprintPure, Category="Effector")
	float GetAttractionForceFalloff() const
	{
		return AttractionForceFalloff;
	}

	UFUNCTION(BlueprintCallable, Category="Effector")
	CLONEREFFECTOR_API void SetGravityForceEnabled(bool bInForceEnabled);

	UFUNCTION(BlueprintPure, Category="Effector")
	bool GetGravityForceEnabled() const
	{
		return bGravityForceEnabled;
	}

	UFUNCTION(BlueprintCallable, Category="Effector")
	CLONEREFFECTOR_API void SetGravityForceAcceleration(const FVector& InAcceleration);

	UFUNCTION(BlueprintPure, Category="Effector")
	FVector GetGravityForceAcceleration() const
	{
		return GravityForceAcceleration;
	}

	UFUNCTION(BlueprintCallable, Category="Effector")
	CLONEREFFECTOR_API void SetDragForceEnabled(bool bInEnabled);

	UFUNCTION(BlueprintPure, Category="Effector")
	bool GetDragForceEnabled() const
	{
		return bDragForceEnabled;
	}

	UFUNCTION(BlueprintCallable, Category="Effector")
	CLONEREFFECTOR_API void SetDragForceLinear(float InStrength);

	UFUNCTION(BlueprintPure, Category="Effector")
	float GetDragForceLinear() const
	{
		return DragForceLinear;
	}

	UFUNCTION(BlueprintCallable, Category="Effector")
	CLONEREFFECTOR_API void SetDragForceRotational(float InStrength);

	UFUNCTION(BlueprintPure, Category="Effector")
	float GetDragForceRotational() const
	{
		return DragForceRotational;
	}

	UFUNCTION(BlueprintCallable, Category="Effector")
	CLONEREFFECTOR_API void SetVectorNoiseForceEnabled(bool bInEnabled);

	UFUNCTION(BlueprintPure, Category="Effector")
	bool GetVectorNoiseForceEnabled() const
	{
		return bVectorNoiseForceEnabled;
	}

	UFUNCTION(BlueprintCallable, Category="Effector")
	CLONEREFFECTOR_API void SetVectorNoiseForceAmount(float InAmount);

	UFUNCTION(BlueprintPure, Category="Effector")
	float GetVectorNoiseForceAmount() const
	{
		return VectorNoiseForceAmount;
	}

protected:
	//~ Begin UObject
#if WITH_EDITOR
	virtual void PostEditChangeProperty(FPropertyChangedEvent& InPropertyChangedEvent) override;
#endif
	//~ End UObject

	//~ Begin UCEEffectorExtensionBase
	virtual void OnExtensionDeactivated() override;
	//~ End UCEEffectorExtensionBase
	
	//~ Begin UCEEffectorEffectBase
	virtual void UpdateEffectChannelData(FCEClonerEffectorChannelData& InChannelData, bool bInEnabled) override;
	//~ End UCEEffectorEffectBase

	void OnForceOptionsChanged();

	/** Forces global state */
	UPROPERTY(EditInstanceOnly, Setter="SetForcesEnabled", Getter="GetForcesEnabled", DisplayName="Enabled", Category="Force", meta=(RefreshPropertyView))
	bool bForcesEnabled = false;

	/** Enable orientation force to allow each clone instance to rotate around its pivot */
	UPROPERTY(EditInstanceOnly, Setter="SetOrientationForceEnabled", Getter="GetOrientationForceEnabled", Category="Force", meta=(EditCondition="bForcesEnabled", EditConditionHides, RefreshPropertyView))
	bool bOrientationForceEnabled = false;

	UPROPERTY(EditInstanceOnly, Setter, Getter, Category="Force", meta=(Delta="0.0001", ClampMin="0", EditCondition="bForcesEnabled && bOrientationForceEnabled", EditConditionHides))
	float OrientationForceRate = 1.f;

	UPROPERTY(EditInstanceOnly, Setter, Getter, Category="Force", meta=(AllowPreserveRatio, EditCondition="bForcesEnabled && bOrientationForceEnabled", EditConditionHides))
	FVector OrientationForceMin = FVector(-0.1f);

	UPROPERTY(EditInstanceOnly, Setter, Getter, Category="Force", meta=(AllowPreserveRatio, EditCondition="bForcesEnabled && bOrientationForceEnabled", EditConditionHides))
	FVector OrientationForceMax = FVector(0.1f);

	/** Enable vortex force to allow each clone instance to rotate around a specific axis on the cloner pivot */
	UPROPERTY(EditInstanceOnly, Setter="SetVortexForceEnabled", Getter="GetVortexForceEnabled", Category="Force", meta=(EditCondition="bForcesEnabled", EditConditionHides, RefreshPropertyView))
	bool bVortexForceEnabled = false;

	UPROPERTY(EditInstanceOnly, Setter, Getter, Category="Force", meta=(EditCondition="bForcesEnabled && bVortexForceEnabled", EditConditionHides))
	float VortexForceAmount = 100.f;

	UPROPERTY(EditInstanceOnly, Setter, Getter, Category="Force", meta=(EditCondition="bForcesEnabled && bVortexForceEnabled", EditConditionHides))
	FVector VortexForceAxis = FVector::ZAxisVector;

	/** Enable curl noise force to allow each clone instance to add random location variation */
	UPROPERTY(EditInstanceOnly, Setter="SetCurlNoiseForceEnabled", Getter="GetCurlNoiseForceEnabled", Category="Force", meta=(EditCondition="bForcesEnabled", EditConditionHides, RefreshPropertyView))
	bool bCurlNoiseForceEnabled = false;

	UPROPERTY(EditInstanceOnly, Setter, Getter, Category="Force", meta=(EditCondition="bForcesEnabled && bCurlNoiseForceEnabled", EditConditionHides))
	float CurlNoiseForceStrength = 100.f;

	UPROPERTY(EditInstanceOnly, Setter, Getter, Category="Force", meta=(EditCondition="bForcesEnabled && bCurlNoiseForceEnabled", EditConditionHides))
	float CurlNoiseForceFrequency = 10.f;

	/** Enable attraction force to allow each clone instances to gravitate toward a location */
	UPROPERTY(EditInstanceOnly, Setter="SetAttractionForceEnabled", Getter="GetAttractionForceEnabled", Category="Force", meta=(EditCondition="bForcesEnabled", EditConditionHides, RefreshPropertyView))
	bool bAttractionForceEnabled = false;

	UPROPERTY(EditInstanceOnly, Setter, Getter, Category="Force", meta=(EditCondition="bForcesEnabled && bAttractionForceEnabled", EditConditionHides))
	float AttractionForceStrength = 100.f;

	UPROPERTY(EditInstanceOnly, Setter, Getter, Category="Force", meta=(EditCondition="bForcesEnabled && bAttractionForceEnabled", EditConditionHides))
	float AttractionForceFalloff = 0.1f;

	/** Enable gravity force to pull particles based on an acceleration vector */
	UPROPERTY(EditInstanceOnly, Setter="SetGravityForceEnabled", Getter="GetGravityForceEnabled", Category="Force", meta=(EditCondition="bForcesEnabled", EditConditionHides, RefreshPropertyView))
	bool bGravityForceEnabled = false;

	UPROPERTY(EditInstanceOnly, Setter, Getter, Category="Force", meta=(EditCondition="bForcesEnabled && bGravityForceEnabled", EditConditionHides))
	FVector GravityForceAcceleration = FVector(0, 0, -980.f);

	/** Enable drag force to decrease particles velocity */
	UPROPERTY(EditInstanceOnly, Setter="SetDragForceEnabled", Getter="GetDragForceEnabled", Category="Force", meta=(EditCondition="bForcesEnabled", EditConditionHides, RefreshPropertyView))
	bool bDragForceEnabled = false;

	UPROPERTY(EditInstanceOnly, Setter, Getter, Category="Force", meta=(EditCondition="bForcesEnabled && bDragForceEnabled", EditConditionHides))
	float DragForceLinear = 0.f;

	UPROPERTY(EditInstanceOnly, Setter, Getter, Category="Force", meta=(EditCondition="bForcesEnabled && bDragForceEnabled", EditConditionHides))
	float DragForceRotational = 0.f;

	/** Enable vector random noise force to add variation to clones behavior */
	UPROPERTY(EditInstanceOnly, Setter="SetVectorNoiseForceEnabled", Getter="GetVectorNoiseForceEnabled", Category="Force", meta=(EditCondition="bForcesEnabled", EditConditionHides, RefreshPropertyView))
	bool bVectorNoiseForceEnabled = false;

	UPROPERTY(EditInstanceOnly, Setter, Getter, Category="Force", meta=(EditCondition="bForcesEnabled && bVectorNoiseForceEnabled", EditConditionHides))
	float VectorNoiseForceAmount = 100.f;

private:
#if WITH_EDITOR
	/** Used for PECP */
	static const TCEPropertyChangeDispatcher<UCEEffectorForceEffect> PropertyChangeDispatcher;
#endif
};