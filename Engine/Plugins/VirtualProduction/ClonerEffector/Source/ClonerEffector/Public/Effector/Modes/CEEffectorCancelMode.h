// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CEClonerEffectorShared.h"
#include "CEEffectorModeBase.h"
#include "CEEffectorCancelMode.generated.h"

class UCEEffectorComponent;

/** Negates all effects applied on clones */
UCLASS(MinimalAPI, BlueprintType, Within=CEEffectorComponent)
class UCEEffectorCancelMode : public UCEEffectorModeBase
{
	GENERATED_BODY()

public:
	UCEEffectorCancelMode()
		: UCEEffectorModeBase(TEXT("Cancel"), static_cast<int32>(ECEClonerEffectorMode::Cancel))
	{}

protected:
	//~ Begin UCEEffectorModeBase
	virtual bool IsEffectSupported(TSubclassOf<UCEEffectorEffectBase> InEffectClass) const override;
	//~ End UCEEffectorModeBase
};
