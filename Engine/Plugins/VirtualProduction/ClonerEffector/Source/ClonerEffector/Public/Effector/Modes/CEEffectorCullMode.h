// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CEClonerEffectorShared.h"
#include "CEEffectorModeBase.h"
#include "CEPropertyChangeDispatcher.h"
#include "CEEffectorCullMode.generated.h"

class UCEEffectorComponent;

UENUM(BlueprintType)
enum class ECEEffectorCullModeBehavior : uint8
{
	/** Kill clones based on effector shape, lower priority than other hide effector */
	Kill,
	/** Hide clones based on effector shape, higher priority than other kill effector */
	Hide
};

UCLASS(MinimalAPI, BlueprintType, Within=CEEffectorComponent)
class UCEEffectorCullMode : public UCEEffectorModeBase
{
	GENERATED_BODY()

public:
	UCEEffectorCullMode()
		: UCEEffectorModeBase(TEXT("Cull"), static_cast<int32>(ECEClonerEffectorMode::Cull))
	{}

	UFUNCTION(BlueprintCallable, Category="Effector")
	CLONEREFFECTOR_API void SetBehavior(ECEEffectorCullModeBehavior InBehavior);

	UFUNCTION(BlueprintPure, Category="Effector")
	ECEEffectorCullModeBehavior GetBehavior() const
	{
		return Behavior;
	}
	
	UFUNCTION(BlueprintCallable, Category="Effector")
	CLONEREFFECTOR_API void SetScale(const FVector& InScale);

	UFUNCTION(BlueprintPure, Category="Effector")
	const FVector& GetScale() const
	{
		return Scale;
	}

protected:
	//~ Begin UCEEffectorNoiseMode
	virtual void OnExtensionParametersChanged(UCEEffectorComponent* InComponent) override;
	//~ End UCEEffectorNoiseMode

	//~ Begin UObject
	virtual void PostInitProperties() override;
#if WITH_EDITOR
	virtual void PostEditChangeProperty(FPropertyChangedEvent& InPropertyChangedEvent) override;
#endif
	//~ End UObject

	void OnBehaviorChanged();

	/** Expected behavior when culling clones */
	UPROPERTY(EditInstanceOnly, Setter, Getter, Category="Mode")
	ECEEffectorCullModeBehavior Behavior = ECEEffectorCullModeBehavior::Hide;
	
	/** Scale applied on affected clones before they are culled to slowly disappear */
	UPROPERTY(EditInstanceOnly, Setter, Getter, Category="Mode", meta=(ClampMin="0", MotionDesignVectorWidget, AllowPreserveRatio="XYZ", Delta="0.01"))
	FVector Scale = FVector::ZeroVector;
	
private:
#if WITH_EDITOR
	/** Used for PECP */
	static const TCEPropertyChangeDispatcher<UCEEffectorCullMode> PropertyChangeDispatcher;
#endif
};