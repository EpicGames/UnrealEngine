// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Effector/CEEffectorExtensionBase.h"
#include "CEEffectorModeBase.generated.h"

class UCEEffectorComponent;
class UCEEffectorEffectBase;

/** Represents a behavior for an effector to affect clones in a specific way */
UCLASS(MinimalAPI, Abstract, BlueprintType, Within=CEEffectorComponent, meta=(Section="Mode", Priority=2))
class UCEEffectorModeBase : public UCEEffectorExtensionBase
{
	GENERATED_BODY()

public:
	UCEEffectorModeBase()
		: UCEEffectorModeBase(NAME_None, INDEX_NONE)
	{}

	UCEEffectorModeBase(FName InModeName, int32 InModeIdentifier)
		: UCEEffectorExtensionBase(
			InModeName
		)
		, ModeIdentifier(InModeIdentifier)
	{}

	int32 GetModeIdentifier() const
	{
		return ModeIdentifier;
	}
	
	TSet<TSubclassOf<UCEEffectorEffectBase>> GetSupportedEffects() const;

protected:
	//~ Begin UCEEffectorExtensionBase
	virtual void OnExtensionParametersChanged(UCEEffectorComponent* InComponent) override;
	//~ End UCEEffectorExtensionBase

	/** Override if a specific effect is not supported by this mode */
	virtual bool IsEffectSupported(TSubclassOf<UCEEffectorEffectBase> InEffectClass) const
	{
		return true;
	}

private:
	/** Unique identifier to pass it to niagara */
	UPROPERTY(Transient)
	int32 ModeIdentifier = 0;
};