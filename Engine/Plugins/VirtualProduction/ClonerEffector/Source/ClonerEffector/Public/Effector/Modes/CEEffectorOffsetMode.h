// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CEClonerEffectorShared.h"
#include "CEEffectorModeBase.h"
#include "CEPropertyChangeDispatcher.h"
#include "CEEffectorOffsetMode.generated.h"

class UCEEffectorComponent;

UCLASS(MinimalAPI, BlueprintType, Within=CEEffectorComponent)
class UCEEffectorOffsetMode : public UCEEffectorModeBase
{
	GENERATED_BODY()

public:
	UCEEffectorOffsetMode()
		: UCEEffectorModeBase(TEXT("Offset"), static_cast<int32>(ECEClonerEffectorMode::Default))
	{}

	UFUNCTION(BlueprintCallable, Category="Effector")
	CLONEREFFECTOR_API void SetOffset(const FVector& InOffset);

	UFUNCTION(BlueprintPure, Category="Effector")
	FVector GetOffset() const
	{
		return Offset;
	}

	UFUNCTION(BlueprintCallable, Category="Effector")
	CLONEREFFECTOR_API void SetRotation(const FRotator& InRotation);

	UFUNCTION(BlueprintPure, Category="Effector")
	FRotator GetRotation() const
	{
		return Rotation;
	}

	UFUNCTION(BlueprintCallable, Category="Effector")
	CLONEREFFECTOR_API void SetScale(const FVector& InScale);

	UFUNCTION(BlueprintPure, Category="Effector")
	FVector GetScale() const
	{
		return Scale;
	}

protected:
	//~ Begin UCEEffectorNoiseMode
	virtual void OnExtensionParametersChanged(UCEEffectorComponent* InComponent) override;
	//~ End UCEEffectorNoiseMode

	//~ Begin UObject
#if WITH_EDITOR
	virtual void PostEditChangeProperty(FPropertyChangedEvent& InPropertyChangedEvent) override;
#endif
	//~ End UObject

	/** Offset applied on affected clones */
	UPROPERTY(EditInstanceOnly, Setter, Getter, Category="Mode")
	FVector Offset = FVector::ZeroVector;

	/** Rotation applied on affected clones */
	UPROPERTY(EditInstanceOnly, Setter, Getter, Category="Mode")
	FRotator Rotation = FRotator::ZeroRotator;

	/** Scale applied on affected clones */
	UPROPERTY(EditInstanceOnly, Setter, Getter, Category="Mode", meta=(ClampMin="0", MotionDesignVectorWidget, AllowPreserveRatio="XYZ", Delta="0.01"))
	FVector Scale = FVector::OneVector;

private:
#if WITH_EDITOR
	/** Used for PECP */
	static const TCEPropertyChangeDispatcher<UCEEffectorOffsetMode> PropertyChangeDispatcher;
#endif
};