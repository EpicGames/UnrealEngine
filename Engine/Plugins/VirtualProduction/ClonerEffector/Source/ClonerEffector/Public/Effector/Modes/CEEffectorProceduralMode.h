// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CEClonerEffectorShared.h"
#include "CEEffectorModeBase.h"
#include "CEPropertyChangeDispatcher.h"
#include "CEEffectorProceduralMode.generated.h"

class UCEEffectorComponent;

UCLASS(MinimalAPI, BlueprintType, Within=CEEffectorComponent)
class UCEEffectorProceduralMode : public UCEEffectorModeBase
{
	GENERATED_BODY()

public:
	UCEEffectorProceduralMode()
		: UCEEffectorModeBase(TEXT("Procedural"), static_cast<int32>(ECEClonerEffectorMode::Procedural))
	{}

	UFUNCTION(BlueprintCallable, Category="Effector")
	void SetPattern(ECEClonerEffectorProceduralPattern InPattern);

	UFUNCTION(BlueprintPure, Category="Effector")
	ECEClonerEffectorProceduralPattern GetPattern() const
	{
		return Pattern;
	}

	UFUNCTION(BlueprintCallable, Category="Effector")
	CLONEREFFECTOR_API void SetLocationStrength(const FVector& InStrength);

	UFUNCTION(BlueprintPure, Category="Effector")
	FVector GetLocationStrength() const
	{
		return LocationStrength;
	}

	UFUNCTION(BlueprintCallable, Category="Effector")
	CLONEREFFECTOR_API void SetRotationStrength(const FRotator& InStrength);

	UFUNCTION(BlueprintPure, Category="Effector")
	FRotator GetRotationStrength() const
	{
		return RotationStrength;
	}

	UFUNCTION(BlueprintCallable, Category="Effector")
	CLONEREFFECTOR_API void SetScaleStrength(const FVector& InStrength);

	UFUNCTION(BlueprintPure, Category="Effector")
	FVector GetScaleStrength() const
	{
		return ScaleStrength;
	}

	UFUNCTION(BlueprintCallable, Category="Effector")
	CLONEREFFECTOR_API void SetPan(const FVector& InPan);

	UFUNCTION(BlueprintPure, Category="Effector")
	FVector GetPan() const
	{
		return Pan;
	}

	UFUNCTION(BlueprintCallable, Category="Effector")
	CLONEREFFECTOR_API void SetFrequency(float InFrequency);

	UFUNCTION(BlueprintPure, Category="Effector")
	float GetFrequency() const
	{
		return Frequency;
	}

protected:
	//~ Begin UCEEffectorNoiseMode
	virtual void OnExtensionParametersChanged(UCEEffectorComponent* InComponent) override;
	virtual bool RedirectExtensionName(FName InOldExtensionName) const override;
	//~ End UCEEffectorNoiseMode

	//~ Begin UObject
	virtual void PostInitProperties() override;
#if WITH_EDITOR
	virtual void PostEditChangeProperty(FPropertyChangedEvent& InPropertyChangedEvent) override;
#endif
	//~ End UObject

	/** Procedural pattern behavior */
	UPROPERTY(EditInstanceOnly, Setter, Getter, Category="Mode")
	ECEClonerEffectorProceduralPattern Pattern = ECEClonerEffectorProceduralPattern::CurlNoise;

	/** Amplitude of the noise field for location */
	UPROPERTY(EditInstanceOnly, Setter, Getter, Category="Mode")
	FVector LocationStrength = FVector::ZeroVector;

	/** Amplitude of the noise field for rotation */
	UPROPERTY(EditInstanceOnly, Setter, Getter, Category="Mode")
	FRotator RotationStrength = FRotator::ZeroRotator;

	/** Amplitude of the noise field for scale */
	UPROPERTY(EditInstanceOnly, Setter, Getter, Category="Mode", meta=(ClampMin="0", MotionDesignVectorWidget, AllowPreserveRatio="XYZ", Delta="0.01"))
	FVector ScaleStrength = FVector::OneVector;

	/** Panning to offset the noise field sampling */
	UPROPERTY(EditInstanceOnly, Setter, Getter, Category="Mode", meta=(MotionDesignVectorWidget, AllowPreserveRatio="XYZ"))
	FVector Pan = FVector::ZeroVector;

	/** Intensity of the noise field */
	UPROPERTY(EditInstanceOnly, Setter, Getter, Category="Mode", meta=(ClampMin="0"))
	float Frequency = 0.5f;

private:
#if WITH_EDITOR
	/** Used for PECP */
	static const TCEPropertyChangeDispatcher<UCEEffectorProceduralMode> PropertyChangeDispatcher;
#endif
};