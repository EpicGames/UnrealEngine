// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CEClonerEffectorShared.h"
#include "CEEffectorModeBase.h"
#include "CEPropertyChangeDispatcher.h"
#include "CEEffectorPushMode.generated.h"

class UCEEffectorComponent;

UCLASS(MinimalAPI, BlueprintType, Within=CEEffectorComponent)
class UCEEffectorPushMode : public UCEEffectorModeBase
{
	GENERATED_BODY()

public:
	UCEEffectorPushMode()
		: UCEEffectorModeBase(TEXT("Push"), static_cast<int32>(ECEClonerEffectorMode::Push))
	{}

	UFUNCTION(BlueprintCallable, Category="Effector")
	CLONEREFFECTOR_API void SetPushDirection(ECEClonerEffectorPushDirection InDirection);

	UFUNCTION(BlueprintPure, Category="Effector")
	ECEClonerEffectorPushDirection GetPushDirection() const
	{
		return PushDirection;
	}

	UFUNCTION(BlueprintCallable, Category="Effector")
	CLONEREFFECTOR_API void SetPushStrength(const FVector& InStrength);

	UFUNCTION(BlueprintPure, Category="Effector")
	const FVector& GetPushStrength() const
	{
		return PushStrength;
	}

protected:
	//~ Begin UCEEffectorNoiseMode
	virtual void OnExtensionParametersChanged(UCEEffectorComponent* InComponent) override;
	//~ End UCEEffectorNoiseMode

	//~ Begin UObject
#if WITH_EDITOR
	virtual void PostEditChangeProperty(FPropertyChangedEvent& InPropertyChangedEvent) override;
#endif
	//~ End UObject

	/** Strength of the push effect */
	UPROPERTY(EditInstanceOnly, Setter, Getter, Category="Mode", meta=(MotionDesignVectorWidget, AllowPreserveRatio="XYZ"))
	FVector PushStrength = FVector(100.f);

	/** Relative direction computed for the push effect on each clone */
	UPROPERTY(EditInstanceOnly, Setter, Getter, Category="Mode")
	ECEClonerEffectorPushDirection PushDirection = ECEClonerEffectorPushDirection::Effector;

private:
#if WITH_EDITOR
	/** Used for PECP */
	static const TCEPropertyChangeDispatcher<UCEEffectorPushMode> PropertyChangeDispatcher;
#endif
};