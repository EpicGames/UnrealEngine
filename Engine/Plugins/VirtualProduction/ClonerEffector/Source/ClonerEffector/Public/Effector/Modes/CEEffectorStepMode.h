// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CEClonerEffectorShared.h"
#include "CEEffectorModeBase.h"
#include "CEPropertyChangeDispatcher.h"
#include "CEEffectorStepMode.generated.h"

class UCEEffectorComponent;

UCLASS(MinimalAPI, BlueprintType, Within=CEEffectorComponent)
class UCEEffectorStepMode : public UCEEffectorModeBase
{
	GENERATED_BODY()

public:
	UCEEffectorStepMode()
		: UCEEffectorModeBase(TEXT("Step"), static_cast<int32>(ECEClonerEffectorMode::Step))
	{}

	UFUNCTION(BlueprintCallable, Category="Effector")
	CLONEREFFECTOR_API void SetStepPosition(const FVector& InPosition);

	UFUNCTION(BlueprintPure, Category="Effector")
	FVector GetStepPosition() const
	{
		return StepPosition;
	}

	UFUNCTION(BlueprintCallable, Category="Effector")
	CLONEREFFECTOR_API void SetStepRotation(const FRotator& InRotation);

	UFUNCTION(BlueprintPure, Category="Effector")
	FRotator GetStepRotation() const
	{
		return StepRotation;
	}

	UFUNCTION(BlueprintCallable, Category="Effector")
	CLONEREFFECTOR_API void SetStepScale(const FVector& InScale);

	UFUNCTION(BlueprintPure, Category="Effector")
	FVector GetStepScale() const
	{
		return StepScale;
	}

protected:
	//~ Begin UObject
#if WITH_EDITOR
	virtual void PostEditChangeProperty(FPropertyChangedEvent& InPropertyChangedEvent) override;
#endif
	//~ End UObject

	//~ Begin UCEEffectorNoiseMode
	virtual void OnExtensionParametersChanged(UCEEffectorComponent* InComponent) override;
	//~ End UCEEffectorNoiseMode

	/** Interpolates from 0 to this position offset based on the particle index and particle count */
	UPROPERTY(EditInstanceOnly, Setter, Getter, DisplayName="Position", Category="Mode")
	FVector StepPosition = FVector::ZeroVector;

	/** Interpolates from 0 to this rotation based on the particle index and particle count */
	UPROPERTY(EditInstanceOnly, Setter, Getter, DisplayName="Rotation", Category="Mode")
	FRotator StepRotation = FRotator::ZeroRotator;

	/** Interpolates from 1 to this scale based on the particle index and particle count */
	UPROPERTY(EditInstanceOnly, Setter, Getter, DisplayName="Scale", Category="Mode", meta=(ClampMin="0", MotionDesignVectorWidget, AllowPreserveRatio="XYZ"))
	FVector StepScale = FVector::OneVector;

private:
#if WITH_EDITOR
	/** Used for PECP */
	static const TCEPropertyChangeDispatcher<UCEEffectorStepMode> PropertyChangeDispatcher;
#endif
};