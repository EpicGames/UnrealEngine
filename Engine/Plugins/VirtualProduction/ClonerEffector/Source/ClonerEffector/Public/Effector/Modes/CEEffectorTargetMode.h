// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CEClonerEffectorShared.h"
#include "CEEffectorModeBase.h"
#include "CEPropertyChangeDispatcher.h"
#include "CEEffectorTargetMode.generated.h"

class AActor;
class UCEEffectorComponent;

UCLASS(MinimalAPI, BlueprintType, Within=CEEffectorComponent)
class UCEEffectorTargetMode : public UCEEffectorModeBase
{
	GENERATED_BODY()

public:
	UCEEffectorTargetMode()
		: UCEEffectorModeBase(TEXT("Target"), static_cast<int32>(ECEClonerEffectorMode::Target))
	{}

	UFUNCTION(BlueprintCallable, Category="Effector")
	CLONEREFFECTOR_API void SetTargetActor(AActor* InTargetActor);

	UFUNCTION(BlueprintPure, Category="Effector")
	AActor* GetTargetActor() const
	{
		return TargetActorWeak.Get();
	}

	void SetTargetActorWeak(const TWeakObjectPtr<AActor>& InTargetActor);

	TWeakObjectPtr<AActor> GetTargetActorWeak() const
	{
		return TargetActorWeak;
	}

protected:
	//~ Begin UCEEffectorModeBase
	virtual void OnExtensionParametersChanged(UCEEffectorComponent* InComponent) override;
	virtual void OnExtensionDeactivated() override;
	virtual void OnExtensionActivated() override;
	//~ End UCEEffectorModeBase

	//~ Begin UObject
#if WITH_EDITOR
	virtual void PostEditChangeProperty(FPropertyChangedEvent& InPropertyChangedEvent) override;
#endif
	//~ End UObject

	/** The actor to track when mode is set to target */
	UPROPERTY(EditInstanceOnly, Setter, Getter, Category="Mode", meta=(DisplayName="TargetActor"))
	TWeakObjectPtr<AActor> TargetActorWeak = nullptr;

	UPROPERTY()
	TWeakObjectPtr<AActor> InternalTargetActorWeak = nullptr;

private:
	void OnTargetActorChanged();

	void OnTargetActorTransformChanged(USceneComponent* InUpdatedComponent, EUpdateTransformFlags InFlags, ETeleportType InTeleport);

	UFUNCTION()
	void OnTargetActorDestroyed(AActor* InActor);

#if WITH_EDITOR
	/** Used for PECP */
	static const TCEPropertyChangeDispatcher<UCEEffectorTargetMode> PropertyChangeDispatcher;
#endif
};