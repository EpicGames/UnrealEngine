// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CEClonerEffectorShared.h"
#include "CEEffectorTypeBase.h"
#include "CEPropertyChangeDispatcher.h"
#include "CEEffectorBoundType.generated.h"

UCLASS(MinimalAPI, Abstract, BlueprintType, Within=CEEffectorComponent)
class UCEEffectorBoundType : public UCEEffectorTypeBase
{
	GENERATED_BODY()

	friend class FCEEditorEffectorTypeDetailCustomization;

public:
	UCEEffectorBoundType()
		: UCEEffectorTypeBase(NAME_None, INDEX_NONE)
	{}

	UCEEffectorBoundType(FName InTypeName, int32 InTypeIdentifier)
		: UCEEffectorTypeBase(InTypeName, InTypeIdentifier)
	{}

	UFUNCTION(BlueprintCallable, Category="Effector")
	CLONEREFFECTOR_API void SetInvertType(bool bInInvert);

	UFUNCTION(BlueprintPure, Category="Effector")
	bool GetInvertType() const
	{
		return bInvertType;
	}

	UFUNCTION(BlueprintCallable, Category="Effector")
	CLONEREFFECTOR_API void SetEasing(ECEClonerEasing InEasing);

	UFUNCTION(BlueprintPure, Category="Effector")
	ECEClonerEasing GetEasing() const
	{
		return Easing;
	}

protected:
	//~ Begin UCEEffectorExtensionBase
	virtual void OnExtensionParametersChanged(UCEEffectorComponent* InComponent) override;
	virtual void OnExtensionActivated() override;
	virtual void OnExtensionDeactivated() override;
	//~ End UCEEffectorExtensionBase

	//~ Begin UObject
#if WITH_EDITOR
	virtual void PostEditChangeProperty(FPropertyChangedEvent& InPropertyChangedEvent) override;
#endif
	//~ End UObject
	
#if WITH_EDITOR
	void OnEffectorDeveloperSettingsChanged(UObject* InSettings, FPropertyChangedEvent& InEvent);
#endif

	/** Invert the type effect, instead of affecting the inside of a zone, will affect the outside */
	UPROPERTY(EditInstanceOnly, Setter="SetInvertType", Getter="GetInvertType", Category="Shape")
	bool bInvertType = false;

	/** Weight easing function applied to lerp transforms */
	UPROPERTY(EditInstanceOnly, Setter, Getter, Category="Shape")
	ECEClonerEasing Easing = ECEClonerEasing::Linear;

private:
#if WITH_EDITOR
	/** Used for PECP */
	static const TCEPropertyChangeDispatcher<UCEEffectorBoundType> PropertyChangeDispatcher;
#endif
};