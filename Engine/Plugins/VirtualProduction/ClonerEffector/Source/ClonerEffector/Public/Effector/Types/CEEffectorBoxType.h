// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CEClonerEffectorShared.h"
#include "CEEffectorBoundType.h"
#include "CEPropertyChangeDispatcher.h"
#include "CEEffectorBoxType.generated.h"

UCLASS(MinimalAPI, BlueprintType, Within=CEEffectorComponent)
class UCEEffectorBoxType : public UCEEffectorBoundType
{
	GENERATED_BODY()

	friend class FAvaEffectorActorVisualizer;

public:
	UCEEffectorBoxType()
		: UCEEffectorBoundType(TEXT("Box"), static_cast<int32>(ECEClonerEffectorType::Box))
	{}

	UFUNCTION(BlueprintCallable, Category="Effector")
	CLONEREFFECTOR_API void SetInnerExtent(const FVector& InExtent);

	UFUNCTION(BlueprintPure, Category="Effector")
	FVector GetInnerExtent() const
	{
		return InnerExtent;
	}

	UFUNCTION(BlueprintCallable, Category="Effector")
	CLONEREFFECTOR_API void SetOuterExtent(const FVector& InExtent);

	UFUNCTION(BlueprintPure, Category="Effector")
	FVector GetOuterExtent() const
	{
		return OuterExtent;
	}

protected:
	//~ Begin UCEEffectorExtensionBase
	virtual void OnExtensionParametersChanged(UCEEffectorComponent* InComponent) override;
	//~ End UCEEffectorExtensionBase

	//~ Begin UCEEffectorTypeBase
	virtual void OnExtensionVisualizerDirty(int32 InDirtyFlags) override;
	//~ End UCEEffectorTypeBase

	//~ Begin UObject
#if WITH_EDITOR
	virtual void PostEditChangeProperty(FPropertyChangedEvent& InPropertyChangedEvent) override;
	virtual void PreEditChange(FEditPropertyChain& InPropertyChain) override;
#endif
	//~ End UObject

	/** Inner extent of box, all clones inside will be affected with a maximum weight */
	UPROPERTY(EditInstanceOnly, Setter, Getter, Category="Shape", meta=(ClampMin="0", MotionDesignVectorWidget, AllowPreserveRatio="XYZ"))
	FVector InnerExtent = FVector(50.f);

	/** Outer extent of box, all clones outside will not be affected */
	UPROPERTY(EditInstanceOnly, Setter, Getter, Category="Shape", meta=(ClampMin="0", MotionDesignVectorWidget, AllowPreserveRatio="XYZ"))
	FVector OuterExtent = FVector(200.f);

private:
#if WITH_EDITOR
	/** Used for PECP */
	static const TCEPropertyChangeDispatcher<UCEEffectorBoxType> PropertyChangeDispatcher;
	static const TCEPropertyChangeDispatcher<UCEEffectorBoxType> PrePropertyChangeDispatcher;
#endif
};