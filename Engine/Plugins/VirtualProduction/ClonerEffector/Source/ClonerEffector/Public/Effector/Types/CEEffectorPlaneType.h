// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CEClonerEffectorShared.h"
#include "CEEffectorBoundType.h"
#include "CEPropertyChangeDispatcher.h"
#include "CEEffectorPlaneType.generated.h"

UCLASS(MinimalAPI, BlueprintType, Within=CEEffectorComponent)
class UCEEffectorPlaneType : public UCEEffectorBoundType
{
	GENERATED_BODY()

	friend class FAvaEffectorActorVisualizer;

public:
	UCEEffectorPlaneType()
		: UCEEffectorBoundType(TEXT("Plane"), static_cast<int32>(ECEClonerEffectorType::Plane))
	{}

	UFUNCTION(BlueprintCallable, Category="Effector")
	CLONEREFFECTOR_API void SetPlaneSpacing(float InSpacing);

	UFUNCTION(BlueprintPure, Category="Effector")
	float GetPlaneSpacing() const
	{
		return PlaneSpacing;
	}

protected:
	//~ Begin UCEEffectorExtensionBase
	virtual void OnExtensionParametersChanged(UCEEffectorComponent* InComponent) override;
	//~ End UCEEffectorExtensionBase

	//~ Begin UCEEffectorTypeBase
	virtual void OnExtensionVisualizerDirty(int32 InDirtyFlags) override;
	//~ End UCEEffectorTypeBase

	//~ Begin UObject
#if WITH_EDITOR
	virtual void PostEditChangeProperty(FPropertyChangedEvent& InPropertyChangedEvent) override;
	virtual void PreEditChange(FEditPropertyChain& InPropertyChain) override;
#endif
	//~ End UObject

	/** Plane spacing, everything inside this zone will be affected */
	UPROPERTY(EditInstanceOnly, Setter, Getter, Category="Shape", meta=(ClampMin="0"))
	float PlaneSpacing = 200.f;

private:
#if WITH_EDITOR
	/** Used for PECP */
	static const TCEPropertyChangeDispatcher<UCEEffectorPlaneType> PropertyChangeDispatcher;
	static const TCEPropertyChangeDispatcher<UCEEffectorPlaneType> PrePropertyChangeDispatcher;
#endif
};