// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CEClonerEffectorShared.h"
#include "CEEffectorBoundType.h"
#include "CEPropertyChangeDispatcher.h"
#include "CEEffectorRadialType.generated.h"

UCLASS(MinimalAPI, BlueprintType, Within=CEEffectorComponent)
class UCEEffectorRadialType : public UCEEffectorBoundType
{
	GENERATED_BODY()

	friend class FAvaEffectorActorVisualizer;

public:
	UCEEffectorRadialType()
		: UCEEffectorBoundType(TEXT("Radial"), static_cast<int32>(ECEClonerEffectorType::Radial))
	{}

	UFUNCTION(BlueprintCallable, Category="Effector")
	CLONEREFFECTOR_API void SetRadialAngle(float InAngle);

	UFUNCTION(BlueprintPure, Category="Effector")
	float GetRadialAngle() const
	{
		return RadialAngle;
	}

	UFUNCTION(BlueprintCallable, Category="Effector")
	CLONEREFFECTOR_API void SetRadialMinRadius(float InRadius);

	UFUNCTION(BlueprintPure, Category="Effector")
	float GetRadialMinRadius() const
	{
		return RadialMinRadius;
	}

	UFUNCTION(BlueprintCallable, Category="Effector")
	CLONEREFFECTOR_API void SetRadialMaxRadius(float InRadius);

	UFUNCTION(BlueprintPure, Category="Effector")
	float GetRadialMaxRadius() const
	{
		return RadialMaxRadius;
	}

protected:
	//~ Begin UCEEffectorExtensionBase
	virtual void OnExtensionParametersChanged(UCEEffectorComponent* InComponent) override;
	//~ End UCEEffectorExtensionBase

	//~ Begin UCEEffectorTypeBase
	virtual void OnExtensionVisualizerDirty(int32 InDirtyFlags) override;
	//~ End UCEEffectorTypeBase

	//~ Begin UObject
#if WITH_EDITOR
	virtual void PostEditChangeProperty(FPropertyChangedEvent& InPropertyChangedEvent) override;
	virtual void PreEditChange(FEditPropertyChain& InPropertyChain) override;
#endif
	//~ End UObject

	/** Radial angle in degree, everything within the angle will be affected */
	UPROPERTY(EditInstanceOnly, Setter, Getter, Category="Shape", meta=(ClampMin="0", ClampMax="360"))
	float RadialAngle = 180.f;

	/** Minimum radius for the radial effect to be applied on clones, below clones will not be affected */
	UPROPERTY(EditInstanceOnly, Setter, Getter, Category="Shape", meta=(ClampMin="0"))
	float RadialMinRadius = 0.f;

	/** Maximum radius for the radial effect to be applied on clones, above clones will not be affected */
	UPROPERTY(EditInstanceOnly, Setter, Getter, Category="Shape", meta=(ClampMin="0"))
	float RadialMaxRadius = 1000.f;

private:
#if WITH_EDITOR
	/** Used for PECP */
	static const TCEPropertyChangeDispatcher<UCEEffectorRadialType> PropertyChangeDispatcher;
	static const TCEPropertyChangeDispatcher<UCEEffectorRadialType> PrePropertyChangeDispatcher;
#endif
};