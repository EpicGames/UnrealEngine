// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CEClonerEffectorShared.h"
#include "CEEffectorBoundType.h"
#include "CEPropertyChangeDispatcher.h"
#include "CEEffectorSphereType.generated.h"

UCLASS(MinimalAPI, BlueprintType, Within=CEEffectorComponent)
class UCEEffectorSphereType : public UCEEffectorBoundType
{
	GENERATED_BODY()

	friend class FAvaEffectorActorVisualizer;

public:
	UCEEffectorSphereType()
		: UCEEffectorBoundType(TEXT("Sphere"), static_cast<int32>(ECEClonerEffectorType::Sphere))
	{}

	UFUNCTION(BlueprintCallable, Category="Effector")
	CLONEREFFECTOR_API void SetOuterRadius(float InRadius);

	UFUNCTION(BlueprintPure, Category="Effector")
	float GetOuterRadius() const
	{
		return OuterRadius;
	}

	UFUNCTION(BlueprintCallable, Category="Effector")
	CLONEREFFECTOR_API void SetInnerRadius(float InRadius);

	UFUNCTION(BlueprintPure, Category="Effector")
	float GetInnerRadius() const
	{
		return InnerRadius;
	}

protected:
	//~ Begin UCEEffectorExtensionBase
	virtual void OnExtensionParametersChanged(UCEEffectorComponent* InComponent) override;
	//~ End UCEEffectorExtensionBase

	//~ Begin UCEEffectorTypeBase
	virtual void OnExtensionVisualizerDirty(int32 InDirtyFlags) override;
	//~ End UCEEffectorTypeBase

	//~ Begin UObject
#if WITH_EDITOR
	virtual void PostEditChangeProperty(FPropertyChangedEvent& InPropertyChangedEvent) override;
	virtual void PreEditChange(FEditPropertyChain& InPropertyChain) override;
#endif
	//~ End UObject

	/** Inner radius of sphere, all clones inside will be affected with a maximum weight */
	UPROPERTY(EditInstanceOnly, Setter, Getter, Category="Shape", meta=(ClampMin="0"))
	float InnerRadius = 50.f;

	/** Outer radius of sphere, all clones outside will not be affected */
	UPROPERTY(EditInstanceOnly, Setter, Getter, Category="Shape", meta=(ClampMin="0"))
	float OuterRadius = 200.f;

private:
#if WITH_EDITOR
	/** Used for PECP */
	static const TCEPropertyChangeDispatcher<UCEEffectorSphereType> PropertyChangeDispatcher;
	static const TCEPropertyChangeDispatcher<UCEEffectorSphereType> PrePropertyChangeDispatcher;
#endif
};