// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CEClonerEffectorShared.h"
#include "CEEffectorBoundType.h"
#include "CEPropertyChangeDispatcher.h"
#include "CEEffectorTorusType.generated.h"

UCLASS(MinimalAPI, BlueprintType, Within=CEEffectorComponent)
class UCEEffectorTorusType : public UCEEffectorBoundType
{
	GENERATED_BODY()

	friend class FAvaEffectorActorVisualizer;

public:
	UCEEffectorTorusType()
		: UCEEffectorBoundType(TEXT("Torus"), static_cast<int32>(ECEClonerEffectorType::Torus))
	{}

	UFUNCTION(BlueprintCallable, Category="Effector")
	CLONEREFFECTOR_API void SetTorusRadius(float InRadius);

	UFUNCTION(BlueprintPure, Category="Effector")
	float GetTorusRadius() const
	{
		return TorusRadius;
	}

	UFUNCTION(BlueprintCallable, Category="Effector")
	CLONEREFFECTOR_API void SetTorusInnerRadius(float InRadius);

	UFUNCTION(BlueprintPure, Category="Effector")
	float GetTorusInnerRadius() const
	{
		return TorusInnerRadius;
	}

	UFUNCTION(BlueprintCallable, Category="Effector")
	CLONEREFFECTOR_API void SetTorusOuterRadius(float InRadius);

	UFUNCTION(BlueprintPure, Category="Effector")
	float GetTorusOuterRadius() const
	{
		return TorusOuterRadius;
	}

protected:
	//~ Begin UCEEffectorExtensionBase
	virtual void OnExtensionParametersChanged(UCEEffectorComponent* InComponent) override;
	//~ End UCEEffectorExtensionBase

	//~ Begin UCEEffectorTypeBase
	virtual void OnExtensionVisualizerDirty(int32 InDirtyFlags) override;
	//~ End UCEEffectorTypeBase

	//~ Begin UObject
#if WITH_EDITOR
	virtual void PostEditChangeProperty(FPropertyChangedEvent& InPropertyChangedEvent) override;
	virtual void PreEditChange(FEditPropertyChain& InPropertyChain) override;
#endif
	//~ End UObject

	/** Main torus radius from center to the edge where inner and outer tube will be revolved */
	UPROPERTY(EditInstanceOnly, Setter, Getter, Category="Shape", meta=(ClampMin="0"))
	float TorusRadius = 250.f;

	/** Minimum revolved radius for the torus effect, clones contained inside will be affected with a maximum weight */
	UPROPERTY(EditInstanceOnly, Setter, Getter, Category="Shape", meta=(ClampMin="0"))
	float TorusInnerRadius = 50.f;

	/** Maximum revolved radius for the torus effect, clones outside of it will not be affected */
	UPROPERTY(EditInstanceOnly, Setter, Getter, Category="Shape", meta=(ClampMin="0"))
	float TorusOuterRadius = 200.f;

private:
#if WITH_EDITOR
	/** Used for PECP */
	static const TCEPropertyChangeDispatcher<UCEEffectorTorusType> PropertyChangeDispatcher;
	static const TCEPropertyChangeDispatcher<UCEEffectorTorusType> PrePropertyChangeDispatcher;
#endif
};