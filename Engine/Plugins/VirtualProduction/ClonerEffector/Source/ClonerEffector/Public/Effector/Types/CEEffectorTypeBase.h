// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CEClonerEffectorShared.h"
#include "Effector/CEEffectorExtensionBase.h"

#if WITH_EDITOR
#include "Containers/Ticker.h"
#endif

#include "CEEffectorTypeBase.generated.h"

class UCEEffectorComponent;
class UDynamicMesh;

/** Represents a shape for an effector to affect clones on specific zones */
UCLASS(MinimalAPI, Abstract, BlueprintType, Within=CEEffectorComponent, meta=(Section="Shape", Priority=1))
class UCEEffectorTypeBase : public UCEEffectorExtensionBase
{
	GENERATED_BODY()

public:
	static constexpr int32 InnerVisualizerFlag = 1 << 0;
	static constexpr int32 OuterVisualizerFlag = 1 << 1;

	UCEEffectorTypeBase()
		: UCEEffectorTypeBase(NAME_None, INDEX_NONE)
	{}

	UCEEffectorTypeBase(FName InTypeName, int32 InTypeIdentifier)
		: UCEEffectorExtensionBase(
			InTypeName
		)
		, TypeIdentifier(InTypeIdentifier)
	{}

	int32 GetTypeIdentifier() const
	{
		return TypeIdentifier;
	}

protected:
	static int32 VisualizerFlagToIdentifier(int32 InVisualizerFlag);

	//~ Begin UObject
#if WITH_EDITOR
	virtual void PostEditUndo() override;
#endif
	//~ End UObject

	//~ Begin UCEEffectorExtensionBase
	virtual void OnExtensionParametersChanged(UCEEffectorComponent* InComponent) override;
	//~ End UCEEffectorExtensionBase

	//~ Begin UCEEffectorExtensionBase
	virtual void OnExtensionActivated() override;
	//~ End UCEEffectorExtensionBase

	void MarkVisualizerDirty(int32 InDirtyFlags);
	void UpdateVisualizer(int32 InVisualizerFlag, TFunctionRef<void(UDynamicMesh*)> InMeshFunction) const;

	/** Called when a visualizer is dirtied to update it **/
	virtual void OnExtensionVisualizerDirty(int32 InDirtyFlags) {}

#if WITH_EDITOR
	void OnVisualizerPropertyChanged();
	bool OnVisualizerTick(float InDeltaTime);
#endif

private:
	/** Unique identifier to pass it to niagara */
	UPROPERTY(Transient)
	int32 TypeIdentifier = 0;

#if WITH_EDITOR
	/** Dirty visualizer ids */
	int32 DirtyVisualizerFlags = 0;

	FTSTicker::FDelegateHandle VisualizerTickHandle;
#endif
};