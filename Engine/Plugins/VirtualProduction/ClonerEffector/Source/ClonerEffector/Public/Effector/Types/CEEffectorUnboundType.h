// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CEClonerEffectorShared.h"
#include "CEEffectorTypeBase.h"
#include "CEEffectorUnboundType.generated.h"

UCLASS(MinimalAPI, BlueprintType, Within=CEEffectorComponent)
class UCEEffectorUnboundType : public UCEEffectorTypeBase
{
	GENERATED_BODY()

public:
	UCEEffectorUnboundType()
		: UCEEffectorTypeBase(TEXT("Unbound"), static_cast<int32>(ECEClonerEffectorType::Unbound))
	{}

protected:
	//~ Begin UCEEffectorTypeBase
	virtual void OnExtensionVisualizerDirty(int32 InDirtyFlags) override;
	//~ End UCEEffectorTypeBase
};
