// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Cloner/Layouts/CEClonerLayoutBase.h"
#include "Containers/Map.h"
#include "Engine/Level.h"
#include "Subsystems/EngineSubsystem.h"
#include "Templates/SharedPointerFwd.h"
#include "UObject/ObjectKey.h"
#include "UObject/WeakObjectPtrTemplates.h"
#include "CEClonerSubsystem.generated.h"

class UCEEffectorComponent;
class AActor;
class ICEClonerAttachmentTreeBehavior;
class ICEClonerSceneTreeCustomResolver;
class UCEClonerExtensionBase;

UCLASS(MinimalAPI)
class UCEClonerSubsystem : public UEngineSubsystem
{
	GENERATED_BODY()

	DECLARE_MULTICAST_DELEGATE(FOnSubsystemInitialized)
	DECLARE_MULTICAST_DELEGATE_ThreeParams(FOnClonerSetEnabled, const UWorld* /** InWorld */, bool /** bInEnabled */, bool /** bInTransact */)

public:
	enum class ECEClonerActionFlags : uint8
	{
		None,
		ShouldTransact = 1 << 0,
		ShouldSelect = 1 << 1,
		All = ShouldTransact | ShouldSelect,
	};

	static FOnSubsystemInitialized::RegistrationType& OnSubsystemInitialized()
	{
		return OnSubsystemInitializedDelegate;
	}

	static FOnClonerSetEnabled::RegistrationType& OnClonerSetEnabled()
	{
		return OnClonerSetEnabledDelegate;
	}

	DECLARE_DELEGATE_RetVal_OneParam(TSharedPtr<ICEClonerSceneTreeCustomResolver>, FOnGetSceneTreeResolver, ULevel* /** Level */)
	static FOnGetSceneTreeResolver::RegistrationType& OnGetSceneTreeResolver()
	{
		return OnGetSceneTreeResolverDelegate;
	}

	/** Get this subsystem instance */
	CLONEREFFECTOR_API static UCEClonerSubsystem* Get();

	CLONEREFFECTOR_API bool RegisterLayoutClass(UClass* InClonerLayoutClass);

	CLONEREFFECTOR_API bool UnregisterLayoutClass(UClass* InClonerLayoutClass);

	CLONEREFFECTOR_API bool IsLayoutClassRegistered(UClass* InClonerLayoutClass);

	/** Get available cloner layout names to use in dropdown */
	TSet<FName> GetLayoutNames() const;

	/** Get available cloner layout classes */
	TSet<TSubclassOf<UCEClonerLayoutBase>> GetLayoutClasses() const;

	/** Based on a layout class, find layout name */
	FName FindLayoutName(TSubclassOf<UCEClonerLayoutBase> InLayoutClass) const;

	/** Based on a layout name, find layout class */
	TSubclassOf<UCEClonerLayoutBase> FindLayoutClass(FName InLayoutName) const;

	/** Creates a new layout instance for a cloner */
	UCEClonerLayoutBase* CreateNewLayout(FName InLayoutName, UCEClonerComponent* InCloner);

	CLONEREFFECTOR_API bool RegisterExtensionClass(UClass* InClass);

	CLONEREFFECTOR_API bool UnregisterExtensionClass(UClass* InClass);

	CLONEREFFECTOR_API bool IsExtensionClassRegistered(UClass* InClass) const;

	/** Get available cloner extension names to use */
	TSet<FName> GetExtensionNames() const;

	/** Get available cloner extension classes to use */
	TSet<TSubclassOf<UCEClonerExtensionBase>> GetExtensionClasses() const;

	/** Based on a extension class, find extension name */
	FName FindExtensionName(TSubclassOf<UCEClonerExtensionBase> InClass) const;

	/** Creates a new extension instance for a cloner */
	UCEClonerExtensionBase* CreateNewExtension(FName InExtensionName, UCEClonerComponent* InCloner);

	/** Set cloners state and optionally transact */
	CLONEREFFECTOR_API void SetClonersEnabled(const TSet<UCEClonerComponent*>& InCloners, bool bInEnable, bool bInShouldTransact);

	/** Set cloners state in world and optionally transact */
	CLONEREFFECTOR_API void SetLevelClonersEnabled(const UWorld* InWorld, bool bInEnable, bool bInShouldTransact);

#if WITH_EDITOR
	/** Converts cloners simulation to a mesh */
	CLONEREFFECTOR_API void ConvertCloners(const TSet<UCEClonerComponent*>& InCloners, ECEClonerMeshConversion InMeshConversion);
#endif

	/** Spawn linked effectors with a generator and optionally transact */
	CLONEREFFECTOR_API TArray<UCEEffectorComponent*> CreateLinkedEffectors(const TArray<UCEClonerComponent*>& InCloners, ECEClonerActionFlags InFlags, TFunctionRef<void(UCEEffectorComponent*)> InGenerator = [](UCEEffectorComponent*){});

	/** Creates a new cloner with actors attached */
	CLONEREFFECTOR_API UCEClonerComponent* CreateClonerWithActors(UWorld* InWorld, const TSet<AActor*>& InActors, ECEClonerActionFlags InFlags);

	/** Fires a warning about unset materials used within a cloner */
	void FireMaterialWarning(const AActor* InClonerActor, const AActor* InContextActor, const TArray<TWeakObjectPtr<UMaterialInterface>>& InUnsetMaterials);

	/** Register an attachment tree behavior */
	bool RegisterAttachmentTreeBehavior(FName InName, TFunction<TSharedRef<ICEClonerAttachmentTreeBehavior>()>& InCreator);

	/** Unregister an attachment tree behavior */
	bool UnregisterAttachmentTreeBehavior(FName InName);

	/** Gets all attachment behavior registered */
	TArray<FName> GetAttachmentTreeBehaviorNames() const;

	/** Creates the specific attachment tree behavior */
	TSharedPtr<ICEClonerAttachmentTreeBehavior> CreateAttachmentTreeBehavior(FName InName) const;

	/** Finds a custom scene tree resolver for a specific level if available */
	TSharedPtr<ICEClonerSceneTreeCustomResolver> FindCustomLevelSceneTreeResolver(ULevel* InLevel);

	/** Finds or loads the template niagara system to be used by the layout */
	const UNiagaraSystem* FindOrLoadTemplateNiagaraSystem(const UCEClonerLayoutBase* InLayout);

	/** Finds or loads the base niagara system all layouts use */
	const UNiagaraSystem* FindOrLoadBaseNiagaraSystem();

protected:
	CLONEREFFECTOR_API static FOnSubsystemInitialized OnSubsystemInitializedDelegate;

	/** Delegate to change state of cloners in a world */
	static FOnClonerSetEnabled OnClonerSetEnabledDelegate;

	CLONEREFFECTOR_API static FOnGetSceneTreeResolver OnGetSceneTreeResolverDelegate;

	//~ Begin USubsystem
	virtual void Initialize(FSubsystemCollectionBase& Collection) override;
	virtual void Deinitialize() override;
	//~ End USubsystem

	void ScanForRegistrableClasses();

	void OnLevelCleanup(ULevel* InLevel);
	void OnWorldCleanup(UWorld* InWorld, bool, bool bInCleanupResources);

	/** Linking name to the layout class */
	UPROPERTY()
	TMap<FName, TSubclassOf<UCEClonerLayoutBase>> LayoutClasses;

	/** Layout name to their template niagara systems */
	UPROPERTY()
	TMap<FName, TObjectPtr<UNiagaraSystem>> TemplateNiagaraSystems;

	/** Parent niagara system that all layouts are based from to validate user parameters */
	UPROPERTY()
	TObjectPtr<UNiagaraSystem> BaseNiagaraSystem;

	/** Linking name to the extension class */
	UPROPERTY()
	TMap<FName, TSubclassOf<UCEClonerExtensionBase>> ExtensionClasses;

	/** Used to create a cloner tree attachment behavior */
	TMap<FName, TFunction<TSharedRef<ICEClonerAttachmentTreeBehavior>()>> TreeBehaviorCreators;

	/** Used to gather ordered actors based on parent */
	TMap<TObjectKey<ULevel>, TSharedRef<ICEClonerSceneTreeCustomResolver>> LevelCustomResolvers;

#if WITH_EDITOR
	double LastNotificationTime = 0.0;
#endif
};