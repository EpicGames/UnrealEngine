// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Engine/World.h"
#include "Subsystems/EngineSubsystem.h"
#include "UObject/WeakObjectPtrTemplates.h"
#include "CEEffectorSubsystem.generated.h"

class UCEEffectorComponent;
class UCEEffectorExtensionBase;
class UNiagaraDataChannelAsset;

UCLASS()
class UCEEffectorSubsystem : public UEngineSubsystem, public FTickableGameObject
{
	GENERATED_BODY()

	DECLARE_MULTICAST_DELEGATE(FOnSubsystemInitialized)
	DECLARE_MULTICAST_DELEGATE_ThreeParams(FOnEffectorIdentifierChanged, UCEEffectorComponent* /** InEffector */, int32 /** OldIdentifier */, int32 /** NewIdentifier */)
	DECLARE_MULTICAST_DELEGATE_ThreeParams(FOnEffectorSetEnabled, const UWorld* /** InWorld */, bool /** bInEnabled */, bool /** bInTransact */)

public:
	static FOnSubsystemInitialized::RegistrationType& OnSubsystemInitialized()
	{
		return OnSubsystemInitializedDelegate;
	}

	static FOnEffectorIdentifierChanged::RegistrationType& OnEffectorIdentifierChanged()
	{
		return OnEffectorIdentifierChangedDelegate;
	}

	static FOnEffectorSetEnabled::RegistrationType& OnEffectorSetEnabled()
	{
		return OnEffectorSetEnabledDelegate;
	}

	/** Get this subsystem instance */
	CLONEREFFECTOR_API static UCEEffectorSubsystem* Get();

	/** Registers an effector actor to use it within a effector channel */
	bool RegisterChannelEffector(UCEEffectorComponent* InEffector);

	/** Unregister an effector actor used within a effector channel */
	bool UnregisterChannelEffector(UCEEffectorComponent* InEffector);

	/** Get the effector using this channel identifier */
	UCEEffectorComponent* GetEffectorByChannelIdentifier(int32 InIdentifier) const;

	bool RegisterExtensionClass(UClass* InClass);

	bool UnregisterExtensionClass(UClass* InClass);

	bool IsExtensionClassRegistered(UClass* InClass) const;

	template<typename InExtensionClass>
	TSet<FName> GetExtensionNames() const
	{
		return GetExtensionNames(InExtensionClass::StaticClass());
	}

	TSet<FName> GetExtensionNames(TSubclassOf<UCEEffectorExtensionBase> InExtensionClass) const;

	template<typename InExtensionClass>
	TSet<TSubclassOf<UCEEffectorExtensionBase>> GetExtensionClasses() const
	{
		return GetExtensionClasses(InExtensionClass::StaticClass());
	}

	TSet<TSubclassOf<UCEEffectorExtensionBase>> GetExtensionClasses(TSubclassOf<UCEEffectorExtensionBase> InExtensionClass) const;

	/** Based on a extension class, find extension name */
	FName FindExtensionName(TSubclassOf<UCEEffectorExtensionBase> InClass) const;

	/** Creates a new extension instance for an effector */
	UCEEffectorExtensionBase* CreateNewExtension(FName InExtensionName, UCEEffectorComponent* InEffector);

	/** Set effectors state and optionally transact */
	CLONEREFFECTOR_API void SetEffectorsEnabled(const TSet<UCEEffectorComponent*>& InEffectors, bool bInEnable, bool bInShouldTransact);

	/** Set effectors state in world and optionally transact */
	CLONEREFFECTOR_API void SetLevelEffectorsEnabled(const UWorld* InWorld, bool bInEnable, bool bInShouldTransact);

protected:
	static constexpr TCHAR DataChannelAssetPath[] = TEXT("/Script/Niagara.NiagaraDataChannelAsset'/ClonerEffector/Channels/NDC_Effector.NDC_Effector'");

	/** Broadcasted when this subsystem is initialized */
	static FOnSubsystemInitialized OnSubsystemInitializedDelegate;

	/** Broadcasted when this effector identifier changed to update linked cloners */
	static FOnEffectorIdentifierChanged OnEffectorIdentifierChangedDelegate;

	/** Delegate to change state of effectors in a world */
	static FOnEffectorSetEnabled OnEffectorSetEnabledDelegate;

	//~ Begin USubsystem
	virtual void Initialize(FSubsystemCollectionBase& Collection) override;
	virtual void Deinitialize() override;
	//~ End USubsystem

	//~ Begin FTickableGameObject
	virtual TStatId GetStatId() const override;
	virtual void Tick(float InDeltaSeconds) override;
	virtual bool IsTickable() const override;
	virtual bool IsTickableInEditor() const override { return true; }
	virtual ETickableTickType GetTickableTickType() const override { return ETickableTickType::Conditional; }
	//~ End FTickableGameObject

	/** Updates all registered effectors */
	void UpdateEffectorChannel(const UWorld* InWorld);

	/** Scan classes and registers them */
	void ScanForRegistrableClasses();

	/** Linking name to the extension class */
	UPROPERTY()
	TMap<FName, TSubclassOf<UCEEffectorExtensionBase>> ExtensionClasses;

	/** Ordered effectors included in this channel */
	UPROPERTY()
	TArray<TWeakObjectPtr<UCEEffectorComponent>> EffectorsWeak;

	/** This represents the data channel structure for effector */
	UPROPERTY()
	TObjectPtr<UNiagaraDataChannelAsset> EffectorDataChannelAsset;
};