// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "UObject/WeakObjectPtr.h"
#include "CEClonerEffectorShared.h"

class AActor;
class FText;
class UMaterialInterface;

namespace UE::ClonerEffector::Utilities
{
#if WITH_EDITOR
	const FText& GetMaterialWarningText();

	/** Show material warning notification when missing niagara usage flag */
	void ShowWarning(const FText& InWarning);
#endif

	/** Only materials transient or part of the content folder can be dirtied, engine or plugins cannot */
	bool IsMaterialDirtyable(const UMaterialInterface* InMaterial);

	/** Check if material has niagara usage flag set */
	bool IsMaterialUsageFlagSet(const UMaterialInterface* InMaterial);

	/** Replaces all unsupported material by default material, gathers unset materials that needs recompiling with proper flags */
	bool FilterSupportedMaterials(TArray<TWeakObjectPtr<UMaterialInterface>>& InMaterials, TArray<TWeakObjectPtr<UMaterialInterface>>& OutUnsetMaterials, UMaterialInterface* InDefaultMaterial);

	/** Check if a material is supported otherwise replaces it with default material */
	bool FilterSupportedMaterial(UMaterialInterface*& InMaterial, UMaterialInterface* InDefaultMaterial);

	/** Set specific actor visibility */
	void SetActorVisibility(AActor* InActor, bool bInVisibility, ECEClonerActorVisibility InTarget = ECEClonerActorVisibility::All);

	/** Look parent attach actor until first cloner component is found */
	AActor* FindClonerActor(AActor* InActor);
}
