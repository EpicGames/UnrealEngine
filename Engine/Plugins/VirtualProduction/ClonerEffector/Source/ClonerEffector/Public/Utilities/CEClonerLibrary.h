// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/ContainersFwd.h"
#include "Kismet/BlueprintFunctionLibrary.h"
#include "Templates/SubclassOf.h"
#include "CEClonerLibrary.generated.h"

class UCEClonerComponent;
class UCEClonerExtensionBase;
class UCEClonerLayoutBase;
struct FLatentActionInfo;

/** Blueprint operations for cloner */
UCLASS(MinimalAPI, DisplayName="Motion Design Cloner Library")
class UCEClonerLibrary : public UBlueprintFunctionLibrary
{
	GENERATED_BODY()

public:
	/**
	 * Retrieves all layout classes available for a cloner
	 * @param OutLayoutClasses [Out] Layout classes available
	 */
	UFUNCTION(BlueprintCallable, Category="Motion Design|Cloner|Utility")
	static CLONEREFFECTOR_API void GetClonerLayoutClasses(TSet<TSubclassOf<UCEClonerLayoutBase>>& OutLayoutClasses);

	/**
	 * Retrieves all extension classes available for a cloner
	 * @param OutExtensionClasses [Out] Extension classes available
	 */
	UFUNCTION(BlueprintCallable, Category="Motion Design|Cloner|Utility")
	static CLONEREFFECTOR_API void GetClonerExtensionClasses(TSet<TSubclassOf<UCEClonerExtensionBase>>& OutExtensionClasses);

	/**
	 * Retrieves the layout name from a layout class
	 * @param InLayoutClass Layout class to get the name from
	 * @param OutLayoutName [Out] Layout name
	 * @return true when the layout name was retrieved
	 */
	UFUNCTION(BlueprintCallable, Category="Motion Design|Cloner|Utility")
	static CLONEREFFECTOR_API bool GetClonerLayoutName(TSubclassOf<UCEClonerLayoutBase> InLayoutClass, FName& OutLayoutName);

	/**
	 * Retrieves all the layout names
	 * @param OutLayoutNames [Out] Layout names
	 * @return true when the layout names were retrieved
	 */
	UFUNCTION(BlueprintCallable, Category="Motion Design|Cloner|Utility")
	static CLONEREFFECTOR_API bool GetClonerLayoutNames(TSet<FName>& OutLayoutNames);

	/**
	 * Retrieves the layout class from a layout name
	 * @param InLayoutName Layout name to get the class from
	 * @param OutLayoutClass [Out] Layout class
	 * @return true when the layout class was retrieved
	 */
	UFUNCTION(BlueprintCallable, Category="Motion Design|Cloner|Utility")
	static CLONEREFFECTOR_API bool GetClonerLayoutClass(FName InLayoutName, TSubclassOf<UCEClonerLayoutBase>& OutLayoutClass);


	/**
	 * Sets the active layout of a cloner and wait until the layout is loaded and active
	 * @param InWorldContext World context object
	 * @param InLatentInfo Latent action info 
	 * @param InCloner Target cloner component
	 * @param InLayoutClass Cloner layout class
	 * @param bOutSuccess [Out] True when the layout class is set
	 * @param OutLayout [Out] Layout object corresponding to the layout class 
	 */
	UFUNCTION(BlueprintCallable, Category="Motion Design|Cloner|Utility", meta=(Latent, LatentInfo="InLatentInfo", WorldContext="InWorldContext"))
	static CLONEREFFECTOR_API void SetClonerLayoutByClass(const UObject* InWorldContext, FLatentActionInfo InLatentInfo, UCEClonerComponent* InCloner, TSubclassOf<UCEClonerLayoutBase> InLayoutClass, bool& bOutSuccess, UCEClonerLayoutBase*& OutLayout);

	/**
	 * Sets the active layout of a cloner and wait until the layout is loaded and active
	 * @param InWorldContext World context object
	 * @param InLatentInfo Latent action info 
	 * @param InCloner Target cloner component
	 * @param InLayoutName Cloner layout name
	 * @param bOutSuccess [Out] True when the layout class is set
	 * @param OutLayout [Out] Layout object corresponding to the layout class 
	 */
	UFUNCTION(BlueprintCallable, Category="Motion Design|Cloner|Utility", meta=(Latent, LatentInfo="InLatentInfo", WorldContext="InWorldContext"))
	static CLONEREFFECTOR_API void SetClonerLayoutByName(const UObject* InWorldContext, FLatentActionInfo InLatentInfo, UCEClonerComponent* InCloner, FName InLayoutName, bool& bOutSuccess, UCEClonerLayoutBase*& OutLayout);
};