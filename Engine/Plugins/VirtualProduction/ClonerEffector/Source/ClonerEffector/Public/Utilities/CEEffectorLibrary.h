// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/ContainersFwd.h"
#include "Kismet/BlueprintFunctionLibrary.h"
#include "Templates/SubclassOf.h"
#include "CEEffectorLibrary.generated.h"

class UCEEffectorEffectBase;
class UCEEffectorExtensionBase;
class UCEEffectorTypeBase;
class UCEEffectorModeBase;

/** Blueprint operations for effector */
UCLASS(MinimalAPI, DisplayName="Motion Design Effector Library")
class UCEEffectorLibrary : public UBlueprintFunctionLibrary
{
	GENERATED_BODY()

public:
	/**
	 * Retrieves all mode classes available for an effector
	 * @param OutModeClasses [Out] Available mode classes
	 */
	UFUNCTION(BlueprintCallable, Category="Motion Design|Effector|Utility")
	static CLONEREFFECTOR_API void GetEffectorModeClasses(TSet<TSubclassOf<UCEEffectorModeBase>>& OutModeClasses);

	/**
	 * Retrieves all type classes available for an effector
	 * @param OutTypeClasses [Out] Available type classes
	 */
	UFUNCTION(BlueprintCallable, Category="Motion Design|Effector|Utility")
	static CLONEREFFECTOR_API void GetEffectorTypeClasses(TSet<TSubclassOf<UCEEffectorTypeBase>>& OutTypeClasses);

	/**
	 * Retrieves all type classes available for an effector
	 * @param OutTypeNames [Out] Available type names
	 */
	UFUNCTION(BlueprintCallable, Category="Motion Design|Effector|Utility")
	static CLONEREFFECTOR_API void GetEffectorTypeNames(TSet<FName>& OutTypeNames);

	/**
	 * Retrieves all effect classes available for an effector
	 * @param OutEffectClasses [Out] Available effect classes
	 */
	UFUNCTION(BlueprintCallable, Category="Motion Design|Effector|Utility")
	static CLONEREFFECTOR_API void GetEffectorEffectClasses(TSet<TSubclassOf<UCEEffectorEffectBase>>& OutEffectClasses);
};