// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class ClonerEffectorEditor : ModuleRules
{
    public ClonerEffectorEditor(ReadOnlyTargetRules Target) : base(Target)
    {
        PCHUsage = ModuleRules.PCHUsageMode.UseExplicitOrSharedPCHs;

        PublicDependencyModuleNames.AddRange(
	        new string[]
	        {
		        "Core",
		        "CoreUObject",
				"EditorSubsystem",
		        "Engine",
		        "PropertyEditor",
		        "UnrealEd"
	        }
        );

        PrivateDependencyModuleNames.AddRange(
	        new string[]
	        {
		        "ClonerEffector",
		        "InputCore",
				"MovieScene",
				"Niagara",
		        "NiagaraEditor",
		        "NiagaraSimCaching",
		        "Projects",
				"Sequencer",
				"Slate",
				"SlateCore",
				"ToolMenus"
	        }
        );

        ShortName = "CEEditor";
    }
}