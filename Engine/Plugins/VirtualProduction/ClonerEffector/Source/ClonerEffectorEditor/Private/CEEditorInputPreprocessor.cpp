// Copyright Epic Games, Inc. All Rights Reserved.

#include "CEEditorInputPreprocessor.h"

#include "Containers/Ticker.h"
#include "Framework/Application/SlateApplication.h"
#include "Widgets/SWidget.h"
#include "Widgets/SWindow.h"

void FCEEditorInputPreprocessor::RefocusLastWidget()
{
	if (!FSlateApplication::IsInitialized() || !bShouldRefocus)
	{
		return;
	}

	bShouldRefocus = false;

	const FSlateApplication& SlateApp = FSlateApplication::Get();

	if (const TSharedPtr<SWidget> FocusedWidget = SlateApp.GetKeyboardFocusedWidget())
	{
		const FVector2D ScreenPos = FocusedWidget->GetCachedGeometry().GetAbsolutePosition() + FocusedWidget->GetCachedGeometry().GetLocalSize() / 2;

		FTSTicker::GetCoreTicker().AddTicker(FTickerDelegate::CreateSPLambda(this, [ScreenPos](float)->bool
		{
			if (!FSlateApplication::IsInitialized())
			{
				return false;
			}

			FSlateApplication& SlateApp = FSlateApplication::Get();

			// Hit-test at the event position
			const FWidgetPath WidgetPath = SlateApp.LocateWindowUnderMouse(ScreenPos, SlateApp.GetInteractiveTopLevelWindows());

			if (WidgetPath.IsValid() && WidgetPath.Widgets.Num() > 0)
			{
				// Last in the path is the most specific widget under the cursor
				TSharedRef<SWidget> TargetWidget = WidgetPath.Widgets.Last().Widget;
				SlateApp.SetAllUserFocus(TargetWidget);
			}

			return false;
		}));
	}
}

void FCEEditorInputPreprocessor::Register()
{
	if (!bRegistered && FSlateApplication::IsInitialized())
	{
		bRegistered = FSlateApplication::Get().RegisterInputPreProcessor(SharedThis(this));
	}
}

void FCEEditorInputPreprocessor::Unregister()
{
	if (bRegistered && FSlateApplication::IsInitialized())
	{
		FSlateApplication::Get().UnregisterInputPreProcessor(SharedThis(this));
		bRegistered = false;
	}
}
