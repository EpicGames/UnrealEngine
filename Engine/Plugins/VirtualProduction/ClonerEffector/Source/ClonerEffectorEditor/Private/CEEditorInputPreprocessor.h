// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Framework/Application/IInputProcessor.h"
#include "Input/Events.h"
#include "InputCoreTypes.h"
#include "Templates/SharedPointer.h"

/**
 * This is a fix used to refocus last selected widget in the details panel
 * Since a details refresh causes property value widget to lose focus
 * This should be removed in the future once PropertyValueWidget can be refocused through the API or refresh does not cause a focus lost
 */
class FCEEditorInputPreprocessor : public IInputProcessor, public TSharedFromThis<FCEEditorInputPreprocessor>
{
public:
	//~ Begin IInputProcessor
	virtual void Tick(const float InDeltaTime, FSlateApplication& InSlateApp, TSharedRef<ICursor> InCursor) override
	{
	};

	virtual bool HandleKeyDownEvent(FSlateApplication& InSlateApp, const FKeyEvent& InKeyEvent) override
	{
		bShouldRefocus = InKeyEvent.GetKey() == EKeys::Tab || InKeyEvent.GetKey() == EKeys::Enter;
		// Return false to allow other processors to handle the event
		return false;
	}

	virtual bool HandleMouseButtonDownEvent(FSlateApplication& InSlateApp, const FPointerEvent& InMouseEvent) override
	{
		bShouldRefocus = false;
		// Return false to allow other processors to handle the event
		return false;
	}

	virtual const TCHAR* GetDebugName() const override
	{
		return TEXT("ClonerEffector");
	}
	//~ End IInputProcessor

	void RefocusLastWidget();
	void Register();
	void Unregister();

private:
	bool bShouldRefocus = false;
	bool bRegistered = false;
};