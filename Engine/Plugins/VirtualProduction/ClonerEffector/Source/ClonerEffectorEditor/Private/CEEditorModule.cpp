// Copyright Epic Games, Inc. All Rights Reserved.

#include "CEEditorModule.h"

#include "CEEditorInputPreprocessor.h"
#include "CEEditorThrottleManager.h"
#include "Cloner/CEClonerComponent.h"
#include "Cloner/CEClonerActor.h"
#include "Cloner/Customizations/CEEditorClonerActorDetailCustomization.h"
#include "Cloner/Customizations/CEEditorClonerComponentDetailCustomization.h"
#include "Cloner/Customizations/CEEditorClonerEffectorExtensionDetailCustomization.h"
#include "Cloner/Customizations/CEEditorClonerLifetimeExtensionDetailCustomization.h"
#include "Cloner/Customizations/CEEditorClonerMeshLayoutDetailCustomization.h"
#include "Cloner/Customizations/CEEditorClonerSplineLayoutDetailCustomization.h"
#include "Cloner/Extensions/CEClonerEffectorExtension.h"
#include "Cloner/Extensions/CEClonerLifetimeExtension.h"
#include "Cloner/Layouts/CEClonerMeshLayout.h"
#include "Cloner/Layouts/CEClonerSplineLayout.h"
#include "Cloner/Sequencer/MovieSceneClonerTrackEditor.h"
#include "Effector/Customizations/CEEditorEffectorComponentDetailCustomization.h"
#include "Effector/Customizations/CEEditorEffectorTypeDetailCustomization.h"
#include "Effector/CEEffectorActor.h"
#include "Effector/CEEffectorComponent.h"
#include "Effector/Customizations/CEEditorEffectorActorDetailCustomization.h"
#include "Effector/Types/CEEffectorBoundType.h"
#include "ISequencerModule.h"
#include "Modules/ModuleManager.h"
#include "PropertyEditorModule.h"
#include "Styles/CEEditorStyle.h"

void FCEEditorModule::StartupModule()
{
	// Load styles
	FCEEditorStyle::Get();

	FPropertyEditorModule& PropertyModule = FModuleManager::LoadModuleChecked<FPropertyEditorModule>(TEXT("PropertyEditor"));

	// Cloner customization
	PropertyModule.RegisterCustomClassLayout(CustomizationNames.Add_GetRef(ACEClonerActor::StaticClass()->GetFName()), FOnGetDetailCustomizationInstance::CreateStatic(&FCEEditorClonerActorDetailCustomization::MakeInstance));
	PropertyModule.RegisterCustomClassLayout(CustomizationNames.Add_GetRef(UCEClonerComponent::StaticClass()->GetFName()), FOnGetDetailCustomizationInstance::CreateStatic(&FCEEditorClonerComponentDetailCustomization::MakeInstance));
	PropertyModule.RegisterCustomClassLayout(CustomizationNames.Add_GetRef(UCEClonerEffectorExtension::StaticClass()->GetFName()), FOnGetDetailCustomizationInstance::CreateStatic(&FCEEditorClonerEffectorExtensionDetailCustomization::MakeInstance));
	PropertyModule.RegisterCustomClassLayout(CustomizationNames.Add_GetRef(UCEClonerSplineLayout::StaticClass()->GetFName()), FOnGetDetailCustomizationInstance::CreateStatic(&FCEEditorClonerSplineLayoutDetailCustomization::MakeInstance));
	PropertyModule.RegisterCustomClassLayout(CustomizationNames.Add_GetRef(UCEClonerMeshLayout::StaticClass()->GetFName()), FOnGetDetailCustomizationInstance::CreateStatic(&FCEEditorClonerMeshLayoutDetailCustomization::MakeInstance));
	PropertyModule.RegisterCustomClassLayout(CustomizationNames.Add_GetRef(UCEClonerLifetimeExtension::StaticClass()->GetFName()), FOnGetDetailCustomizationInstance::CreateStatic(&FCEEditorClonerLifetimeExtensionDetailCustomization::MakeInstance));

	InputPreprocessor = MakeShared<FCEEditorInputPreprocessor>();

	// Effector customization
	PropertyModule.RegisterCustomClassLayout(CustomizationNames.Add_GetRef(ACEEffectorActor::StaticClass()->GetFName()), FOnGetDetailCustomizationInstance::CreateStatic(&FCEEditorEffectorActorDetailCustomization::MakeInstance));
	PropertyModule.RegisterCustomClassLayout(CustomizationNames.Add_GetRef(UCEEffectorComponent::StaticClass()->GetFName()), FOnGetDetailCustomizationInstance::CreateStatic(&FCEEditorEffectorComponentDetailCustomization::MakeInstance));
	PropertyModule.RegisterCustomClassLayout(CustomizationNames.Add_GetRef(UCEEffectorBoundType::StaticClass()->GetFName()), FOnGetDetailCustomizationInstance::CreateStatic(&FCEEditorEffectorTypeDetailCustomization::MakeInstance));

	// Custom cloner track
	ISequencerModule& SequencerModule = FModuleManager::LoadModuleChecked<ISequencerModule>(TEXT("Sequencer"));
	ClonerTrackCreateEditorHandle = SequencerModule.RegisterTrackEditor(FOnCreateTrackEditor::CreateStatic(&FMovieSceneClonerTrackEditor::CreateTrackEditor));

	// Disable slate throttling for interactive changes
	ThrottleManager = MakeShared<FCEEditorThrottleManager>();
	ThrottleManager->Init();
}

void FCEEditorModule::ShutdownModule()
{
	if (FPropertyEditorModule* PropertyModule = FModuleManager::GetModulePtr<FPropertyEditorModule>("PropertyEditor"))
	{
		for (const FName CustomizationName : CustomizationNames)
		{
			PropertyModule->UnregisterCustomClassLayout(CustomizationName);
		}

		CustomizationNames.Empty();
	}

	// Custom cloner track
	if (ISequencerModule* SequencerModule = FModuleManager::GetModulePtr<ISequencerModule>(TEXT("Sequencer")))
	{
		SequencerModule->UnRegisterTrackEditor(ClonerTrackCreateEditorHandle);
		ClonerTrackCreateEditorHandle.Reset();
	}

	if (InputPreprocessor.IsValid())
	{
		InputPreprocessor->Unregister();
		InputPreprocessor.Reset();
	}

	ThrottleManager.Reset();
	ThrottleManager = nullptr;
}

TSharedPtr<FCEEditorInputPreprocessor> FCEEditorModule::GetInputPreprocessor()
{
	if (FCEEditorModule* EditorModule = FModuleManager::GetModulePtr<FCEEditorModule>(UE_MODULE_NAME))
	{
		return EditorModule->InputPreprocessor;
	}

	return nullptr;
}

IMPLEMENT_MODULE(FCEEditorModule, ClonerEffectorEditor)
