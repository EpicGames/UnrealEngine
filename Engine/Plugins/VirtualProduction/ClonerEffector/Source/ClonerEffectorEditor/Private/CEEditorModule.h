// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Modules/ModuleManager.h"

class FCEEditorInputPreprocessor;
class FCEEditorThrottleManager;

class FCEEditorModule : public IModuleInterface
{
public:
	//~ Begin IModuleInterface
    virtual void StartupModule() override;
    virtual void ShutdownModule() override;
	//~ End IModuleInterface

	static TSharedPtr<FCEEditorInputPreprocessor> GetInputPreprocessor();

protected:
	FDelegateHandle ClonerTrackCreateEditorHandle;
	TSharedPtr<FCEEditorThrottleManager> ThrottleManager;
	TSharedPtr<FCEEditorInputPreprocessor> InputPreprocessor;
	TArray<FName> CustomizationNames;
};
