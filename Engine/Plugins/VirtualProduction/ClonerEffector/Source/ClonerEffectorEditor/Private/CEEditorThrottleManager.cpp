// Copyright Epic Games, Inc. All Rights Reserved.

#include "CEEditorThrottleManager.h"

#include "Application/ThrottleManager.h"
#include "Cloner/CEClonerActor.h"
#include "Effector/CEEffectorActor.h"
#include "Misc/Optional.h"
#include "Templates/SharedPointer.h"
#include "UObject/Object.h"
#include "UObject/UObjectGlobals.h"

FCEEditorThrottleScope::FCEEditorThrottleScope(FName InPropertyName)
	: PropertyName(InPropertyName)
{
	FSlateThrottleManager::Get().DisableThrottle(true);
}

FCEEditorThrottleScope::~FCEEditorThrottleScope()
{
	FSlateThrottleManager::Get().DisableThrottle(false);
}

FCEEditorThrottleManager::~FCEEditorThrottleManager()
{
	Stop();
}

void FCEEditorThrottleManager::Init()
{
	FCoreUObjectDelegates::OnObjectPropertyChanged.AddSP(this, &FCEEditorThrottleManager::OnPostPropertyChanged);
}

void FCEEditorThrottleManager::Stop()
{
	FCoreUObjectDelegates::OnObjectPropertyChanged.RemoveAll(this);
}

void FCEEditorThrottleManager::OnPostPropertyChanged(UObject* InObject, FPropertyChangedEvent& InEvent)
{
	if (!IsValid(InObject))
	{
		return;
	}

	const AActor* Owner = InObject->GetTypedOuter<AActor>();
	if (!Owner || (!Owner->IsA<ACEClonerActor>() && !Owner->IsA<ACEEffectorActor>()))
	{
		return;
	}

	// Due to multiple events chaining, use property name to filter events
	const FName PropertyName = InEvent.GetMemberPropertyName();
	if (PropertyName.IsNone())
	{
		return;
	}

	// Disable throttling after first interactive event and enable it back once value is set
	if (InEvent.ChangeType == EPropertyChangeType::Interactive)
	{
		if (!ThrottleScope.IsSet())
		{
			// Disable throttling
			ThrottleScope.Emplace(PropertyName);
		}

		return;
	}

	if (!ThrottleScope.IsSet() || !ThrottleScope->GetPropertyName().IsEqual(PropertyName))
	{
		return;
	}

	// Enable throttling
	ThrottleScope.Reset();
}
