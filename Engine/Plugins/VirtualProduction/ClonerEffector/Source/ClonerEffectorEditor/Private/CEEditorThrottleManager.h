// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Misc/OptionalFwd.h"
#include "Templates/SharedPointer.h"
#include "UObject/NameTypes.h"

class UObject;
struct FPropertyChangedEvent;

struct FCEEditorThrottleScope
{
	explicit FCEEditorThrottleScope(FName InPropertyName);

	virtual ~FCEEditorThrottleScope();

	FName GetPropertyName() const
	{
		return PropertyName;
	}

private:
	/** Property that is updating */
	FName PropertyName;
};

/**
 * Used to allow preview when interactively changing a property within cloner/effector
 * Disable slate throttling to allow viewport updates
 */
class FCEEditorThrottleManager : public TSharedFromThis<FCEEditorThrottleManager>
{
public:
	FCEEditorThrottleManager() = default;
	virtual ~FCEEditorThrottleManager();

	void Init();

	void Stop();

private:
	void OnPostPropertyChanged(UObject* InObject, FPropertyChangedEvent& InEvent);

	TOptional<FCEEditorThrottleScope> ThrottleScope;
};