// Copyright Epic Games, Inc. All Rights Reserved.

#include "Cloner/CEClonerActorFactory.h"
#include "Cloner/CEClonerActor.h"
#include "Cloner/CEClonerComponent.h"
#include "EngineAnalytics.h"
#include "Subsystems/PlacementSubsystem.h"

UCEClonerActorFactory::UCEClonerActorFactory()
{
	NewActorClass = ACEClonerActor::StaticClass();
}

void UCEClonerActorFactory::SetClonerLayout(FName InLayoutName)
{
	ClonerLayoutName = InLayoutName;
}

void UCEClonerActorFactory::PostSpawnActor(UObject* InAsset, AActor* InNewActor)
{
	Super::PostSpawnActor(InAsset, InNewActor);

	if (!ClonerLayoutName.IsNone())
	{
		if (ACEClonerActor* ClonerActor = Cast<ACEClonerActor>(InNewActor))
		{
			ClonerActor->SpawnDefaultActorAttached();

			if (UCEClonerComponent* ClonerComponent = ClonerActor->GetClonerComponent())
			{
				ClonerComponent->SetLayoutName(ClonerLayoutName);
			}
		}
	}
}

void UCEClonerActorFactory::PostPlaceAsset(TArrayView<const FTypedElementHandle> InHandle, const FAssetPlacementInfo& InPlacementInfo, const FPlacementOptions& InPlacementOptions)
{
	Super::PostPlaceAsset(InHandle, InPlacementInfo, InPlacementOptions);

	if (!InPlacementOptions.bIsCreatingPreviewElements && FEngineAnalytics::IsAvailable())
	{
		TArray<FAnalyticsEventAttribute> Attributes;
		Attributes.Reserve(2);
		Attributes.Emplace(TEXT("ToolClass"), GetNameSafe(GetClass()));
		Attributes.Emplace(TEXT("ActorClass"), GetNameSafe(NewActorClass));
		FEngineAnalytics::GetProvider().RecordEvent(TEXT("Editor.Usage.MotionDesign.PlaceActor"), Attributes);
	}
}
