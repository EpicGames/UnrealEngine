// Copyright Epic Games, Inc. All Rights Reserved.

#include "Cloner/Customizations/CEEditorClonerActorDetailCustomization.h"

#include "Cloner/CEClonerActor.h"
#include "Modules/ModuleManager.h"
#include "PropertyEditorModule.h"

void FCEEditorClonerActorDetailCustomization::CustomizeDetails(IDetailLayoutBuilder& InDetailBuilder)
{
	// Add customizations for actor here
}

void FCEEditorClonerActorDetailCustomization::RemoveEmptySections()
{
	FPropertyEditorModule& PropertyModule = FModuleManager::GetModuleChecked<FPropertyEditorModule>("PropertyEditor");
	const FName ComponentClassName = ACEClonerActor::StaticClass()->GetFName();

	// Remove Streaming section by removing its categories
	TSharedRef<FPropertySection> StreamingSection = PropertyModule.FindOrCreateSection(ComponentClassName, "Streaming", FText::GetEmpty());
	StreamingSection->RemoveCategory("WorldPartition");
	StreamingSection->RemoveCategory("DataLayers");
	StreamingSection->RemoveCategory("HLOD");

	// Remove Effects section by removing its categories
	TSharedRef<FPropertySection> EffectsSection = PropertyModule.FindOrCreateSection(ComponentClassName, "Effects", FText::GetEmpty());
	EffectsSection->RemoveCategory("Niagara");
	EffectsSection->RemoveCategory("NiagaraComponent_Parameters");
	EffectsSection->RemoveCategory("NiagaraComponent_Utilities");
	EffectsSection->RemoveCategory("Activation");
	EffectsSection->RemoveCategory("Parameters");
	EffectsSection->RemoveCategory("Randomness");
	EffectsSection->RemoveCategory("Warmup");
}
