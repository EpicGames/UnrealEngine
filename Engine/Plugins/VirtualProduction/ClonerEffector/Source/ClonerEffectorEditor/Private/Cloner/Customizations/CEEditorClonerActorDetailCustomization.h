// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IDetailCustomization.h"
#include "Templates/SharedPointer.h"

/** Used to customize cloner actor properties in details panel */
class FCEEditorClonerActorDetailCustomization : public IDetailCustomization
{
public:
	static TSharedRef<IDetailCustomization> MakeInstance()
	{
		return MakeShared<FCEEditorClonerActorDetailCustomization>();
	}

	explicit FCEEditorClonerActorDetailCustomization()
	{
		RemoveEmptySections();
	}

	//~ Begin IDetailCustomization
	virtual void CustomizeDetails(IDetailLayoutBuilder& InDetailBuilder) override;
	//~ End IDetailCustomization

protected:
	static void RemoveEmptySections();
};
