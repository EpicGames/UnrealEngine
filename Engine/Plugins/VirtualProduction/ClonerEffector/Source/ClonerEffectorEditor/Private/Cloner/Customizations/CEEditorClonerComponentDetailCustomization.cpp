// Copyright Epic Games, Inc. All Rights Reserved.

#include "Cloner/Customizations/CEEditorClonerComponentDetailCustomization.h"

#include "CEEditorInputPreprocessor.h"
#include "CEEditorModule.h"
#include "Cloner/CEClonerComponent.h"
#include "Cloner/Extensions/CEClonerExtensionBase.h"
#include "Cloner/Layouts/CEClonerLayoutBase.h"
#include "Cloner/Sequencer/MovieSceneClonerTrackEditor.h"
#include "DetailLayoutBuilder.h"
#include "DetailWidgetRow.h"
#include "IDetailChildrenBuilder.h"
#include "IDetailGroup.h"
#include "Input/Reply.h"
#include "IPropertyUtilities.h"
#include "Modules/ModuleManager.h"
#include "PropertyEditorModule.h"
#include "UObject/Class.h"
#include "UObject/Object.h"
#include "Widgets/Input/SButton.h"

#define LOCTEXT_NAMESPACE "CEEditorClonerComponentDetailCustomization"

void FCEEditorClonerComponentDetailCustomization::CustomizeDetails(IDetailLayoutBuilder& InDetailBuilder)
{
	FPropertyEditorModule& PropertyModule = FModuleManager::GetModuleChecked<FPropertyEditorModule>(TEXT("PropertyEditor"));
	const FName ComponentClassName = UCEClonerComponent::StaticClass()->GetFName();

	// Remove exposed user parameters
	InDetailBuilder.HideCategory(TEXT("NiagaraComponent_Parameters"));

	// Remove niagara utilities
	InDetailBuilder.HideCategory(TEXT("NiagaraComponent_Utilities"));
	InDetailBuilder.HideCategory(TEXT("Niagara Utilities"));

	// Remove details sections by hiding specific categories
	InDetailBuilder.HideCategory(TEXT("LOD"));	
	InDetailBuilder.HideCategory(TEXT("HLOD"));	
	InDetailBuilder.HideCategory(TEXT("Navigation"));	
	InDetailBuilder.HideCategory(TEXT("AssetUserData"));	
	InDetailBuilder.HideCategory(TEXT("Cooking"));	
	InDetailBuilder.HideCategory(TEXT("Tags"));

	TSharedRef<IPropertyUtilities> PropertyUtilities = InDetailBuilder.GetPropertyUtilities();
	PropertyUtilitiesWeak = PropertyUtilities.ToWeakPtr();
	ClonerComponentsWeak = InDetailBuilder.GetObjectsOfTypeBeingCustomized<UCEClonerComponent>();

	// Place LayoutName property above all properties in the category
	{
		TSharedRef<IPropertyHandle> LayoutHandle = InDetailBuilder.GetProperty(UCEClonerComponent::GetLayoutNamePropertyName(), UCEClonerComponent::StaticClass());
		LayoutHandle->SetOnPropertyValueChangedWithData(TDelegate<void(const FPropertyChangedEvent&)>::CreateStatic(&FCEEditorClonerComponentDetailCustomization::OnPropertyChanged, PropertyUtilitiesWeak));

		IDetailCategoryBuilder& LayoutCategoryBuilder = InDetailBuilder.EditCategory(TEXT("Layout"), FText::FromName(TEXT("Layout")));
		LayoutCategoryBuilder.AddProperty(LayoutHandle);
	}

	{
		IDetailCategoryBuilder& GeneralCategory = InDetailBuilder.EditCategory(TEXT("General"), LOCTEXT("GeneralCategory", "General"), ECategoryPriority::Important);
		IDetailCategoryBuilder& TransformCategory = InDetailBuilder.EditCategory(TEXT("Transform"), LOCTEXT("TransformCategory", "Transform"), ECategoryPriority::Important);
		TransformCategory.SetSortOrder(GeneralCategory.GetSortOrder() + 1);

		const TSharedRef<FPropertySection> GeneralSection = PropertyModule.FindOrCreateSection(ComponentClassName, TEXT("General"), FText::FromName(TEXT("General")));
		GeneralSection->AddCategory(TEXT("General"));
	}

	// Everything needs to be below Cloner category
	const IDetailCategoryBuilder& ClonerCategoryBuilder = InDetailBuilder.EditCategory(TEXT("Cloner"), FText::FromName(TEXT("Cloner")));
	const int32 ExtensionSortOrderOffset = 1;
	const int32 StartSortOrder = ClonerCategoryBuilder.GetSortOrder() + 1;

	// Group same class objects together so their properties are grouped in the details panel when multiple cloners are selected
	struct FDetailsCategoryData
	{
		FName SectionName;
		int32 SortOrder;
		TArray<UObject*> Objects;
	};

	TMap<FName, FDetailsCategoryData> CategoryToData;
	for (const TWeakObjectPtr<UCEClonerComponent>& ClonerComponentWeak : ClonerComponentsWeak)
	{
		const UCEClonerComponent* ClonerComponent = ClonerComponentWeak.Get();

		if (!ClonerComponent)
		{
			continue;
		}

		if (UCEClonerLayoutBase* ActiveLayout = ClonerComponent->GetActiveLayout())
		{
			FDetailsCategoryData& CategoryData = CategoryToData.FindOrAdd(TEXT("Layout"));
			CategoryData.SectionName = TEXT("Cloner");
			CategoryData.SortOrder = StartSortOrder;
			CategoryData.Objects.Add(ActiveLayout);
		}

		for (UCEClonerExtensionBase* ActiveExtension : ClonerComponent->GetActiveExtensions())
		{
			FDetailsCategoryData& CategoryData = CategoryToData.FindOrAdd(ActiveExtension->GetExtensionName());
			CategoryData.SectionName = ActiveExtension->GetExtensionSection().SectionName;
			CategoryData.SortOrder = StartSortOrder + ExtensionSortOrderOffset + ActiveExtension->GetExtensionSection().SectionOrder;
			CategoryData.Objects.Add(ActiveExtension);
		}
	}

	FAddPropertyParams AddParams;
	AddParams.CreateCategoryNodes(false);
	AddParams.HideRootObjectNode(true);

	const TSharedPtr<FCEEditorInputPreprocessor> InputPreprocessor = FCEEditorModule::GetInputPreprocessor();

	if (InputPreprocessor.IsValid())
	{
		InputPreprocessor->Unregister();
	}

	for (const TPair<FName,FDetailsCategoryData>& CategoryToObjectsPair : CategoryToData)
	{
		const FName CategoryName = CategoryToObjectsPair.Key;

		if (CategoryName.IsNone() || CategoryToObjectsPair.Value.Objects.IsEmpty())
		{
			continue;
		}

		IDetailCategoryBuilder& CategoryBuilder = InDetailBuilder.EditCategory(CategoryName, FText::FromName(CategoryName));
		CategoryBuilder.SetSortOrder(CategoryToObjectsPair.Value.SortOrder);

		const TSharedRef<FPropertySection> PropertySection = PropertyModule.FindOrCreateSection(ComponentClassName, CategoryToObjectsPair.Value.SectionName, FText::FromName(CategoryToObjectsPair.Value.SectionName));
		PropertySection->AddCategory(CategoryName);

		if (IDetailPropertyRow* ObjectRow = CategoryBuilder.AddExternalObjects(CategoryToObjectsPair.Value.Objects, EPropertyLocation::Default, AddParams))
		{
			TSharedPtr<IPropertyHandle> ObjectPropertyHandle = ObjectRow->GetPropertyHandle();

			// Fix for EditConditionHides not appearing when condition is met due to AddExternalObjects not rebuilding children
			ObjectPropertyHandle->SetOnChildPropertyValueChangedWithData(TDelegate<void(const FPropertyChangedEvent&)>::CreateStatic(&FCEEditorClonerComponentDetailCustomization::OnChildPropertyChanged, ObjectPropertyHandle.ToWeakPtr()));

			// Fix to refocus last widget when TAB is used since RequestRebuildChildren will lose focus
			if (InputPreprocessor.IsValid())
			{
				InputPreprocessor->Register();
			}
		}
	}

	if (!InDetailBuilder.GetSelectedObjectsOfType<UCEClonerComponent>().IsEmpty())
	{
		return;
	}

	// Handle ufunctions
	TMap<FName, FName> FunctionToCategory;
	for (const TWeakObjectPtr<UCEClonerComponent>& ClonerComponentWeak : ClonerComponentsWeak)
	{
		UCEClonerComponent* ClonerComponent = ClonerComponentWeak.Get();

		if (!ClonerComponent)
		{
			continue;
		}

		// Look for ufunction in component
		for (UFunction* Function : TFieldRange<UFunction>(ClonerComponent->GetClass(), EFieldIteratorFlags::ExcludeSuper))
		{
			// Only CallInEditor function with 0 parameters
			if (Function && Function->HasMetaData("CallInEditor") && Function->NumParms == 0)
			{
				FName FunctionName = Function->GetFName();
				TMap<TWeakObjectPtr<UObject>, TWeakObjectPtr<UFunction>>& ObjectFunctions = LayoutFunctionNames.FindOrAdd(FunctionName);
				ObjectFunctions.Add(ClonerComponent, Function);
				FunctionToCategory.Add(FunctionName, FName(Function->GetMetaData(TEXT("Category"))));
			}
		}
	}

	// Add cloner create track button
	FunctionToCategory.Add(TrackEditor, TEXT("Utilities"));

	if (!FunctionToCategory.IsEmpty())
	{
		TMap<FName, const TSharedPtr<SVerticalBox>> FunctionToWidget;

		// Add buttons for ufunctions based on their category
		for (const TPair<FName, FName>& FunctionToCategoryPair : FunctionToCategory)
		{
			const FName FunctionsCategoryName = FunctionToCategoryPair.Value;

			IDetailCategoryBuilder& FunctionsCategory = InDetailBuilder.EditCategory(FunctionsCategoryName, FText::FromName(FunctionsCategoryName), ECategoryPriority::Uncommon);

			// Subcategories are not supported in sections
			if (!FunctionsCategoryName.ToString().Contains(TEXT("|")))
			{
				const TSharedRef<FPropertySection> FunctionsSection = PropertyModule.FindOrCreateSection(ComponentClassName, FunctionsCategoryName, FText::FromName(FunctionsCategoryName));
				FunctionsSection->AddCategory(FunctionsCategoryName);
			}

			const TSharedPtr<SVerticalBox> FunctionsWidget = SNew(SVerticalBox);

			FunctionsCategory.AddCustomRow(FText::GetEmpty())
				.WholeRowContent()
				.HAlign(HAlign_Left)
				[
					FunctionsWidget.ToSharedRef()
				];

			FunctionToWidget.Add(FunctionToCategoryPair.Key, FunctionsWidget);
		}

		for (const TPair<FName, const TSharedPtr<SVerticalBox>>& FunctionNameToWidget : FunctionToWidget)
		{
			const FText ButtonLabel = FText::FromString(FName::NameToDisplayString(FunctionNameToWidget.Key.ToString(), /** IsBool */false));

			const TSharedPtr<SVerticalBox>& FunctionsWidget = FunctionNameToWidget.Value;

			FunctionsWidget->AddSlot()
				.Padding(0.f, 3.f)
				.AutoHeight()
				[
					SNew(SButton)
					.Text(ButtonLabel)
					.HAlign(HAlign_Center)
					.VAlign(VAlign_Fill)
					.IsEnabled(this, &FCEEditorClonerComponentDetailCustomization::IsFunctionButtonEnabled, FunctionNameToWidget.Key)
					.OnClicked(this, &FCEEditorClonerComponentDetailCustomization::OnFunctionButtonClicked, FunctionNameToWidget.Key)
				];
		}
	}

	// Show properties from parent class (UNiagaraComponent and above) in a separate category
	{
		IDetailCategoryBuilder& CategoryBuilder = InDetailBuilder.EditCategory(TEXT("NiagaraComponent"));
		CategoryBuilder.SetSortOrder(10000);
		TMap<FName, IDetailGroup*> NiagaraComponentGroups;
		for (FProperty* Property : TFieldRange<FProperty>(UNiagaraComponent::StaticClass(), EFieldIterationFlags::IncludeSuper))
		{
			if (Property && Property->HasAnyPropertyFlags(CPF_Edit | CPF_EditConst))
			{
				TSharedRef<IPropertyHandle> NiagaraComponentPropertyHandle = InDetailBuilder.GetProperty(Property->GetFName(), Property->GetOwnerClass());
				if (NiagaraComponentPropertyHandle->IsValidHandle())
				{
					const FName CategoryName = NiagaraComponentPropertyHandle->GetDefaultCategoryName();

					IDetailGroup* DetailGroup = nullptr;
					if (NiagaraComponentGroups.Contains(CategoryName))
					{
						DetailGroup = NiagaraComponentGroups.FindChecked(CategoryName);
					}
					else
					{
						DetailGroup = &CategoryBuilder.AddGroup(CategoryName, NiagaraComponentPropertyHandle->GetDefaultCategoryText());
						NiagaraComponentGroups.Add(CategoryName, DetailGroup);
					}

					DetailGroup->AddPropertyRow(NiagaraComponentPropertyHandle);
				}
			}
		}
	}
}

void FCEEditorClonerComponentDetailCustomization::RemoveEmptySections()
{
	FPropertyEditorModule& PropertyModule = FModuleManager::GetModuleChecked<FPropertyEditorModule>(TEXT("PropertyEditor"));
	const FName ComponentClassName = UCEClonerComponent::StaticClass()->GetFName();

	// Remove sections
	PropertyModule.RemoveSection(ComponentClassName, TEXT("Rendering"));
	PropertyModule.RemoveSection(ComponentClassName, TEXT("Effects"));
	PropertyModule.RemoveSection(ComponentClassName, TEXT("Streaming"));
}

void FCEEditorClonerComponentDetailCustomization::OnPropertyChanged(const FPropertyChangedEvent& InEvent, TWeakPtr<IPropertyUtilities> InUtilitiesWeak)
{
	if (const TSharedPtr<IPropertyUtilities> PropertyUtilities = InUtilitiesWeak.Pin())
	{
		if (InEvent.ChangeType != EPropertyChangeType::Interactive)
		{
			PropertyUtilities->RequestForceRefresh();
		}
	}
}

void FCEEditorClonerComponentDetailCustomization::Init()
{
	// When using a layout for the first time, it is not yet loaded and the property change will trigger an async load
	UCEClonerComponent::OnClonerLayoutLoaded().RemoveAll(this);
	UCEClonerComponent::OnClonerLayoutLoaded().AddSP(this, &FCEEditorClonerComponentDetailCustomization::OnClonerLayoutLoaded);
}

void FCEEditorClonerComponentDetailCustomization::OnChildPropertyChanged(const FPropertyChangedEvent& InEvent, TWeakPtr<IPropertyHandle> InParentHandleWeak)
{
	const TSharedPtr<IPropertyHandle> PropertyHandle = InParentHandleWeak.Pin();

	if (!PropertyHandle)
	{
		return;
	}

	if (InEvent.ChangeType == EPropertyChangeType::Interactive || !InEvent.MemberProperty)
	{
		return;
	}

	if (!InEvent.MemberProperty->HasMetaData(TEXT("RefreshPropertyView")))
	{
		return;
	}

	const TSharedPtr<FCEEditorInputPreprocessor> InputPreprocessor = FCEEditorModule::GetInputPreprocessor();

	if (InputPreprocessor.IsValid())
	{
		// Refocus last focused widget next tick because rebuilding children loses focus
		InputPreprocessor->RefocusLastWidget();
		InputPreprocessor->Unregister();
	}

	PropertyHandle->RequestRebuildChildren();
}

void FCEEditorClonerComponentDetailCustomization::OnClonerLayoutLoaded(UCEClonerComponent* InCloner, UCEClonerLayoutBase* InLayout)
{
	const TSharedPtr<IPropertyUtilities> PropertyUtilities = PropertyUtilitiesWeak.Pin();

	if (PropertyUtilities && InCloner && ClonerComponentsWeak.Contains(InCloner))
	{
		PropertyUtilities->RequestForceRefresh();
	}
}

bool FCEEditorClonerComponentDetailCustomization::IsFunctionButtonEnabled(FName InFunctionName) const
{
	if (InFunctionName.IsEqual(TrackEditor))
	{
		return CanAddSequencerTracks();
	}

	return true;
}

FReply FCEEditorClonerComponentDetailCustomization::OnFunctionButtonClicked(FName InFunctionName)
{
	if (InFunctionName.IsEqual(TrackEditor))
	{
		OnAddSequencerTracks();
	}
	else if (TMap<TWeakObjectPtr<UObject>, TWeakObjectPtr<UFunction>> const* ObjectFunctions = LayoutFunctionNames.Find(InFunctionName))
	{
		for (const TPair<TWeakObjectPtr<UObject>, TWeakObjectPtr<UFunction>>& ObjectFunction : *ObjectFunctions)
		{
			UObject* Object = ObjectFunction.Key.Get();
			UFunction* Function = ObjectFunction.Value.Get();

			if (!Object || !Function)
			{
				continue;
			}

			Object->ProcessEvent(Function, nullptr);
		}
	}

	return FReply::Handled();
}

bool FCEEditorClonerComponentDetailCustomization::CanAddSequencerTracks() const
{
	// Lifecycle + cache tracks
	constexpr uint32 ExpectedTrackCount = 2;

	for (const TWeakObjectPtr<UCEClonerComponent>& ClonerComponentWeak : ClonerComponentsWeak)
	{
		if (UCEClonerComponent* ClonerComponent = ClonerComponentWeak.Get())
		{
			uint32 TrackCount = 0;
			FMovieSceneClonerTrackEditor::OnClonerTrackExists.Broadcast(ClonerComponent, TrackCount);

			if (TrackCount < ExpectedTrackCount)
			{
				return true;
			}
		}
	}

	return false;
}

FReply FCEEditorClonerComponentDetailCustomization::OnAddSequencerTracks()
{
	TSet<UCEClonerComponent*> ClonersWeak;

	Algo::TransformIf(
		ClonerComponentsWeak
		, ClonersWeak
		, [](const TWeakObjectPtr<UCEClonerComponent>& InClonerWeak)
		{
			return InClonerWeak.IsValid();
		}
		, [](const TWeakObjectPtr<UCEClonerComponent>& InClonerWeak)
		{
			return InClonerWeak.Get();
		}
	);

	FMovieSceneClonerTrackEditor::OnAddClonerTrack.Broadcast(ClonersWeak);

	return FReply::Handled();
}

#undef LOCTEXT_NAMESPACE
