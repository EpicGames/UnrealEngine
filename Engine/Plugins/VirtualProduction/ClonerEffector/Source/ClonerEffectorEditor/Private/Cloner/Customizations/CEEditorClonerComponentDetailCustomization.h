// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Map.h"
#include "IDetailCustomization.h"
#include "UObject/NameTypes.h"
#include "UObject/WeakObjectPtr.h"
#include "Templates/SharedPointerFwd.h"

class FReply;
class IPropertyHandle;
class IPropertyUtilities;
class UCEClonerComponent;
class UCEClonerLayoutBase;
class UFunction;
class UObject;

/** Used to customize cloner component properties in details panel */
class FCEEditorClonerComponentDetailCustomization : public IDetailCustomization
{
public:
	static TSharedRef<IDetailCustomization> MakeInstance()
	{
		TSharedRef<FCEEditorClonerComponentDetailCustomization> Customization = MakeShared<FCEEditorClonerComponentDetailCustomization>();
		Customization->Init();
		return Customization;
	}

	explicit FCEEditorClonerComponentDetailCustomization()
	{
		RemoveEmptySections();
	}

	//~ Begin IDetailCustomization
	virtual void CustomizeDetails(IDetailLayoutBuilder& InDetailBuilder) override;
	//~ End IDetailCustomization

protected:
	static constexpr const TCHAR* TrackEditor = TEXT("CreateClonerSequencerTracks");
	static void RemoveEmptySections();
	static void OnChildPropertyChanged(const FPropertyChangedEvent& InEvent, TWeakPtr<IPropertyHandle> InParentHandleWeak);
	static void OnPropertyChanged(const FPropertyChangedEvent& InEvent, TWeakPtr<IPropertyUtilities> InUtilitiesWeak);

	bool CanAddSequencerTracks() const;
	FReply OnAddSequencerTracks();

	/** Bind delegates needed */
	void Init();

	/** Used to refresh details view when layout changes */
	void OnClonerLayoutLoaded(UCEClonerComponent* InCloner, UCEClonerLayoutBase* InLayout);

	bool IsFunctionButtonEnabled(FName InFunctionName) const;

	/** Execute ufunction with that name on selected objects */
	FReply OnFunctionButtonClicked(FName InFunctionName);

	/** Function name to object ufunction mapping */
	TMap<FName, TMap<TWeakObjectPtr<UObject>, TWeakObjectPtr<UFunction>>> LayoutFunctionNames;

	/** Customized objects */
	TArray<TWeakObjectPtr<UCEClonerComponent>> ClonerComponentsWeak;

	/** Property utilities for details view refresh */
	TWeakPtr<IPropertyUtilities> PropertyUtilitiesWeak;
};