// Copyright Epic Games, Inc. All Rights Reserved.

#include "Cloner/Customizations/CEEditorClonerCustomActorPickerNodeBuilder.h"

#include "DetailWidgetRow.h"
#include "Editor.h"
#include "GameFramework/Actor.h"
#include "PropertyCustomizationHelpers.h"
#include "ScopedTransaction.h"
#include "Selection.h"
#include "Widgets/Input/SComboButton.h"
#include "Widgets/Text/STextBlock.h"

#define LOCTEXT_NAMESPACE "CEEditorClonerCustomActorPickerNodeBuilder"

FName FCEEditorClonerCustomActorPickerNodeBuilder::GetName() const
{
	return TEXT("FCEEditorClonerCustomActorPickerNodeBuilder");
}

void FCEEditorClonerCustomActorPickerNodeBuilder::OnSelectActor() const
{
	if (!GEditor)
	{
		return;
	}

	AActor* Actor = GetPropertyActor();

	if (IsValid(Actor))
	{
		const FScopedTransaction Transaction(NSLOCTEXT("UnrealEd", "ClickingOnElements", "Clicking on Elements"));

		constexpr bool bNotifySelectionChanged = true;
		GEditor->SelectNone(!bNotifySelectionChanged, /** DeselectBSP */true);
		GEditor->SelectActor(Actor, /** IsSelected */true, bNotifySelectionChanged);
	}
}

void FCEEditorClonerCustomActorPickerNodeBuilder::OnUseSelectedActor() const
{
	if (AActor* SelectedActor = GetSelectedActor())
	{
		if (ActorFilterDelegate.Execute(SelectedActor))
		{
			OnActorSelected(SelectedActor);
		}
	}
}

AActor* FCEEditorClonerCustomActorPickerNodeBuilder::GetPropertyActor() const
{
	if (!PropertyHandle->IsValidHandle())
	{
		return nullptr;
	}

	UObject* SelectedObject;
	if (PropertyHandle->GetValue(SelectedObject) != FPropertyAccess::Success)
	{
		return nullptr;
	}

	return Cast<AActor>(SelectedObject);
}

AActor* FCEEditorClonerCustomActorPickerNodeBuilder::GetSelectedActor() const
{
	if (!GEditor || !GEditor->GetSelectedActors())
	{
		return nullptr;
	}

	return GEditor->GetSelectedActors()->GetTop<AActor>();
}

void FCEEditorClonerCustomActorPickerNodeBuilder::GenerateHeaderRowContent(FDetailWidgetRow& InNodeRow)
{
	if (!PropertyHandle->IsValidHandle())
	{
		return;
	}

	bool bIsArrayProperty = false;
	if (TSharedPtr<IPropertyHandle> ParentHandle = PropertyHandle->GetParentHandle())
	{
		if (ParentHandle->AsArray())
		{
			bIsArrayProperty = true;
		}
	}

	TSharedRef<SHorizontalBox> CustomValueWidget = SNew(SHorizontalBox)
		+ SHorizontalBox::Slot()
		.HAlign(HAlign_Fill)
		.FillWidth(VAlign_Fill)
		.FillWidth(1.f)
		[
			SAssignNew(ComboButton, SComboButton)
			.ContentPadding(2.0f)
			.HasDownArrow(true)
			.OnGetMenuContent_Raw(this, &FCEEditorClonerCustomActorPickerNodeBuilder::GetActorPickerWidget)
			.ButtonContent()
			[
				SNew(STextBlock)
				.Text_Raw(this, &FCEEditorClonerCustomActorPickerNodeBuilder::GetPickerLabelText)
			]
		];

	/* Add use selected actor button */
	CustomValueWidget->AddSlot()
		.HAlign(HAlign_Fill)
		.FillWidth(VAlign_Fill)
		.AutoWidth()
		[
			PropertyCustomizationHelpers::MakeUseSelectedButton(
				FSimpleDelegate::CreateSP(this, &FCEEditorClonerCustomActorPickerNodeBuilder::OnUseSelectedActor)
				, TAttribute<FText>(this, &FCEEditorClonerCustomActorPickerNodeBuilder::GetUseSelectTooltipText)
				, /** IsEnabled */true
				, /** IsActor */true)
		];

	/* Add browse to actor button */
	CustomValueWidget->AddSlot()
		.HAlign(HAlign_Fill)
		.FillWidth(VAlign_Fill)
		.AutoWidth()
		[
			PropertyCustomizationHelpers::MakeBrowseButton(
				FSimpleDelegate::CreateSP(this, &FCEEditorClonerCustomActorPickerNodeBuilder::OnSelectActor)
				, TAttribute<FText>(this, &FCEEditorClonerCustomActorPickerNodeBuilder::GetSelectTooltipText)
				, /** IsEnabled */true
				, /** IsActor */true)
		];

	/* Add pick level actor button */

	FOnGetAllowedClasses OnGetAllowedClassesDelegate;

	CustomValueWidget->AddSlot()
		.HAlign(HAlign_Fill)
		.FillWidth(VAlign_Fill)
		.AutoWidth()
		[
			PropertyCustomizationHelpers::MakeInteractiveActorPicker(
				OnGetAllowedClassesDelegate
				, ActorFilterDelegate
				, FOnActorSelected::CreateSP(this, &FCEEditorClonerCustomActorPickerNodeBuilder::OnActorSelected)
			)
		];

	if (bIsArrayProperty)
	{
		TSharedRef<SWidget> DefaultValueWidget = PropertyHandle->CreatePropertyValueWidget();

		// Hide original selector widget
		const TSharedPtr<SWidget> SelectorWidget = FindWidgetType(DefaultValueWidget, TEXT("SPropertyEditorAsset"), /** reverse search */false);
		check(SelectorWidget)
		SelectorWidget->SetVisibility(EVisibility::Collapsed);

		// But keep original array widget visible (insert/duplicate/remove)
		const TSharedPtr<SWidget> ArrayButtons = FindWidgetType(DefaultValueWidget, TEXT("SComboButton"), /** reverse search */true);
		check(ArrayButtons)
		ArrayButtons->SetVisibility(EVisibility::Visible);

		CustomValueWidget->AddSlot()
			.HAlign(HAlign_Fill)
			.FillWidth(VAlign_Fill)
			.AutoWidth()
			[
				DefaultValueWidget
			];
	}

	InNodeRow.NameContent()
	[
		PropertyHandle->CreatePropertyNameWidget()
	];

	InNodeRow.ValueContent()
	[
		CustomValueWidget
	];
}

void FCEEditorClonerCustomActorPickerNodeBuilder::GenerateChildContent(IDetailChildrenBuilder& InChildrenBuilder)
{
}

TSharedPtr<IPropertyHandle> FCEEditorClonerCustomActorPickerNodeBuilder::GetPropertyHandle() const
{
	return PropertyHandle;
}

FText FCEEditorClonerCustomActorPickerNodeBuilder::GetPickerLabelText() const
{
	const AActor* Actor = GetPropertyActor();

	if (IsValid(Actor))
	{
		return FText::FromString(Actor->GetActorNameOrLabel());
	}

	return FText::GetEmpty();
}

FText FCEEditorClonerCustomActorPickerNodeBuilder::GetSelectTooltipText() const
{
	const AActor* Actor = GetPropertyActor();

	if (IsValid(Actor))
	{
		return FText::Format(LOCTEXT("SelectActor", "Select '{0}' in the viewport"), FText::FromString(Actor->GetName()));
	}

	return FText::GetEmpty();
}

FText FCEEditorClonerCustomActorPickerNodeBuilder::GetUseSelectTooltipText() const
{
	const AActor* Actor = GetSelectedActor();

	if (IsValid(Actor))
	{
		return FText::Format(LOCTEXT("UseSelectedActor", "Use '{0}' from the viewport"), FText::FromString(Actor->GetName()));
	}

	return FText::GetEmpty();
}

TSharedRef<SWidget> FCEEditorClonerCustomActorPickerNodeBuilder::GetActorPickerWidget() const
{
	FSimpleDelegate EmptyDelegate;

	return PropertyCustomizationHelpers::MakeActorPickerWithMenu(
		GetPropertyActor()
		, /** AllowClear */true
		, ActorFilterDelegate
		, FOnActorSelected::CreateSP(this, &FCEEditorClonerCustomActorPickerNodeBuilder::OnActorSelected)
		, EmptyDelegate
		, EmptyDelegate);
}

void FCEEditorClonerCustomActorPickerNodeBuilder::OnActorSelected(AActor* InSelection) const
{
	if (PropertyHandle->IsValidHandle())
	{
		PropertyHandle->SetValue(InSelection);
	}

	if (ComboButton)
	{
		ComboButton->SetIsOpen(false);
	}
}

TSharedPtr<SWidget> FCEEditorClonerCustomActorPickerNodeBuilder::FindWidgetType(TSharedPtr<SWidget> InSearchWidget, FName InTypeName, bool bInReverseSearch)
{
	if (!InSearchWidget.IsValid())
	{
		return nullptr;
	}

	const FName WidgetTypeName = InSearchWidget->GetType();

	if (WidgetTypeName == InTypeName)
	{
		return InSearchWidget;
	}

	if (FChildren* Children = InSearchWidget->GetAllChildren())
	{
		if (bInReverseSearch)
		{
			for (int32 Index = Children->Num() - 1; Index >= 0; Index--)
			{
				if (TSharedPtr<SWidget> FoundWidget = FindWidgetType(Children->GetChildAt(Index), InTypeName, bInReverseSearch))
				{
					return FoundWidget;
				}
			}
		}
		else
		{
			for (int32 Index = 0; Index < Children->Num(); Index++)
			{
				if (TSharedPtr<SWidget> FoundWidget = FindWidgetType(Children->GetChildAt(Index), InTypeName, bInReverseSearch))
				{
					return FoundWidget;
				}
			}
		}
	}

	return nullptr;
}

#undef LOCTEXT_NAMESPACE