// Copyright Epic Games, Inc. All Rights Reserved.

#include "Cloner/Customizations/CEEditorClonerEffectorExtensionDetailCustomization.h"

#include "Cloner/Customizations/CEEditorClonerCustomActorPickerNodeBuilder.h"
#include "Cloner/Extensions/CEClonerEffectorExtension.h"
#include "DetailLayoutBuilder.h"
#include "Effector/CEEffectorComponent.h"
#include "GameFramework/Actor.h"
#include "IDetailChildrenBuilder.h"
#include "PropertyCustomizationHelpers.h"
#include "PropertyHandle.h"

void FCEEditorClonerEffectorExtensionDetailCustomization::CustomizeDetails(IDetailLayoutBuilder& InDetailBuilder)
{
	TSharedRef<IPropertyHandle> EffectorsPropertyHandle = InDetailBuilder.GetProperty(UCEClonerEffectorExtension::GetEffectorActorsWeakName(), UCEClonerEffectorExtension::StaticClass());

	if (!EffectorsPropertyHandle->IsValidHandle())
	{
		return;
	}

	EffectorsPropertyHandle->MarkHiddenByCustomization();

	IDetailCategoryBuilder& EffectorsCategoryBuilder = InDetailBuilder.EditCategory(EffectorsPropertyHandle->GetDefaultCategoryName(), EffectorsPropertyHandle->GetDefaultCategoryText());

	CustomizeEffectorsProperty(EffectorsPropertyHandle, EffectorsCategoryBuilder);
}

void FCEEditorClonerEffectorExtensionDetailCustomization::CustomizeEffectorsProperty(TSharedRef<IPropertyHandle> InProperty, IDetailCategoryBuilder& InCategoryBuilder)
{
	const TSharedRef<FDetailArrayBuilder> EffectorsArrayBuilder = MakeShared<FDetailArrayBuilder>(InProperty, /*GenerateHeader*/true, /*DisplayResetToDefault*/true, /*bDisplayElementNum*/true);

	auto FilterActorDelegate = FOnShouldFilterActor::CreateStatic(&FCEEditorClonerEffectorExtensionDetailCustomization::OnFilterEffectorActor);

	EffectorsArrayBuilder->OnGenerateArrayElementWidget(FOnGenerateArrayElementWidget::CreateLambda(
		[FilterActorDelegate](TSharedRef<IPropertyHandle> InAttributeHandle, int32 InArrayIndex, IDetailChildrenBuilder& InChildrenBuilder)
		{
			InChildrenBuilder.AddCustomBuilder(MakeShared<FCEEditorClonerCustomActorPickerNodeBuilder>(InAttributeHandle, FilterActorDelegate));
		}));

	InCategoryBuilder.AddCustomBuilder(EffectorsArrayBuilder);
}

bool FCEEditorClonerEffectorExtensionDetailCustomization::OnFilterEffectorActor(const AActor* InActor)
{
	if (!IsValid(InActor))
	{
		return false;
	}

	return !!InActor->FindComponentByClass<UCEEffectorComponent>();
}
