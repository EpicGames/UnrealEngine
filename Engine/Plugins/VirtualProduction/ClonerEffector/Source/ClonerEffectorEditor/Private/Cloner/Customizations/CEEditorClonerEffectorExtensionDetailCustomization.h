// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IDetailCustomization.h"
#include "Templates/SharedPointer.h"

class AActor;
class IDetailCategoryBuilder;
class IPropertyHandle;

/** Used to customize cloner effector extension properties in details panel */
class FCEEditorClonerEffectorExtensionDetailCustomization : public IDetailCustomization
{
public:
	static TSharedRef<IDetailCustomization> MakeInstance()
	{
		return MakeShared<FCEEditorClonerEffectorExtensionDetailCustomization>();
	}

	//~ Begin IDetailCustomization
	virtual void CustomizeDetails(IDetailLayoutBuilder& InDetailBuilder) override;
	//~ End IDetailCustomization

protected:
	static void CustomizeEffectorsProperty(TSharedRef<IPropertyHandle> InProperty, IDetailCategoryBuilder& InCategoryBuilder);
	static bool OnFilterEffectorActor(const AActor* InActor);
};
