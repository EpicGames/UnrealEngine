// Copyright Epic Games, Inc. All Rights Reserved.

#include "Cloner/Customizations/CEEditorClonerLifetimeExtensionDetailCustomization.h"

#include "Cloner/Extensions/CEClonerLifetimeExtension.h"
#include "DetailBuilderTypes.h"
#include "DetailCategoryBuilder.h"
#include "DetailLayoutBuilder.h"
#include "DetailWidgetRow.h"
#include "GameFramework/Actor.h"
#include "NiagaraDataInterfaceCurve.h"
#include "PropertyHandle.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/SBoxPanel.h"

#define LOCTEXT_NAMESPACE "CEEditorClonerLifetimeExtensionDetailCustomization"

void FCEEditorClonerLifetimeExtensionDetailCustomization::CustomizeDetails(IDetailLayoutBuilder& InDetailBuilder)
{
	const TArray<TWeakObjectPtr<UCEClonerLifetimeExtension>> LifetimeExtensionsWeak = InDetailBuilder.GetObjectsOfTypeBeingCustomized<UCEClonerLifetimeExtension>();

	FAddPropertyParams Params;
	Params.HideRootObjectNode(true);
	Params.CreateCategoryNodes(false);

	for (const TWeakObjectPtr<UCEClonerLifetimeExtension>& LifetimeExtensionWeak : LifetimeExtensionsWeak)
	{
		const UCEClonerLifetimeExtension* LifetimeExtension = LifetimeExtensionWeak.Get();

		if (!LifetimeExtension)
		{
			continue;
		}

		UNiagaraDataInterfaceCurve* CurveDI = LifetimeExtension->GetLifetimeScaleCurveDI();

		if (!CurveDI)
		{
			continue;
		}

		const FName CategoryName = LifetimeExtension->GetExtensionName();
		IDetailCategoryBuilder& CurveCategoryBuilder = InDetailBuilder.EditCategory(FName(CategoryName.ToString() + TEXT("Curve")), FText::GetEmpty(), ECategoryPriority::Uncommon);

		// Hide other properties, only curve will be shown instead of tree
		for (FProperty* Property : TFieldRange<FProperty>(CurveDI->GetClass()))
		{
			if (Property)
			{
				Property->SetMetaData(TEXT("EditCondition"), TEXT("false"));
				Property->SetMetaData(TEXT("EditConditionHides"), TEXT("true"));
			}
		}

		// UNiagaraDataInterfaceCurve cannot display simultaneously multiple curves, so we need to add them separately
		if (IDetailPropertyRow* Row = CurveCategoryBuilder.AddExternalObjects({CurveDI}, EPropertyLocation::Common, Params))
		{
			const TAttribute<EVisibility> VisibilityAttr = TAttribute<EVisibility>::CreateSP(this, &FCEEditorClonerLifetimeExtensionDetailCustomization::GetCurveVisibility, LifetimeExtensionWeak);
			Row->Visibility(VisibilityAttr);
		}
	}
}

EVisibility FCEEditorClonerLifetimeExtensionDetailCustomization::GetCurveVisibility(TWeakObjectPtr<UCEClonerLifetimeExtension> InExtensionWeak) const
{
	const UCEClonerLifetimeExtension* Extension = InExtensionWeak.Get();

	if (!Extension)
	{
		return EVisibility::Collapsed;
	}

	return Extension->GetLifetimeEnabled()
		&& Extension->GetLifetimeScaleEnabled()
		? EVisibility::Visible
		: EVisibility::Collapsed;
}

#undef LOCTEXT_NAMESPACE
