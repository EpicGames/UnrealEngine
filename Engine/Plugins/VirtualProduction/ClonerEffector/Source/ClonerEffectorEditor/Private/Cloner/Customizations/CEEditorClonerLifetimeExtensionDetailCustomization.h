// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IDetailCustomization.h"
#include "Templates/SharedPointer.h"
#include "UObject/WeakObjectPtrTemplatesFwd.h"

class FReply;
class IDetailCategoryBuilder;
class IPropertyHandle;
class UCEClonerLifetimeExtension;
struct EVisibility;

/** Used to customize cloner lifetime extension properties in details panel */
class FCEEditorClonerLifetimeExtensionDetailCustomization : public IDetailCustomization
{
public:
	static TSharedRef<IDetailCustomization> MakeInstance()
	{
		return MakeShared<FCEEditorClonerLifetimeExtensionDetailCustomization>();
	}

	//~ Begin IDetailCustomization
	virtual void CustomizeDetails(IDetailLayoutBuilder& InDetailBuilder) override;
	//~ End IDetailCustomization

protected:
	EVisibility GetCurveVisibility(TWeakObjectPtr<UCEClonerLifetimeExtension> InExtensionWeak) const;

	TSharedPtr<IPropertyHandle> LifetimeEnabledPropertyHandle;
	TSharedPtr<IPropertyHandle> LifetimeScaleEnabledPropertyHandle;
};
