// Copyright Epic Games, Inc. All Rights Reserved.

#include "CEEditorClonerMeshLayoutDetailCustomization.h"

#include "Cloner/Customizations/CEEditorClonerCustomActorPickerNodeBuilder.h"
#include "Cloner/Layouts/CEClonerMeshLayout.h"
#include "Components/SkeletalMeshComponent.h"
#include "Components/StaticMeshComponent.h"
#include "DetailCategoryBuilder.h"
#include "DetailLayoutBuilder.h"
#include "GameFramework/Actor.h"
#include "PropertyHandle.h"

void FCEEditorClonerMeshLayoutDetailCustomization::CustomizeDetails(IDetailLayoutBuilder& InDetailBuilder)
{
	AssetPropertyHandle = InDetailBuilder.GetProperty(UCEClonerMeshLayout::GetAssetName(), UCEClonerMeshLayout::StaticClass());

	TSharedRef<IPropertyHandle> MeshPropertyHandle = InDetailBuilder.GetProperty(UCEClonerMeshLayout::GetSampleActorWeakName(), UCEClonerMeshLayout::StaticClass());

	if (!MeshPropertyHandle->IsValidHandle())
	{
		return;
	}

	MeshPropertyHandle->MarkHiddenByCustomization();

	IDetailCategoryBuilder& MeshCategoryBuilder = InDetailBuilder.EditCategory(MeshPropertyHandle->GetDefaultCategoryName(), MeshPropertyHandle->GetDefaultCategoryText());

	MeshCategoryBuilder.AddCustomBuilder(MakeShared<FCEEditorClonerCustomActorPickerNodeBuilder>(
		MeshPropertyHandle
		, FOnShouldFilterActor::CreateSP(this, &FCEEditorClonerMeshLayoutDetailCustomization::OnFilterMeshActor))
	);
}

bool FCEEditorClonerMeshLayoutDetailCustomization::OnFilterMeshActor(const AActor* InActor)
{
	if (!IsValid(InActor))
	{
		return false;
	}

	if (AssetPropertyHandle.IsValid() && AssetPropertyHandle->IsValidHandle())
	{
		uint8 Value = 0;

		if (AssetPropertyHandle->GetValue(Value) == FPropertyAccess::Success)
		{
			if (static_cast<ECEClonerMeshAsset>(Value) == ECEClonerMeshAsset::SkeletalMesh)
			{
				return !!InActor->FindComponentByClass<USkeletalMeshComponent>();
			}

			if (static_cast<ECEClonerMeshAsset>(Value) == ECEClonerMeshAsset::StaticMesh)
			{
				return !!InActor->FindComponentByClass<UStaticMeshComponent>();
			}
		}
	}

	return true;
}
