// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IDetailCustomization.h"
#include "Templates/SharedPointer.h"

class AActor;
class IPropertyHandle;

/** Used to customize cloner mesh layout properties in details panel */
class FCEEditorClonerMeshLayoutDetailCustomization : public IDetailCustomization
{
public:
	static TSharedRef<IDetailCustomization> MakeInstance()
	{
		return MakeShared<FCEEditorClonerMeshLayoutDetailCustomization>();
	}

	//~ Begin IDetailCustomization
	virtual void CustomizeDetails(IDetailLayoutBuilder& InDetailBuilder) override;
	//~ End IDetailCustomization

private:
	bool OnFilterMeshActor(const AActor* InActor);

	TSharedPtr<IPropertyHandle> AssetPropertyHandle;
};
