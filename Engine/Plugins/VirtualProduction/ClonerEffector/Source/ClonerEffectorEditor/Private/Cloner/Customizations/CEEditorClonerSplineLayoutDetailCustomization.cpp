// Copyright Epic Games, Inc. All Rights Reserved.

#include "Cloner/Customizations/CEEditorClonerSplineLayoutDetailCustomization.h"

#include "Cloner/Customizations/CEEditorClonerCustomActorPickerNodeBuilder.h"
#include "Cloner/Layouts/CEClonerSplineLayout.h"
#include "Components/SplineComponent.h"
#include "DetailCategoryBuilder.h"
#include "DetailLayoutBuilder.h"

void FCEEditorClonerSplineLayoutDetailCustomization::CustomizeDetails(IDetailLayoutBuilder& InDetailBuilder)
{
	TSharedRef<IPropertyHandle> SplinePropertyHandle = InDetailBuilder.GetProperty(UCEClonerSplineLayout::GetSplineActorWeakName(), UCEClonerSplineLayout::StaticClass());

	if (!SplinePropertyHandle->IsValidHandle())
	{
		return;
	}

	SplinePropertyHandle->MarkHiddenByCustomization();

	IDetailCategoryBuilder& SplineCategoryBuilder = InDetailBuilder.EditCategory(SplinePropertyHandle->GetDefaultCategoryName(), SplinePropertyHandle->GetDefaultCategoryText());

	SplineCategoryBuilder.AddCustomBuilder(MakeShared<FCEEditorClonerCustomActorPickerNodeBuilder>(
		SplinePropertyHandle
		, FOnShouldFilterActor::CreateStatic(&FCEEditorClonerSplineLayoutDetailCustomization::OnFilterSplineActor))
	);
}

bool FCEEditorClonerSplineLayoutDetailCustomization::OnFilterSplineActor(const AActor* InActor)
{
	if (!IsValid(InActor))
	{
		return false;
	}

	return !!InActor->FindComponentByClass<USplineComponent>();
}
