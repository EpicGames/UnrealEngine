// Copyright Epic Games, Inc. All Rights Reserved.

#include "Cloner/Menus/CEEditorClonerMenu.h"

#include "Cloner/CEClonerActor.h"
#include "Cloner/CEClonerComponent.h"
#include "Effector/CEEffectorActor.h"
#include "Effector/CEEffectorComponent.h"
#include "Engine/World.h"
#include "Styling/SlateIconFinder.h"
#include "Subsystems/CEClonerSubsystem.h"
#include "Utilities/CEClonerLibrary.h"
#include "Utilities/CEEffectorLibrary.h"

#define LOCTEXT_NAMESPACE "CEEditorClonerMenu"

FToolMenuSection* UE::ClonerEditor::Menu::FindOrAddClonerSection(UToolMenu* InMenu)
{
	static const FName ClonerSectionName("ContextClonerActions");

	FToolMenuSection* ClonerSection = InMenu->FindSection(ClonerSectionName);

	if (!ClonerSection)
	{
		ClonerSection = &InMenu->AddSection(ClonerSectionName
			, LOCTEXT("ContexClonerActions", "Cloner Actions")
			, FToolMenuInsert(NAME_None, EToolMenuInsertType::First));
	}

	return ClonerSection;
}

void UE::ClonerEditor::Menu::FillEnableClonerSection(UToolMenu* InMenu, const FCEEditorClonerMenuData& InMenuData)
{
	if (!InMenu
		|| InMenuData.Context.IsEmpty()
		|| !InMenuData.Context.ContainsAnyDisabledCloner())
	{
		return;
	}

	FToolMenuSection& EnableClonerSection = InMenu->FindOrAddSection(
		TEXT("EnableCloner")
		, LOCTEXT("EnableCloner.Label", "Enable cloner")
	);

	constexpr bool bEnable = true;

	EnableClonerSection.AddMenuEntry(
		TEXT("EnableClonerComponent")
		, LOCTEXT("EnableClonerComponent.Label", "Enable cloner")
		, LOCTEXT("EnableClonerComponent.Tooltip", "Enable selected cloners")
		, FSlateIcon()
		, FUIAction(
			FExecuteAction::CreateLambda(&ExecuteEnableClonerAction, InMenuData, bEnable)
		)
	);

	EnableClonerSection.AddMenuEntry(
		TEXT("EnableClonerLevel")
		, LOCTEXT("EnableClonerLevel.Label", "Enable level cloner")
		, LOCTEXT("EnableClonerLevel.Tooltip", "Enable selected level cloners")
		, FSlateIcon()
		, FUIAction(
			FExecuteAction::CreateLambda(&ExecuteEnableLevelClonerAction, InMenuData, bEnable)
		)
	);
}

void UE::ClonerEditor::Menu::FillDisableClonerSection(UToolMenu* InMenu, const FCEEditorClonerMenuData& InMenuData)
{
	if (!InMenu
		|| InMenuData.Context.IsEmpty()
		|| !InMenuData.Context.ContainsAnyEnabledCloner())
	{
		return;
	}

	FToolMenuSection& DisableClonerSection = InMenu->FindOrAddSection(
		TEXT("DisableCloner")
		, LOCTEXT("DisableCloner.Label", "Disable cloner")
	);

	constexpr bool bEnable = false;

	DisableClonerSection.AddMenuEntry(
		TEXT("DisableClonerComponent")
		, LOCTEXT("DisableClonerComponent.Label", "Disable cloner")
		, LOCTEXT("DisableClonerComponent.Tooltip", "Disable selected cloners")
		, FSlateIcon()
		, FUIAction(
			FExecuteAction::CreateLambda(&ExecuteEnableClonerAction, InMenuData, bEnable)
		)
	);

	DisableClonerSection.AddMenuEntry(
		TEXT("DisableClonerLevel")
		, LOCTEXT("DisableClonerLevel.Label", "Disable level cloner")
		, LOCTEXT("DisableClonerLevel.Tooltip", "Disable selected level cloners")
		, FSlateIcon()
		, FUIAction(
			FExecuteAction::CreateLambda(&ExecuteEnableLevelClonerAction, InMenuData, bEnable)
		)
	);
}

void UE::ClonerEditor::Menu::FillCreateClonerEffectorSection(UToolMenu* InMenu, const FCEEditorClonerMenuData& InMenuData)
{
	if (!InMenu
		|| InMenuData.Context.IsEmpty()
		|| !InMenuData.Context.ContainsAnyEnabledCloner())
	{
		return;
	}

	FToolMenuSection* ClonerSection = FindOrAddClonerSection(InMenu);

	check(ClonerSection)

	TSet<FName> EffectorTypes;
	UCEEffectorLibrary::GetEffectorTypeNames(EffectorTypes);

	for (const FName& EffectorType : EffectorTypes)
	{
		const FName MenuName(TEXT("CreateClonerLinkedEffector") + EffectorType.ToString());
		const FText MenuLabel = FText::Format(LOCTEXT("CreateClonerLinkedEffector.Label", "Create {0} effector"), FText::FromName(EffectorType));
		const FText MenuTooltip = FText::Format(LOCTEXT("CreateClonerLinkedEffector.Tooltip", "Create a linked {0} effector for selected cloners"), FText::FromName(EffectorType));
		
		ClonerSection->AddMenuEntry(
			MenuName
			, MenuLabel
			, MenuTooltip
			, FSlateIconFinder::FindIconForClass(ACEEffectorActor::StaticClass())
			, FUIAction(
				FExecuteAction::CreateLambda(&ExecuteCreateClonerEffectorAction, InMenuData, EffectorType)
			)
		);
	}
}

void UE::ClonerEditor::Menu::FillConvertClonerSection(UToolMenu* InMenu, const FCEEditorClonerMenuData& InMenuData)
{
	if (!InMenu
		|| InMenuData.Context.IsEmpty()
		|| !InMenuData.Context.ContainsAnyEnabledCloner())
	{
		return;
	}

	FToolMenuSection& ConvertClonerSection = InMenu->FindOrAddSection(
		TEXT("ConvertCloner")
		, LOCTEXT("ConvertCloner.Label", "Convert cloner")
	);

	ConvertClonerSection.AddMenuEntry(
		TEXT("ConvertClonerToStaticMesh")
		, LOCTEXT("ConvertClonerToStaticMesh.Label", "To Static Mesh")
		, LOCTEXT("ConvertClonerToStaticMesh.Tooltip", "Convert selected cloners to static mesh")
		, FSlateIcon()
		, FUIAction(
			FExecuteAction::CreateLambda(&ExecuteConvertClonerAction, InMenuData, ECEClonerMeshConversion::StaticMesh)
		)
	);

	ConvertClonerSection.AddMenuEntry(
		TEXT("ConvertClonerToStaticMeshes")
		, LOCTEXT("ConvertClonerToStaticMeshes.Label", "To Static Meshes")
		, LOCTEXT("ConvertClonerToStaticMeshes.Tooltip", "Convert selected cloners to static meshes")
		, FSlateIcon()
		, FUIAction(
			FExecuteAction::CreateLambda(&ExecuteConvertClonerAction, InMenuData, ECEClonerMeshConversion::StaticMeshes)
		)
	);

	ConvertClonerSection.AddMenuEntry(
		TEXT("ConvertClonerToDynamicMesh")
		, LOCTEXT("ConvertClonerToDynamicMesh.Label", "To Dynamic Mesh")
		, LOCTEXT("ConvertClonerToDynamicMesh.Tooltip", "Convert selected cloners to dynamic mesh")
		, FSlateIcon()
		, FUIAction(
			FExecuteAction::CreateLambda(&ExecuteConvertClonerAction, InMenuData, ECEClonerMeshConversion::DynamicMesh)
		)
	);

	ConvertClonerSection.AddMenuEntry(
		TEXT("ConvertClonerToDynamicMeshes")
		, LOCTEXT("ConvertClonerToDynamicMeshes.Label", "To Dynamic Meshes")
		, LOCTEXT("ConvertClonerToDynamicMeshes.Tooltip", "Convert selected cloners to dynamic meshes")
		, FSlateIcon()
		, FUIAction(
			FExecuteAction::CreateLambda(&ExecuteConvertClonerAction, InMenuData, ECEClonerMeshConversion::DynamicMeshes)
		)
	);

	ConvertClonerSection.AddMenuEntry(
		TEXT("ConvertClonerToInstancedStaticMesh")
		, LOCTEXT("ConvertClonerToInstancedStaticMesh.Label", "To Instanced Static Mesh")
		, LOCTEXT("ConvertClonerToInstancedStaticMesh.Tooltip", "Convert selected cloners to instanced static mesh")
		, FSlateIcon()
		, FUIAction(
			FExecuteAction::CreateLambda(&ExecuteConvertClonerAction, InMenuData, ECEClonerMeshConversion::InstancedStaticMesh)
		)
	);
}

void UE::ClonerEditor::Menu::FillCreateClonerSection(UToolMenu* InMenu, const FCEEditorClonerMenuData& InMenuData)
{
	if (!InMenu
		|| InMenuData.Context.IsEmpty()
		|| !InMenuData.Context.ContainsAnyActor())
	{
		return;
	}

	FToolMenuSection* ClonerSection = FindOrAddClonerSection(InMenu);

	check(ClonerSection)

	TSet<FName> ClonerLayouts;
	UCEClonerLibrary::GetClonerLayoutNames(ClonerLayouts);

	for (const FName& ClonerLayout : ClonerLayouts)
	{
		const FName MenuName(TEXT("CreateCloner") + ClonerLayout.ToString());
		const FText MenuLabel = FText::Format(LOCTEXT("CreateCloner.Label", "Create {0} cloner"), FText::FromName(ClonerLayout));
		const FText MenuTooltip = FText::Format(LOCTEXT("CreateCloner.Tooltip", "Create a {0} cloner with current selection attached"), FText::FromName(ClonerLayout));

		ClonerSection->AddMenuEntry(
			MenuName
			, MenuLabel
			, MenuTooltip
			, FSlateIconFinder::FindIconForClass(ACEClonerActor::StaticClass())
			, FUIAction(
				FExecuteAction::CreateLambda(&ExecuteCreateClonerAction, InMenuData, ClonerLayout)
			)
		);
	}
}

void UE::ClonerEditor::Menu::ExecuteEnableClonerAction(const FCEEditorClonerMenuData& InMenuData, bool bInEnable)
{
	UCEClonerSubsystem* Subsystem = UCEClonerSubsystem::Get();

	if (InMenuData.Context.IsEmpty() || !Subsystem)
	{
		return;
	}

	Subsystem->SetClonersEnabled(InMenuData.Context.GetCloners(), bInEnable, InMenuData.Options.ShouldTransact());
}

void UE::ClonerEditor::Menu::ExecuteEnableLevelClonerAction(const FCEEditorClonerMenuData& InMenuData, bool bInEnable)
{
	UCEClonerSubsystem* Subsystem = UCEClonerSubsystem::Get();
	const UWorld* World = InMenuData.Context.GetWorld();

	if (!IsValid(World) || !Subsystem)
	{
		return;
	}

	Subsystem->SetLevelClonersEnabled(World, bInEnable, InMenuData.Options.ShouldTransact());
}

void UE::ClonerEditor::Menu::ExecuteCreateClonerEffectorAction(const FCEEditorClonerMenuData& InMenuData, FName InEffectorType)
{
	UCEClonerSubsystem* Subsystem = UCEClonerSubsystem::Get();

	if (!Subsystem)
	{
		return;
	}

	UCEClonerSubsystem::ECEClonerActionFlags Flags = UCEClonerSubsystem::ECEClonerActionFlags::ShouldSelect;

	if (InMenuData.Options.ShouldTransact())
	{
		EnumAddFlags(Flags, UCEClonerSubsystem::ECEClonerActionFlags::ShouldTransact);
	}

	Subsystem->CreateLinkedEffectors(
		InMenuData.Context.GetCloners().Array(),
		Flags,
		[InEffectorType](UCEEffectorComponent* InEffector)
		{
			InEffector->SetTypeName(InEffectorType);
		}
	);
}

void UE::ClonerEditor::Menu::ExecuteConvertClonerAction(const FCEEditorClonerMenuData& InMenuData, ECEClonerMeshConversion InToMeshType)
{
	UCEClonerSubsystem* Subsystem = UCEClonerSubsystem::Get();

	if (InMenuData.Context.IsEmpty() || !Subsystem)
	{
		return;
	}

	Subsystem->ConvertCloners(InMenuData.Context.GetEnabledCloners(), InToMeshType);
}

void UE::ClonerEditor::Menu::ExecuteCreateClonerAction(const FCEEditorClonerMenuData& InMenuData, FName InClonerLayout)
{
	UCEClonerSubsystem* Subsystem = UCEClonerSubsystem::Get();

	if (InMenuData.Context.IsEmpty() || !Subsystem)
	{
		return;
	}

	UCEClonerSubsystem::ECEClonerActionFlags Flags = UCEClonerSubsystem::ECEClonerActionFlags::ShouldSelect;

	if (InMenuData.Options.ShouldTransact())
	{
		EnumAddFlags(Flags, UCEClonerSubsystem::ECEClonerActionFlags::ShouldTransact);
	}

	if (UCEClonerComponent* Cloner = Subsystem->CreateClonerWithActors(InMenuData.Context.GetWorld(), InMenuData.Context.GetActors(), Flags))
	{
		Cloner->SetLayoutName(InClonerLayout);
	}
}

#undef LOCTEXT_NAMESPACE
