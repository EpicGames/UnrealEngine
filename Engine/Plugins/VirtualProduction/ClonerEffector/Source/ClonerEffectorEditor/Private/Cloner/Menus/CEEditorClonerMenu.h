// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CEClonerEffectorShared.h"
#include "Cloner/Menus/CEEditorClonerMenuContext.h"
#include "Cloner/Menus/CEEditorClonerMenuOptions.h"
#include "Templates/SharedPointer.h"
#include "ToolMenus.h"

namespace UE::ClonerEditor::Menu
{
	/** Used internally to group menu data together */
	struct FCEEditorClonerMenuData
	{
		FCEEditorClonerMenuData(const FCEEditorClonerMenuContext& InContext, const FCEEditorClonerMenuOptions& InOptions)
			: Context(InContext)
			, Options(InOptions)
		{}

		const FCEEditorClonerMenuContext Context;
		const FCEEditorClonerMenuOptions Options;
	};

	/** Sections */

	FToolMenuSection* FindOrAddClonerSection(UToolMenu* InMenu);

	void FillEnableClonerSection(UToolMenu* InMenu, const FCEEditorClonerMenuData& InMenuData);

	void FillDisableClonerSection(UToolMenu* InMenu, const FCEEditorClonerMenuData& InMenuData);

	void FillCreateClonerEffectorSection(UToolMenu* InMenu, const FCEEditorClonerMenuData& InMenuData);

	void FillConvertClonerSection(UToolMenu* InMenu, const FCEEditorClonerMenuData& InMenuData);

	void FillCreateClonerSection(UToolMenu* InMenu, const FCEEditorClonerMenuData& InMenuData);

	/** Actions */

	void ExecuteEnableClonerAction(const FCEEditorClonerMenuData& InMenuData, bool bInEnable);

	void ExecuteEnableLevelClonerAction(const FCEEditorClonerMenuData& InMenuData, bool bInEnable);

	void ExecuteCreateClonerEffectorAction(const FCEEditorClonerMenuData& InMenuData, FName InEffectorType);

	void ExecuteConvertClonerAction(const FCEEditorClonerMenuData& InMenuData, ECEClonerMeshConversion InToMeshType);

	void ExecuteCreateClonerAction(const FCEEditorClonerMenuData& InMenuData, FName InClonerLayout);
}
