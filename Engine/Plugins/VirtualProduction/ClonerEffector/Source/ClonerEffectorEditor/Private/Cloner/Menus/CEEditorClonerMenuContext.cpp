// Copyright Epic Games, Inc. All Rights Reserved.

#include "Cloner/Menus/CEEditorClonerMenuContext.h"

#include "GameFramework/Actor.h"
#include "Cloner/CEClonerComponent.h"

FCEEditorClonerMenuContext::FCEEditorClonerMenuContext(const TSet<UObject*>& InObjects)
{
	for (UObject* Object : InObjects)
	{
		if (!IsValid(Object))
		{
			continue;
		}

		if (AActor* Actor = Cast<AActor>(Object))
		{
			ContextActorsKey.Add(Actor);

			TArray<UCEClonerComponent*> ClonerComponents;
			Actor->GetComponents(ClonerComponents, /** IncludeChildren */false);

			for (UCEClonerComponent* Component : ClonerComponents)
			{
				if (IsValid(Component))
				{
					ContextComponentsKey.Add(Component);
				}
			}
		}
		else if (UCEClonerComponent* Component = Cast<UCEClonerComponent>(Object))
		{
			ContextComponentsKey.Add(Component);
			ContextActorsKey.Add(Component->GetOwner());
		}
	}
}

TSet<AActor*> FCEEditorClonerMenuContext::GetActors() const
{
	TSet<AActor*> Actors;
	Actors.Reserve(ContextActorsKey.Num());

	Algo::TransformIf(
		ContextActorsKey
		, Actors
		, [](const TObjectKey<AActor>& InActorKey)
		{
			return IsValid(InActorKey.ResolveObjectPtr());
		}
		, [](const TObjectKey<AActor>& InActorKey)
		{
			return InActorKey.ResolveObjectPtr();
		}
	);

	return Actors;
}

TSet<UCEClonerComponent*> FCEEditorClonerMenuContext::GetCloners() const
{
	TSet<UCEClonerComponent*> Components;
	Components.Reserve(ContextComponentsKey.Num());

	Algo::TransformIf(
		ContextComponentsKey
		, Components
		, [](const TObjectKey<UCEClonerComponent>& InComponentKey)
		{
			return IsValid(InComponentKey.ResolveObjectPtr());
		}
		, [](const TObjectKey<UCEClonerComponent>& InComponentKey)
		{
			return InComponentKey.ResolveObjectPtr();
		}
	);

	return Components;
}

TSet<UCEClonerComponent*> FCEEditorClonerMenuContext::GetDisabledCloners() const
{
	return GetStateCloners(/** IsEnabled */false);
}

TSet<UCEClonerComponent*> FCEEditorClonerMenuContext::GetEnabledCloners() const
{
	return GetStateCloners(/** IsEnabled */true);
}

UWorld* FCEEditorClonerMenuContext::GetWorld() const
{
	for (const TObjectKey<AActor>& ActorKey : ContextActorsKey)
	{
		if (const AActor* Actor = ActorKey.ResolveObjectPtr())
		{
			return Actor->GetWorld();
		}
	}

	for (const TObjectKey<UCEClonerComponent>& ComponentKey : ContextComponentsKey)
	{
		if (const UCEClonerComponent* Component = ComponentKey.ResolveObjectPtr())
		{
			return Component->GetWorld();
		}
	}

	return nullptr;
}

bool FCEEditorClonerMenuContext::IsEmpty() const
{
	return ContextComponentsKey.IsEmpty() && ContextActorsKey.IsEmpty();
}

bool FCEEditorClonerMenuContext::ContainsAnyActor() const
{
	return !ContextActorsKey.IsEmpty();
}

bool FCEEditorClonerMenuContext::ContainsAnyCloner() const
{
	return !ContextComponentsKey.IsEmpty();
}

bool FCEEditorClonerMenuContext::ContainsAnyDisabledCloner() const
{
	return ContainsClonerState(/** IsEnabled */false);
}

bool FCEEditorClonerMenuContext::ContainsAnyEnabledCloner() const
{
	return ContainsClonerState(/** IsEnabled */true);
}

bool FCEEditorClonerMenuContext::ContainsClonerState(bool bInState) const
{
	for (const TObjectKey<UCEClonerComponent>& ComponentKey : ContextComponentsKey)
	{
		if (const UCEClonerComponent* Component = ComponentKey.ResolveObjectPtr())
		{
			if (Component->GetEnabled() == bInState)
			{
				return true;
			}
		}
	}

	return false;
}

TSet<UCEClonerComponent*> FCEEditorClonerMenuContext::GetStateCloners(bool bInState) const
{
	TSet<UCEClonerComponent*> Cloners;
	Cloners.Reserve(ContextComponentsKey.Num());

	for (const TObjectKey<UCEClonerComponent>& ComponentKey : ContextComponentsKey)
	{
		if (UCEClonerComponent* Component = ComponentKey.ResolveObjectPtr())
		{
			if (Component->GetEnabled() == bInState)
			{
				Cloners.Add(Component);
			}
		}
	}

	return Cloners;
}