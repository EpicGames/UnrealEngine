// Copyright Epic Games, Inc. All Rights Reserved.

#include "Cloner/Menus/CEEditorClonerMenuOptions.h"

#include "Misc/EnumClassFlags.h"

FCEEditorClonerMenuOptions::FCEEditorClonerMenuOptions(const TSet<ECEEditorClonerMenuType>& InMenus)
{
	for (const ECEEditorClonerMenuType& Menu : InMenus)
	{
		MenuTypes |= static_cast<uint8>(Menu);
	}
}

FCEEditorClonerMenuOptions& FCEEditorClonerMenuOptions::CreateSubMenu(bool bInCreateSubMenu)
{
	bCreateSubMenu = bInCreateSubMenu;
	return *this;
}

FCEEditorClonerMenuOptions& FCEEditorClonerMenuOptions::UseTransact(bool bInUseTransact)
{
	bUseTransact = bInUseTransact;
	return *this;
}

bool FCEEditorClonerMenuOptions::IsMenuType(ECEEditorClonerMenuType InMenuType) const
{
	return EnumHasAnyFlags(static_cast<ECEEditorClonerMenuType>(MenuTypes), InMenuType);
}