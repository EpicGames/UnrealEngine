// Copyright Epic Games, Inc. All Rights Reserved.

#include "Cloner/Sequencer/MovieSceneClonerTrackEditor.h"

#include "Cloner/CEClonerComponent.h"
#include "MovieScene/MovieSceneNiagaraSystemSpawnSection.h"
#include "MovieScene/MovieSceneNiagaraSystemTrack.h"
#include "Niagara/Sequencer/MovieSceneNiagaraCacheSection.h"
#include "Niagara/Sequencer/MovieSceneNiagaraCacheTrack.h"

#define LOCTEXT_NAMESPACE "MovieSceneClonerTrackEditor"

FMovieSceneClonerTrackEditor::FOnAddClonerTrack FMovieSceneClonerTrackEditor::OnAddClonerTrack;
FMovieSceneClonerTrackEditor::FOnClonerTrackExists FMovieSceneClonerTrackEditor::OnClonerTrackExists;

FMovieSceneClonerTrackEditor::~FMovieSceneClonerTrackEditor()
{
	OnAddClonerTrack.RemoveAll(this);
	OnClonerTrackExists.RemoveAll(this);
}

FText FMovieSceneClonerTrackEditor::GetDisplayName() const
{
	return LOCTEXT("ClonerTrackEditor_DisplayName", "Cloner");
}

bool FMovieSceneClonerTrackEditor::SupportsType(TSubclassOf<UMovieSceneTrack> InTrackClass) const
{
	return InTrackClass == UMovieSceneNiagaraCacheTrack::StaticClass()
		|| InTrackClass == UMovieSceneNiagaraSystemTrack::StaticClass();
}

void FMovieSceneClonerTrackEditor::BindDelegates()
{
	OnAddClonerTrack.AddSP(this, &FMovieSceneClonerTrackEditor::ExecuteAddTrack);
	OnClonerTrackExists.AddSP(this, &FMovieSceneClonerTrackEditor::ExecuteTrackExists);
}

void FMovieSceneClonerTrackEditor::ExecuteAddTrack(const TSet<UCEClonerComponent*>& InCloners)
{
	if (InCloners.IsEmpty())
	{
		return;
	}

	UMovieScene* FocusedMovieScene = GetFocusedMovieScene();
	if (!FocusedMovieScene || FocusedMovieScene->IsReadOnly())
	{
		return;
	}

	const TSharedPtr<ISequencer> SequencerPtr = GetSequencer();
	if (!SequencerPtr)
	{
		return;
	}

	const FScopedTransaction Transaction(LOCTEXT("AddClonerTracks", "Add Cloner Tracks"));

	FocusedMovieScene->Modify();

	for (UCEClonerComponent* Cloner : InCloners)
	{
		if (!IsValid(Cloner))
		{
			continue;
		}

		const FGuid ComponentBinding = SequencerPtr->GetHandleToObject(Cloner);
		if (!ComponentBinding.IsValid() || FocusedMovieScene->FindSpawnable(ComponentBinding))
		{
			// we only want to add tracks for possessables
			return;
		}

		// Add lifecycle track
		const UMovieSceneTrack* LifeCycleTrack = FocusedMovieScene->FindTrack(UMovieSceneNiagaraSystemTrack::StaticClass(), ComponentBinding);

		if (!LifeCycleTrack)
		{
			if (UMovieSceneNiagaraSystemTrack* NiagaraSystemTrack = FocusedMovieScene->AddTrack<UMovieSceneNiagaraSystemTrack>(ComponentBinding))
			{
				NiagaraSystemTrack->SetDisplayName(LOCTEXT("ClonerLifeCycleTrackName", "Cloner Life Cycle"));

				UMovieSceneNiagaraSystemSpawnSection* NiagaraSpawnSection = CastChecked<UMovieSceneNiagaraSystemSpawnSection>(NiagaraSystemTrack->CreateNewSection());
				NiagaraSpawnSection->SetAgeUpdateMode(ENiagaraAgeUpdateMode::DesiredAge);

				NiagaraSpawnSection->SetRange(TRange<FFrameNumber>(
					FocusedMovieScene->GetPlaybackRange().GetLowerBound(),
					FocusedMovieScene->GetPlaybackRange().GetUpperBound()
				));

				NiagaraSystemTrack->AddSection(*NiagaraSpawnSection);
			}
		}

		// Add cache track
		const UMovieSceneTrack* CacheTrack = FocusedMovieScene->FindTrack(UMovieSceneNiagaraCacheTrack::StaticClass(), ComponentBinding);

		if (!CacheTrack)
		{
			if (UMovieSceneNiagaraCacheTrack* NiagaraCacheTrack = FocusedMovieScene->AddTrack<UMovieSceneNiagaraCacheTrack>(ComponentBinding))
			{
				NiagaraCacheTrack->SetDisplayName(LOCTEXT("ClonerSimCacheTrackName", "Cloner Sim Cache"));

				UMovieSceneNiagaraCacheSection* NiagaraCacheSection = CastChecked<UMovieSceneNiagaraCacheSection>(NiagaraCacheTrack->CreateNewSection());

				NiagaraCacheSection->SetRange(TRange<FFrameNumber>(
					FocusedMovieScene->GetPlaybackRange().GetLowerBound(),
					FocusedMovieScene->GetPlaybackRange().GetUpperBound()
				));

				NiagaraCacheTrack->AddSection(*NiagaraCacheSection);
			}
		}
	}

	SequencerPtr->NotifyMovieSceneDataChanged(EMovieSceneDataChangeType::MovieSceneStructureItemAdded);
}

void FMovieSceneClonerTrackEditor::ExecuteTrackExists(UCEClonerComponent* InCloner, uint32& OutCount) const
{
	const UMovieScene* FocusedMovieScene = GetFocusedMovieScene();
	if (!FocusedMovieScene)
	{
		return;
	}

	const TSharedPtr<ISequencer> SequencerPtr = GetSequencer();
	if (!SequencerPtr)
	{
		return;
	}

	const FGuid ComponentBinding = SequencerPtr->GetHandleToObject(InCloner, /** CreateIfMissing */false);
	if (!ComponentBinding.IsValid())
	{
		return;
	}

	if (UMovieSceneTrack* NiagaraSystemTrack = FocusedMovieScene->FindTrack(UMovieSceneNiagaraSystemTrack::StaticClass(), ComponentBinding))
	{
		OutCount++;
	}

	if (UMovieSceneTrack* NiagaraCacheTrack = FocusedMovieScene->FindTrack(UMovieSceneNiagaraCacheTrack::StaticClass(), ComponentBinding))
	{
		OutCount++;
	}
}

#undef LOCTEXT_NAMESPACE
