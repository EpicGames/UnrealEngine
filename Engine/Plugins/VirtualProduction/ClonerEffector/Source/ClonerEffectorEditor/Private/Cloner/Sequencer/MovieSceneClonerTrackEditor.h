// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MovieSceneTrackEditor.h"

class UCEClonerComponent;
class ISequencer;

/** Cloner track editor to add niagara track and section */
class FMovieSceneClonerTrackEditor : public FMovieSceneTrackEditor
{
public:
	DECLARE_MULTICAST_DELEGATE_OneParam(FOnAddClonerTrack, const TSet<UCEClonerComponent*>& /** InCloners */)
	static FOnAddClonerTrack OnAddClonerTrack;

	DECLARE_MULTICAST_DELEGATE_TwoParams(FOnClonerTrackExists, UCEClonerComponent* /** InCloner */, uint32& /** OutTrackCount */)
	static FOnClonerTrackExists OnClonerTrackExists;

	FMovieSceneClonerTrackEditor(const TSharedRef<ISequencer>& InSequencer)
		: FMovieSceneTrackEditor(InSequencer)
	{}

	virtual ~FMovieSceneClonerTrackEditor() override;

	static TSharedRef<ISequencerTrackEditor> CreateTrackEditor(TSharedRef<ISequencer> InSequencer)
	{
		TSharedRef<FMovieSceneClonerTrackEditor> TrackEditor = MakeShared<FMovieSceneClonerTrackEditor>(InSequencer);
		TrackEditor->BindDelegates();
		return TrackEditor;
	}

private:
	//~ Begin FMovieSceneTrackEditor
	virtual FText GetDisplayName() const override;
	virtual bool SupportsType(TSubclassOf<UMovieSceneTrack> InTrackClass) const override;
	//~ End FMovieSceneTrackEditor

	void BindDelegates();
	void ExecuteAddTrack(const TSet<UCEClonerComponent*>& InCloners);
	void ExecuteTrackExists(UCEClonerComponent* InCloner, uint32& OutCount) const;

};