// Copyright Epic Games, Inc. All Rights Reserved.

#include "Effector/CEEffectorActorFactory.h"
#include "Effector/CEEffectorActor.h"
#include "Effector/CEEffectorComponent.h"
#include "EngineAnalytics.h"
#include "Subsystems/PlacementSubsystem.h"

UCEEffectorActorFactory::UCEEffectorActorFactory()
{
	NewActorClass = ACEEffectorActor::StaticClass();
}

void UCEEffectorActorFactory::SetEffectorTypeName(FName InEffectorTypeName)
{
	EffectorTypeName = InEffectorTypeName;
}

void UCEEffectorActorFactory::PostSpawnActor(UObject* InAsset, AActor* InNewActor)
{
	Super::PostSpawnActor(InAsset, InNewActor);

	if (!EffectorTypeName.IsNone())
	{
		if (const ACEEffectorActor* EffectorActor = Cast<ACEEffectorActor>(InNewActor))
		{
			if (UCEEffectorComponent* EffectorComponent = EffectorActor->GetEffectorComponent())
			{
				EffectorComponent->SetTypeName(EffectorTypeName);
			}
		}
	}
}

void UCEEffectorActorFactory::PostPlaceAsset(TArrayView<const FTypedElementHandle> InHandle, const FAssetPlacementInfo& InPlacementInfo, const FPlacementOptions& InPlacementOptions)
{
	Super::PostPlaceAsset(InHandle, InPlacementInfo, InPlacementOptions);

	if (!InPlacementOptions.bIsCreatingPreviewElements && FEngineAnalytics::IsAvailable())
	{
		TArray<FAnalyticsEventAttribute> Attributes;
		Attributes.Reserve(2);
		Attributes.Emplace(TEXT("ToolClass"), GetNameSafe(GetClass()));
		Attributes.Emplace(TEXT("ActorClass"), GetNameSafe(NewActorClass));
		FEngineAnalytics::GetProvider().RecordEvent(TEXT("Editor.Usage.MotionDesign.PlaceActor"), Attributes);
	}
}
