// Copyright Epic Games, Inc. All Rights Reserved.

#include "Effector/Customizations/CEEditorEffectorActorDetailCustomization.h"

#include "Effector/CEEffectorActor.h"
#include "Modules/ModuleManager.h"
#include "PropertyEditorModule.h"

void FCEEditorEffectorActorDetailCustomization::CustomizeDetails(IDetailLayoutBuilder& InDetailBuilder)
{
	// Add customizations for actor here
}

void FCEEditorEffectorActorDetailCustomization::RemoveEmptySections()
{
	FPropertyEditorModule& PropertyModule = FModuleManager::GetModuleChecked<FPropertyEditorModule>("PropertyEditor");
	const FName ComponentClassName = ACEEffectorActor::StaticClass()->GetFName();

	// Remove Streaming section by removing its categories
	TSharedRef<FPropertySection> StreamingSection = PropertyModule.FindOrCreateSection(ComponentClassName, "Streaming", FText::GetEmpty());
	StreamingSection->RemoveCategory("WorldPartition");
	StreamingSection->RemoveCategory("DataLayers");
	StreamingSection->RemoveCategory("HLOD");
}
