// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IDetailCustomization.h"
#include "Templates/SharedPointer.h"

/** Used to customize effector actor properties in details panel */
class FCEEditorEffectorActorDetailCustomization : public IDetailCustomization
{
public:
	static TSharedRef<IDetailCustomization> MakeInstance()
	{
		return MakeShared<FCEEditorEffectorActorDetailCustomization>();
	}

	explicit FCEEditorEffectorActorDetailCustomization()
	{
		RemoveEmptySections();
	}

	//~ Begin IDetailCustomization
	virtual void CustomizeDetails(IDetailLayoutBuilder& InDetailBuilder) override;
	//~ End IDetailCustomization

protected:
	static void RemoveEmptySections();
};
