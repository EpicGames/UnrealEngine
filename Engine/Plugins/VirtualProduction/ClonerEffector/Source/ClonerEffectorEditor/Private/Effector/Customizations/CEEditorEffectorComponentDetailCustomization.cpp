// Copyright Epic Games, Inc. All Rights Reserved.

#include "Effector/Customizations/CEEditorEffectorComponentDetailCustomization.h"

#include "CEEditorInputPreprocessor.h"
#include "CEEditorModule.h"
#include "DetailLayoutBuilder.h"
#include "DetailWidgetRow.h"
#include "Effector/CEEffectorComponent.h"
#include "Effector/CEEffectorExtensionBase.h"
#include "Effector/Effects/CEEffectorEffectBase.h"
#include "Effector/Modes/CEEffectorModeBase.h"
#include "Effector/Types/CEEffectorTypeBase.h"
#include "IPropertyUtilities.h"
#include "Modules/ModuleManager.h"
#include "PropertyEditorModule.h"

#define LOCTEXT_NAMESPACE "CEEditorEffectorComponentDetailCustomization"

void FCEEditorEffectorComponentDetailCustomization::CustomizeDetails(IDetailLayoutBuilder& InDetailBuilder)
{
	FPropertyEditorModule& PropertyModule = FModuleManager::GetModuleChecked<FPropertyEditorModule>(TEXT("PropertyEditor"));
	const FName ComponentClassName = UCEEffectorComponent::StaticClass()->GetFName();

	TSharedRef<IPropertyUtilities> PropertyUtilities = InDetailBuilder.GetPropertyUtilities();

	// Place TypeName property above all properties in the category
	{
		TSharedRef<IPropertyHandle> TypeHandle = InDetailBuilder.GetProperty(UCEEffectorComponent::GetTypeNamePropertyName(), UCEEffectorComponent::StaticClass());
		TypeHandle->SetOnPropertyValueChangedWithData(TDelegate<void(const FPropertyChangedEvent&)>::CreateStatic(&FCEEditorEffectorComponentDetailCustomization::OnPropertyChanged, PropertyUtilities.ToWeakPtr()));

		IDetailCategoryBuilder& ShapeCategoryBuilder = InDetailBuilder.EditCategory(TEXT("Shape"), FText::FromName(TEXT("Shape")));
		ShapeCategoryBuilder.AddProperty(TypeHandle);
	}

	// Place ModeName property above all properties in the category
	{
		TSharedRef<IPropertyHandle> ModeHandle = InDetailBuilder.GetProperty(UCEEffectorComponent::GetModeNamePropertyName(), UCEEffectorComponent::StaticClass());
		ModeHandle->SetOnPropertyValueChangedWithData(TDelegate<void(const FPropertyChangedEvent&)>::CreateStatic(&FCEEditorEffectorComponentDetailCustomization::OnPropertyChanged, PropertyUtilities.ToWeakPtr()));

		IDetailCategoryBuilder& ModeCategoryBuilder = InDetailBuilder.EditCategory(TEXT("Mode"), FText::FromName(TEXT("Mode")));
		ModeCategoryBuilder.AddProperty(ModeHandle);
	}

	const TArray<TWeakObjectPtr<UCEEffectorComponent>> EffectorComponentsWeak = InDetailBuilder.GetObjectsOfTypeBeingCustomized<UCEEffectorComponent>();

	// Everything needs to be below Effector category
	const IDetailCategoryBuilder& EffectorCategoryBuilder = InDetailBuilder.EditCategory(TEXT("Effector"), FText::FromName(TEXT("Effector")));
	const int32 ExtensionSortOrderOffset = 2;
	const int32 StartSortOrder = EffectorCategoryBuilder.GetSortOrder() + 1;

	// Group same class objects together so their properties are grouped in the details panel when multiple effectors are selected
	struct FDetailsCategoryData
	{
		FName SectionName;
		int32 SortOrder;
		TArray<UObject*> Objects;
	};

	TMap<FName, FDetailsCategoryData> CategoryToData;
	for (const TWeakObjectPtr<UCEEffectorComponent>& EffectorComponentWeak : EffectorComponentsWeak)
	{
		const UCEEffectorComponent* EffectorComponent = EffectorComponentWeak.Get();

		if (!EffectorComponent)
		{
			continue;
		}

		if (UCEEffectorTypeBase* ActiveType = EffectorComponent->GetActiveType())
		{
			FDetailsCategoryData& CategoryData = CategoryToData.FindOrAdd(TEXT("Shape"));
			CategoryData.SectionName = ActiveType->GetExtensionSection().SectionName;
			CategoryData.SortOrder = StartSortOrder + ActiveType->GetExtensionSection().SectionOrder;
			CategoryData.Objects.Add(ActiveType);
		}

		if (UCEEffectorModeBase* ActiveMode = EffectorComponent->GetActiveMode())
		{
			FDetailsCategoryData& CategoryData = CategoryToData.FindOrAdd(TEXT("Mode"));
			CategoryData.SectionName = ActiveMode->GetExtensionSection().SectionName;
			CategoryData.SortOrder = StartSortOrder + ActiveMode->GetExtensionSection().SectionOrder;
			CategoryData.Objects.Add(ActiveMode);
		}

		for (const TObjectPtr<UCEEffectorEffectBase>& ActiveEffect : EffectorComponent->GetActiveEffects())
		{
			FDetailsCategoryData& CategoryData = CategoryToData.FindOrAdd(ActiveEffect->GetExtensionName());
			CategoryData.SectionName = ActiveEffect->GetExtensionSection().SectionName;
			CategoryData.SortOrder = StartSortOrder + ExtensionSortOrderOffset + ActiveEffect->GetExtensionSection().SectionOrder;
			CategoryData.Objects.Add(ActiveEffect);
		}
	}

	const FName EffectorSectionName = TEXT("Effector");
	const TSharedRef<FPropertySection> EffectorSection = PropertyModule.FindOrCreateSection(ComponentClassName, EffectorSectionName, FText::FromName(EffectorSectionName));
	EffectorSection->AddCategory(TEXT("Effector"));

	FAddPropertyParams AddParams;
	AddParams.CreateCategoryNodes(false);
	AddParams.HideRootObjectNode(true);

	const TSharedPtr<FCEEditorInputPreprocessor> InputPreprocessor = FCEEditorModule::GetInputPreprocessor();

	if (InputPreprocessor.IsValid())
	{
		InputPreprocessor->Unregister();
	}

	for (const TPair<FName,FDetailsCategoryData>& CategoryToObjectsPair : CategoryToData)
	{
		const FName CategoryName = CategoryToObjectsPair.Key;

		if (CategoryName.IsNone() || CategoryToObjectsPair.Value.Objects.IsEmpty())
		{
			continue;
		}

		IDetailCategoryBuilder& CategoryBuilder = InDetailBuilder.EditCategory(CategoryName, FText::FromName(CategoryName));
		CategoryBuilder.SetSortOrder(CategoryToObjectsPair.Value.SortOrder);

		const TSharedRef<FPropertySection> PropertySection = PropertyModule.FindOrCreateSection(ComponentClassName, CategoryToObjectsPair.Value.SectionName, FText::FromName(CategoryToObjectsPair.Value.SectionName));
		PropertySection->AddCategory(CategoryName);

		if (IDetailPropertyRow* ObjectRow = CategoryBuilder.AddExternalObjects(CategoryToObjectsPair.Value.Objects, EPropertyLocation::Default, AddParams))
		{
			TSharedPtr<IPropertyHandle> ObjectPropertyHandle = ObjectRow->GetPropertyHandle();

			// Fix for EditConditionHides not appearing when condition is met due to AddExternalObjects not rebuilding children
			ObjectPropertyHandle->SetOnChildPropertyValueChangedWithData(TDelegate<void(const FPropertyChangedEvent&)>::CreateStatic(&FCEEditorEffectorComponentDetailCustomization::OnChildPropertyChanged, ObjectPropertyHandle->AsWeak()));

			// Fix to refocus last widget when TAB is used since RequestRebuildChildren will lose focus
			if (InputPreprocessor.IsValid())
			{
				InputPreprocessor->Register();
			}
		}
	}
}

void FCEEditorEffectorComponentDetailCustomization::RemoveEmptySections()
{
	FPropertyEditorModule& PropertyModule = FModuleManager::GetModuleChecked<FPropertyEditorModule>(TEXT("PropertyEditor"));
	const FName ComponentClassName = UCEEffectorComponent::StaticClass()->GetFName();

	// Remove sections
	PropertyModule.RemoveSection(ComponentClassName, TEXT("Streaming"));
}

void FCEEditorEffectorComponentDetailCustomization::OnPropertyChanged(const FPropertyChangedEvent& InEvent, TWeakPtr<IPropertyUtilities> InUtilitiesWeak)
{
	if (const TSharedPtr<IPropertyUtilities> PropertyUtilities = InUtilitiesWeak.Pin())
	{
		if (InEvent.ChangeType != EPropertyChangeType::Interactive)
		{
			PropertyUtilities->RequestForceRefresh();
		}
	}
}

void FCEEditorEffectorComponentDetailCustomization::OnChildPropertyChanged(const FPropertyChangedEvent& InEvent, TWeakPtr<IPropertyHandle> InParentHandleWeak)
{
	const TSharedPtr<IPropertyHandle> PropertyHandle = InParentHandleWeak.Pin();

	if (!PropertyHandle)
	{
		return;
	}

	if (InEvent.ChangeType == EPropertyChangeType::Interactive || !InEvent.MemberProperty)
	{
		return;
	}

	if (!InEvent.MemberProperty->HasMetaData(TEXT("RefreshPropertyView")))
	{
		return;
	}

	const TSharedPtr<FCEEditorInputPreprocessor> InputPreprocessor = FCEEditorModule::GetInputPreprocessor();

	if (InputPreprocessor.IsValid())
	{
		// Refocus last focused widget next tick because rebuilding children loses focus
		InputPreprocessor->RefocusLastWidget();
		InputPreprocessor->Unregister();
	}

	PropertyHandle->RequestRebuildChildren();
}

#undef LOCTEXT_NAMESPACE
