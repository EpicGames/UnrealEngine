// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IDetailCustomization.h"
#include "Templates/SharedPointerFwd.h"

struct FPropertyChangedEvent;
class IPropertyHandle;
class IPropertyUtilities;

/** Used to customize effector component properties in details panel */
class FCEEditorEffectorComponentDetailCustomization : public IDetailCustomization
{
public:
	static TSharedRef<IDetailCustomization> MakeInstance()
	{
		return MakeShared<FCEEditorEffectorComponentDetailCustomization>();
	}

	explicit FCEEditorEffectorComponentDetailCustomization()
	{
		RemoveEmptySections();
	}

	//~ Begin IDetailCustomization
	virtual void CustomizeDetails(IDetailLayoutBuilder& InDetailBuilder) override;
	//~ End IDetailCustomization

protected:
	static void RemoveEmptySections();
	static void OnChildPropertyChanged(const FPropertyChangedEvent& InEvent, TWeakPtr<IPropertyHandle> InParentHandleWeak);
	static void OnPropertyChanged(const FPropertyChangedEvent& InEvent, TWeakPtr<IPropertyUtilities> InUtilitiesWeak);
};
