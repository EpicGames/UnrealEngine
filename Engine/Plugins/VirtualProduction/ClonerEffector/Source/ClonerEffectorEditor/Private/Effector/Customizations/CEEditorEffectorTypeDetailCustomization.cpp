// Copyright Epic Games, Inc. All Rights Reserved.

#include "Effector/Customizations/CEEditorEffectorTypeDetailCustomization.h"

#include "DetailLayoutBuilder.h"
#include "DetailWidgetRow.h"
#include "Effector/Types/CEEffectorBoundType.h"
#include "Styles/CEEditorStyle.h"
#include "Widgets/Layout/SBox.h"
#include "Widgets/Layout/SScaleBox.h"
#include "Widgets/Text/STextBlock.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Input/SComboBox.h"

DEFINE_LOG_CATEGORY_STATIC(LogCEEditorEffectorTypeDetailCustomization, Log, All);

#define LOCTEXT_NAMESPACE "CEEditorEffectorTypeDetailCustomization"

namespace UE::EffectorType
{
	// Sort from most dramatic to least IN then OUT then IN OUT, then specials
	static const TArray<ECEClonerEasing> SortedEasings
	{
		ECEClonerEasing::InExpo,
		ECEClonerEasing::InCirc,
		ECEClonerEasing::InQuint,
		ECEClonerEasing::InQuart,
		ECEClonerEasing::InQuad,
		ECEClonerEasing::InCubic,
		ECEClonerEasing::InSine,
		ECEClonerEasing::OutExpo,
		ECEClonerEasing::OutCirc,
		ECEClonerEasing::OutQuint,
		ECEClonerEasing::OutQuart,
		ECEClonerEasing::OutQuad,
		ECEClonerEasing::OutCubic,
		ECEClonerEasing::OutSine,
		ECEClonerEasing::InOutExpo,
		ECEClonerEasing::InOutCirc,
		ECEClonerEasing::InOutQuint,
		ECEClonerEasing::InOutQuart,
		ECEClonerEasing::InOutQuad,
		ECEClonerEasing::InOutCubic,
		ECEClonerEasing::InOutSine,
		ECEClonerEasing::Linear,
		ECEClonerEasing::InBounce,
		ECEClonerEasing::InBack,
		ECEClonerEasing::InElastic,
		ECEClonerEasing::OutBounce,
		ECEClonerEasing::OutBack,
		ECEClonerEasing::OutElastic,
		ECEClonerEasing::InOutBounce,
		ECEClonerEasing::InOutBack,
		ECEClonerEasing::InOutElastic,
		ECEClonerEasing::Random
	};
}

void FCEEditorEffectorTypeDetailCustomization::CustomizeDetails(IDetailLayoutBuilder& InDetailBuilder)
{
	EasingPropertyHandle = InDetailBuilder.GetProperty(
		GET_MEMBER_NAME_CHECKED(UCEEffectorBoundType, Easing),
		UCEEffectorBoundType::StaticClass()
	);

	// Customize easing curve property
	if (!EasingPropertyHandle->IsValidHandle())
	{
		return;
	}

	IDetailPropertyRow& EasingRow = InDetailBuilder.AddPropertyToCategory(EasingPropertyHandle);

	PopulateEasingInfos();

	FDetailWidgetRow& CustomWidget = EasingRow.CustomWidget();

	CustomWidget.NameContent()
		.VAlign(EVerticalAlignment::VAlign_Fill)
		[
			EasingPropertyHandle->CreatePropertyNameWidget()
		];

	CustomWidget.ValueContent()
		.VAlign(EVerticalAlignment::VAlign_Fill)
		[
			SNew(SComboBox<FName>)
			.ComboBoxStyle(&FCEEditorStyle::Get().GetWidgetStyle<FComboBoxStyle>(TEXT("ComboBox")))
			.OptionsSource(&EasingNames)
			.InitiallySelectedItem(GetCurrentEasingName())
			.ToolTipText(LOCTEXT("EasingTooltip", "Easings sorted from most dramatic to least and specials at the end"))
			.OnGenerateWidget(this, &FCEEditorEffectorTypeDetailCustomization::OnGenerateEasingEntry)
			.OnSelectionChanged(this, &FCEEditorEffectorTypeDetailCustomization::OnSelectionChanged)
			.ContentPadding(0.f)
			.Content()
			[
				OnGenerateEasingEntry(NAME_None)
			]
		];
}

void FCEEditorEffectorTypeDetailCustomization::PopulateEasingInfos()
{
	UEnum* EasingEnum = StaticEnum<ECEClonerEasing>();

	if (!EasingEnum)
	{
		return;
	}

	EasingEnumWeak = EasingEnum;

	check((EasingEnum->NumEnums()-1) == UE::EffectorType::SortedEasings.Num());

	EasingNames.Empty(UE::EffectorType::SortedEasings.Num());
	for (const ECEClonerEasing& Easing : UE::EffectorType::SortedEasings)
	{
		EasingNames.Add(FName(EasingEnum->GetNameStringByValue(static_cast<uint8>(Easing))));
	}
}

TSharedRef<SWidget> FCEEditorEffectorTypeDetailCustomization::OnGenerateEasingEntry(FName InName) const
{
	TSharedPtr<SWidget> ImageWidget;
	TSharedPtr<SWidget> TextWidget;
	const TSharedRef<SHorizontalBox> HorizontalWidget = SNew(SHorizontalBox).Visibility(EVisibility::Visible);

	static const FVector2D ImageSizeClosed(16.f, 16.f);
	static const FVector2D ImageSizeOpened(32.f, 32.f);

	// If none = update with current value else set fixed value once
	if (InName == NAME_None)
	{
		SAssignNew(ImageWidget, SImage)
			.ColorAndOpacity(FAppStyle::GetSlateColor("SelectionColor"))
			.DesiredSizeOverride(ImageSizeClosed)
			.Image(this, &FCEEditorEffectorTypeDetailCustomization::GetEasingImage, InName);

		SAssignNew(TextWidget, STextBlock)
			.Justification(ETextJustify::Center)
			.Text(this, &FCEEditorEffectorTypeDetailCustomization::GetEasingText, InName);
	}
	else
	{
		// We need to switch color on hover to avoid having the selected color equals the image color
		const TSharedPtr<SWidget> HoverWidget = HorizontalWidget;

		SAssignNew(ImageWidget, SImage)
			.ColorAndOpacity_Static(&FCEEditorEffectorTypeDetailCustomization::GetImageColorAndOpacity, HoverWidget)
			.DesiredSizeOverride(ImageSizeOpened)
			.Image(GetEasingImage(InName));

		SAssignNew(TextWidget, STextBlock)
			.Justification(ETextJustify::Center)
			.Text(GetEasingText(InName));
	}

	// Make inner widgets hit test invisible and only horizontal is hit testable

	HorizontalWidget->AddSlot()
		.AutoWidth()
		.Padding(2.f)
		[
			SNew(SScaleBox)
			.Visibility(EVisibility::HitTestInvisible)
			.Stretch(EStretch::UserSpecified)
			.UserSpecifiedScale(1.5)
			[
				ImageWidget.ToSharedRef()
			]
		];

	HorizontalWidget->AddSlot()
		.FillWidth(1.f)
		.Padding(8.f, 2.f)
		.HAlign(EHorizontalAlignment::HAlign_Fill)
		.VAlign(EVerticalAlignment::VAlign_Center)
		[
			SNew(SBox)
			.Visibility(EVisibility::HitTestInvisible)
			[
				TextWidget.ToSharedRef()
			]
		];

	return HorizontalWidget;
}

void FCEEditorEffectorTypeDetailCustomization::OnSelectionChanged(FName InSelection, ESelectInfo::Type InSelectInfo) const
{
	UEnum* EasingEnum = EasingEnumWeak.Get();

	if (EasingEnum && EasingPropertyHandle.IsValid())
	{
		const int32 Idx = EasingEnum->GetValueByNameString(InSelection.ToString());

		if (Idx != INDEX_NONE && EasingPropertyHandle->SetValue(static_cast<uint8>(Idx), EPropertyValueSetFlags::DefaultFlags) != FPropertyAccess::Success)
		{
			UE_LOGF(LogCEEditorEffectorTypeDetailCustomization, Warning, "EffectorTypeDetailCustomization : Failed to set property value %ls on selection", *InSelection.ToString())
		}
	}
}

FName FCEEditorEffectorTypeDetailCustomization::GetCurrentEasingName() const
{
	uint8 CurrentValue;

	if (EasingPropertyHandle.IsValid() && EasingPropertyHandle->GetValue(CurrentValue) == FPropertyAccess::Result::Success)
	{
		if (UEnum* EasingEnum = EasingEnumWeak.Get())
		{
			return FName(EasingEnum->GetNameStringByValue(CurrentValue));
		}
	}

	return NAME_None;
}

const FSlateBrush* FCEEditorEffectorTypeDetailCustomization::GetEasingImage(FName InName) const
{
	if (InName == NAME_None)
	{
		InName = GetCurrentEasingName();

		// multiple values
		if (InName == NAME_None)
		{
			return nullptr;
		}
	}

	return FCEEditorStyle::Get().GetBrush(FName(TEXT("EasingIcons.") + InName.ToString()));
}

FText FCEEditorEffectorTypeDetailCustomization::GetEasingText(FName InName) const
{
	if (InName == NAME_None)
	{
		InName = GetCurrentEasingName();

		// multiple values
		if (InName == NAME_None)
		{
			return LOCTEXT("MultipleValue", "Multiple values selected");
		}
	}

	if (UEnum* EasingEnum = EasingEnumWeak.Get())
	{
		const int32 EnumValue = EasingEnum->GetValueByNameString(InName.ToString());
		return EasingEnum->GetDisplayNameTextByValue(EnumValue);
	}

	return FText::GetEmpty();
}

FSlateColor FCEEditorEffectorTypeDetailCustomization::GetImageColorAndOpacity(const TSharedPtr<SWidget> InWidget)
{
	if (InWidget.IsValid())
	{
		return InWidget->IsHovered() ? FLinearColor::White : FAppStyle::GetSlateColor("SelectionColor");
	}

	return FLinearColor::White;
}

#undef LOCTEXT_NAMESPACE