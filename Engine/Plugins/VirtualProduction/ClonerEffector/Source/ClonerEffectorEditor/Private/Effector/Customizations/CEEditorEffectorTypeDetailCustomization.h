// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IDetailCustomization.h"
#include "Templates/SharedPointer.h"
#include "Types/SlateEnums.h"
#include "UObject/WeakObjectPtr.h"

class IPropertyHandle;
class SWidget;
class UEnum;
struct FLinearColor;
struct FSlateBrush;
struct FSlateColor;

/** Used to customize effector type properties in details panel */
class FCEEditorEffectorTypeDetailCustomization : public IDetailCustomization
{
public:
	static TSharedRef<IDetailCustomization> MakeInstance()
	{
		return MakeShared<FCEEditorEffectorTypeDetailCustomization>();
	}

	//~ Begin IDetailCustomization
	virtual void CustomizeDetails(IDetailLayoutBuilder& InDetailBuilder) override;
	//~ End IDetailCustomization

protected:
	static FSlateColor GetImageColorAndOpacity(const TSharedPtr<SWidget> InWidget);

	void PopulateEasingInfos();
	FName GetCurrentEasingName() const;
	const FSlateBrush* GetEasingImage(FName InName) const;
	FText GetEasingText(FName InName) const;

	TSharedRef<SWidget> OnGenerateEasingEntry(FName InName) const;
	void OnSelectionChanged(FName InSelection, ESelectInfo::Type InSelectInfo) const;

	TArray<FName> EasingNames;
	TWeakObjectPtr<UEnum> EasingEnumWeak;
	TSharedPtr<IPropertyHandle> EasingPropertyHandle;
};
