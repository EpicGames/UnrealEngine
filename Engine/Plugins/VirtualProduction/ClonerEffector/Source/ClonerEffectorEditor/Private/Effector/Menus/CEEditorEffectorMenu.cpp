// Copyright Epic Games, Inc. All Rights Reserved.

#include "Effector/Menus/CEEditorEffectorMenu.h"

#include "Subsystems/CEEffectorSubsystem.h"

#define LOCTEXT_NAMESPACE "CEEditorEffectorMenu"

FToolMenuSection* UE::EffectorEditor::Menu::FindOrAddEffectorSection(UToolMenu* InMenu)
{
	static const FName EffectorSectionName("ContextEffectorActions");

	FToolMenuSection* EffectorSection = InMenu->FindSection(EffectorSectionName);

	if (!EffectorSection)
	{
		EffectorSection = &InMenu->AddSection(EffectorSectionName
			, LOCTEXT("ContexEffectorActions", "Effector Actions")
			, FToolMenuInsert(NAME_None, EToolMenuInsertType::First));
	}

	return EffectorSection;
}

void UE::EffectorEditor::Menu::FillEnableEffectorSection(UToolMenu* InMenu, const FCEEditorEffectorMenuData& InMenuData)
{
	if (!InMenu
		|| InMenuData.Context.IsEmpty()
		|| !InMenuData.Context.ContainsAnyDisabledEffectors())
	{
		return;
	}

	FToolMenuSection& EnableEffectorSection = InMenu->FindOrAddSection(
		TEXT("EnableEffector")
		, LOCTEXT("EnableEffector.Label", "Enable effector")
	);

	constexpr bool bEnable = true;

	EnableEffectorSection.AddMenuEntry(
		TEXT("EnableEffectorComponent")
		, LOCTEXT("EnableEffectorComponent.Label", "Enable effector")
		, LOCTEXT("EnableEffectorComponent.Tooltip", "Enable selected effectors")
		, FSlateIcon()
		, FUIAction(
			FExecuteAction::CreateLambda(&ExecuteEnableEffectorAction, InMenuData, bEnable)
		)
	);

	EnableEffectorSection.AddMenuEntry(
		TEXT("EnableEffectorLevel")
		, LOCTEXT("EnableEffectorLevel.Label", "Enable level effector")
		, LOCTEXT("EnableEffectorLevel.Tooltip", "Enable selected level effectors")
		, FSlateIcon()
		, FUIAction(
			FExecuteAction::CreateLambda(&ExecuteEnableLevelEffectorAction, InMenuData, bEnable)
		)
	);
}

void UE::EffectorEditor::Menu::FillDisableEffectorSection(UToolMenu* InMenu, const FCEEditorEffectorMenuData& InMenuData)
{
	if (!InMenu
		|| InMenuData.Context.IsEmpty()
		|| !InMenuData.Context.ContainsAnyEnabledEffectors())
	{
		return;
	}

	FToolMenuSection& DisableEffectorSection = InMenu->FindOrAddSection(
		TEXT("DisableEffector")
		, LOCTEXT("DisableEffector.Label", "Disable effector")
	);

	constexpr bool bEnable = false;

	DisableEffectorSection.AddMenuEntry(
		TEXT("DisableEffectorComponent")
		, LOCTEXT("DisableEffectorComponent.Label", "Disable effector")
		, LOCTEXT("DisableEffectorComponent.Tooltip", "Disable selected effectors")
		, FSlateIcon()
		, FUIAction(
			FExecuteAction::CreateLambda(&ExecuteEnableEffectorAction, InMenuData, bEnable)
		)
	);

	DisableEffectorSection.AddMenuEntry(
		TEXT("DisableEffectorLevel")
		, LOCTEXT("DisableEffectorLevel.Label", "Disable level effector")
		, LOCTEXT("DisableEffectorLevel.Tooltip", "Disable selected level effectors")
		, FSlateIcon()
		, FUIAction(
			FExecuteAction::CreateLambda(&ExecuteEnableLevelEffectorAction, InMenuData, bEnable)
		)
	);
}

void UE::EffectorEditor::Menu::ExecuteEnableEffectorAction(const FCEEditorEffectorMenuData& InMenuData, bool bInEnable)
{
	UCEEffectorSubsystem* Subsystem = UCEEffectorSubsystem::Get();

	if (InMenuData.Context.IsEmpty() || !Subsystem)
	{
		return;
	}

	Subsystem->SetEffectorsEnabled(InMenuData.Context.GetComponents(), bInEnable, InMenuData.Options.ShouldTransact());
}

void UE::EffectorEditor::Menu::ExecuteEnableLevelEffectorAction(const FCEEditorEffectorMenuData& InMenuData, bool bInEnable)
{
	UCEEffectorSubsystem* Subsystem = UCEEffectorSubsystem::Get();
	const UWorld* World = InMenuData.Context.GetWorld();

	if (!IsValid(World) || !Subsystem)
	{
		return;
	}

	Subsystem->SetLevelEffectorsEnabled(World, bInEnable, InMenuData.Options.ShouldTransact());
}

#undef LOCTEXT_NAMESPACE
