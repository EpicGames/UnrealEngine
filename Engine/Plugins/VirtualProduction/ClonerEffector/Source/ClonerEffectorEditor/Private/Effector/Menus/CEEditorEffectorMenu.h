// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Effector/Menus/CEEditorEffectorMenuContext.h"
#include "Effector/Menus/CEEditorEffectorMenuOptions.h"
#include "Templates/SharedPointer.h"
#include "ToolMenus.h"

namespace UE::EffectorEditor::Menu
{
	/** Used internally to group menu data together */
	struct FCEEditorEffectorMenuData
	{
		FCEEditorEffectorMenuData(const FCEEditorEffectorMenuContext& InContext, const FCEEditorEffectorMenuOptions& InOptions)
			: Context(InContext)
			, Options(InOptions)
		{}

		const FCEEditorEffectorMenuContext Context;
		const FCEEditorEffectorMenuOptions Options;
	};

	/** Sections */

	FToolMenuSection* FindOrAddEffectorSection(UToolMenu* InMenu);

	void FillEnableEffectorSection(UToolMenu* InMenu, const FCEEditorEffectorMenuData& InMenuData);

	void FillDisableEffectorSection(UToolMenu* InMenu, const FCEEditorEffectorMenuData& InMenuData);

	/** Actions */

	void ExecuteEnableEffectorAction(const FCEEditorEffectorMenuData& InMenuData, bool bInEnable);

	void ExecuteEnableLevelEffectorAction(const FCEEditorEffectorMenuData& InMenuData, bool bInEnable);
}
