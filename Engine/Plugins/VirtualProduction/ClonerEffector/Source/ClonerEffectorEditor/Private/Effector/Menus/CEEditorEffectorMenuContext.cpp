// Copyright Epic Games, Inc. All Rights Reserved.

#include "Effector/Menus/CEEditorEffectorMenuContext.h"

#include "Effector/CEEffectorComponent.h"
#include "GameFramework/Actor.h"

FCEEditorEffectorMenuContext::FCEEditorEffectorMenuContext(const TSet<UObject*>& InObjects)
{
	for (UObject* Object : InObjects)
	{
		if (!IsValid(Object))
		{
			continue;
		}

		if (const AActor* Actor = Cast<AActor>(Object))
		{
			TArray<UCEEffectorComponent*> EffectorComponents;
			Actor->GetComponents(EffectorComponents, /** IncludeChildren */false);

			for (UCEEffectorComponent* Component : EffectorComponents)
			{
				if (IsValid(Component))
				{
					ContextComponentsKey.Add(Component);
				}
			}
		}
		else if (UCEEffectorComponent* Component = Cast<UCEEffectorComponent>(Object))
		{
			ContextComponentsKey.Add(Component);
		}
	}
}

TSet<UCEEffectorComponent*> FCEEditorEffectorMenuContext::GetComponents() const
{
	TSet<UCEEffectorComponent*> Components;
	Components.Reserve(ContextComponentsKey.Num());

	Algo::TransformIf(
		ContextComponentsKey
		, Components
		, [](const TObjectKey<UCEEffectorComponent>& InComponentKey)
		{
			return IsValid(InComponentKey.ResolveObjectPtr());
		}
		, [](const TObjectKey<UCEEffectorComponent>& InComponentKey)
		{
			return InComponentKey.ResolveObjectPtr();
		}
	);

	return Components;
}

TSet<UCEEffectorComponent*> FCEEditorEffectorMenuContext::GetDisabledEffectors() const
{
	return GetStateEffectors(/** IsEnabled */ false);
}

TSet<UCEEffectorComponent*> FCEEditorEffectorMenuContext::GetEnabledEffectors() const
{
	return GetStateEffectors(/** IsEnabled */ true);
}

UWorld* FCEEditorEffectorMenuContext::GetWorld() const
{
	for (const TObjectKey<UCEEffectorComponent>& ComponentKey : ContextComponentsKey)
	{
		if (const UCEEffectorComponent* Component = ComponentKey.ResolveObjectPtr())
		{
			return Component->GetWorld();
		}
	}

	return nullptr;
}

bool FCEEditorEffectorMenuContext::IsEmpty() const
{
	return ContextComponentsKey.IsEmpty();
}

bool FCEEditorEffectorMenuContext::ContainsAnyComponent() const
{
	return !ContextComponentsKey.IsEmpty();
}

bool FCEEditorEffectorMenuContext::ContainsAnyDisabledEffectors() const
{
	return ContainsEffectorState(/** IsEnabled */ false);
}

bool FCEEditorEffectorMenuContext::ContainsAnyEnabledEffectors() const
{
	return ContainsEffectorState(/** IsEnabled */ true);
}

bool FCEEditorEffectorMenuContext::ContainsEffectorState(bool bInState) const
{
	for (const TObjectKey<UCEEffectorComponent>& ComponentKey : ContextComponentsKey)
	{
		if (const UCEEffectorComponent* Component = ComponentKey.ResolveObjectPtr())
		{
			if (Component->GetEnabled() == bInState)
			{
				return true;
			}
		}
	}

	return false;
}

TSet<UCEEffectorComponent*> FCEEditorEffectorMenuContext::GetStateEffectors(bool bInState) const
{
	TSet<UCEEffectorComponent*> Effectors;
	Effectors.Reserve(ContextComponentsKey.Num());

	for (const TObjectKey<UCEEffectorComponent>& ComponentKey : ContextComponentsKey)
	{
		if (UCEEffectorComponent* Component = ComponentKey.ResolveObjectPtr())
		{
			if (Component->GetEnabled() == bInState)
			{
				Effectors.Add(Component);
			}
		}
	}

	return Effectors;
}