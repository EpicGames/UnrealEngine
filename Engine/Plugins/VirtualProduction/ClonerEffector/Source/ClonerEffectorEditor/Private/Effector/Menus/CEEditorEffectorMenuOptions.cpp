// Copyright Epic Games, Inc. All Rights Reserved.

#include "Effector/Menus/CEEditorEffectorMenuOptions.h"

#include "Misc/EnumClassFlags.h"

FCEEditorEffectorMenuOptions::FCEEditorEffectorMenuOptions(const TSet<ECEEditorEffectorMenuType>& InMenus)
{
	for (const ECEEditorEffectorMenuType& Menu : InMenus)
	{
		MenuTypes |= static_cast<uint8>(Menu);
	}
}

FCEEditorEffectorMenuOptions& FCEEditorEffectorMenuOptions::CreateSubMenu(bool bInCreateSubMenu)
{
	bCreateSubMenu = bInCreateSubMenu;
	return *this;
}

FCEEditorEffectorMenuOptions& FCEEditorEffectorMenuOptions::UseTransact(bool bInUseTransact)
{
	bUseTransact = bInUseTransact;
	return *this;
}

bool FCEEditorEffectorMenuOptions::IsMenuType(ECEEditorEffectorMenuType InMenuType) const
{
	return EnumHasAnyFlags(static_cast<ECEEditorEffectorMenuType>(MenuTypes), InMenuType);
}