// Copyright Epic Games, Inc. All Rights Reserved.

#include "Styles/CEEditorStyle.h"

#include "CEClonerEffectorShared.h"
#include "Interfaces/IPluginManager.h"
#include "Misc/Paths.h"
#include "Styling/AppStyle.h"
#include "Styling/SlateStyleMacros.h"
#include "Styling/StyleColors.h"
#include "Styling/SlateTypes.h"
#include "Styling/SlateStyleRegistry.h"

FCEEditorStyle::FCEEditorStyle()
	: FSlateStyleSet(UE_MODULE_NAME)
{
	const FVector2f Icon32x32(32.0f, 32.0f);

	const TSharedPtr<IPlugin> Plugin = IPluginManager::Get().FindPlugin(UE_PLUGIN_NAME);

	check(Plugin.IsValid());

	ContentRootDir = FPaths::Combine(Plugin->GetBaseDir(), TEXT("Resources"));

	Set(TEXT("ClassIcon.CEClonerActor"),	new IMAGE_BRUSH_SVG("cloner",	Icon32x32));
	Set(TEXT("ClassIcon.CEEffectorActor"),  new IMAGE_BRUSH_SVG("effector", Icon32x32));

	// Easing
	if (const UEnum* EasingEnum = StaticEnum<ECEClonerEasing>())
	{
		for (int32 Idx = 0; Idx < EasingEnum->GetMaxEnumValue(); Idx++)
		{
			FText ValueText;
			EasingEnum->GetDisplayValueAsText(static_cast<ECEClonerEasing>(Idx), ValueText);
			const FString EasingString = ValueText.ToString().Replace(TEXT(" "), TEXT(""));
			const FName EasingName(TEXT("EasingIcons.") + EasingString);

			Set(EasingName, new IMAGE_BRUSH_SVG("EasingIcons/" + EasingString, Icon32x32));
		}
	}

	// Combobox
	FComboBoxStyle ComboBoxStyle = FAppStyle::Get().GetWidgetStyle<FComboBoxStyle>(TEXT("ComboBox"));
	ComboBoxStyle.SetMenuRowPadding(FMargin(0));
	Set("ComboBox", ComboBoxStyle);

	FSlateStyleRegistry::RegisterSlateStyle(*this);
}

FCEEditorStyle::~FCEEditorStyle()
{
	FSlateStyleRegistry::UnRegisterSlateStyle(*this);
}