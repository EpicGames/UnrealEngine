// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Styling/SlateStyle.h"

class FCEEditorStyle : public FSlateStyleSet
{
public:
	FCEEditorStyle();

	virtual ~FCEEditorStyle() override;

	static FCEEditorStyle& Get()
	{
		static FCEEditorStyle StyleSet;
		return StyleSet;
	}
};