// Copyright Epic Games, Inc. All Rights Reserved.

#include "Subsystems/CEEditorClonerSubsystem.h"

#include "Cloner/Menus/CEEditorClonerMenu.h"
#include "Editor.h"

#define LOCTEXT_NAMESPACE "CEEditorClonerSubsystem"

UCEEditorClonerSubsystem::UCEEditorClonerSubsystem()
	: UEditorSubsystem()
{
}

UCEEditorClonerSubsystem* UCEEditorClonerSubsystem::Get()
{
	if (GEditor)
	{
		return GEditor->GetEditorSubsystem<UCEEditorClonerSubsystem>();
	}

	return nullptr;
}

void UCEEditorClonerSubsystem::FillClonerMenu(UToolMenu* InMenu, const FCEEditorClonerMenuContext& InContext, const FCEEditorClonerMenuOptions& InOptions)
{
	if (!IsValid(InMenu) || InContext.IsEmpty())
	{
		return;
	}

	using namespace UE::ClonerEditor::Menu;

	FToolMenuSection* ClonerSection = nullptr;
	if (InOptions.ShouldCreateSubMenu())
	{
		ClonerSection = FindOrAddClonerSection(InMenu);
	}

	FCEEditorClonerMenuData MenuData(InContext, InOptions);

	if (InOptions.IsMenuType(ECEEditorClonerMenuType::Enable) && InContext.ContainsAnyDisabledCloner())
	{
		if (InOptions.ShouldCreateSubMenu())
		{
			check(ClonerSection)

			ClonerSection->AddSubMenu(
				TEXT("EnableClonerMenu"),
				LOCTEXT("EnableClonerMenu.Label", "Enable cloners"),
				LOCTEXT("EnableClonerMenu.Tooltip", "Enable selected cloners"),
				FNewToolMenuDelegate::CreateLambda(&FillEnableClonerSection, MenuData));
		}
		else
		{
			FillEnableClonerSection(InMenu, MenuData);
		}
	}

	if (InOptions.IsMenuType(ECEEditorClonerMenuType::Disable) && InContext.ContainsAnyEnabledCloner())
	{
		if (InOptions.ShouldCreateSubMenu())
		{
			check(ClonerSection)

			ClonerSection->AddSubMenu(
				TEXT("DisableClonerMenu"),
				LOCTEXT("DisableClonerMenu.Label", "Disable cloners"),
				LOCTEXT("DisableClonerMenu.Tooltip", "Disable selected cloners"),
				FNewToolMenuDelegate::CreateLambda(&FillDisableClonerSection, MenuData));
		}
		else
		{
			FillDisableClonerSection(InMenu, MenuData);
		}
	}

	if (InOptions.IsMenuType(ECEEditorClonerMenuType::CreateEffector) && InContext.ContainsAnyCloner())
	{
		if (InOptions.ShouldCreateSubMenu())
		{
			check(ClonerSection)

			ClonerSection->AddSubMenu(
				TEXT("CreateEffectorMenu"),
				LOCTEXT("CreateEffectorMenu.Label", "Create effectors"),
				LOCTEXT("CreateEffectorMenu.Tooltip", "Create linked effectors for selected cloners"),
				FNewToolMenuDelegate::CreateLambda(&FillCreateClonerEffectorSection, MenuData));
		}
		else
		{
			FillCreateClonerEffectorSection(InMenu, MenuData);
		}
	}

	if (InOptions.IsMenuType(ECEEditorClonerMenuType::Convert) && InContext.ContainsAnyEnabledCloner())
	{
		if (InOptions.ShouldCreateSubMenu())
		{
			check(ClonerSection)

			ClonerSection->AddSubMenu(
				TEXT("ConvertClonerMenu"),
				LOCTEXT("ConvertClonerMenu.Label", "Convert cloners"),
				LOCTEXT("ConvertClonerMenu.Tooltip", "Convert selected cloners"),
				FNewToolMenuDelegate::CreateLambda(&FillConvertClonerSection, MenuData));
		}
		else
		{
			FillConvertClonerSection(InMenu, MenuData);
		}
	}

	if (InOptions.IsMenuType(ECEEditorClonerMenuType::CreateCloner) && !InContext.ContainsAnyCloner() && InContext.ContainsAnyActor())
	{
		if (InOptions.ShouldCreateSubMenu())
		{
			check(ClonerSection)

			ClonerSection->AddSubMenu(
				TEXT("CreateClonerMenu"),
				LOCTEXT("CreateClonerMenu.Label", "Create cloner"),
				LOCTEXT("CreateClonerMenu.Tooltip", "Create cloner with selected actors"),
				FNewToolMenuDelegate::CreateLambda(&FillCreateClonerSection, MenuData));
		}
		else
		{
			FillCreateClonerSection(InMenu, MenuData);
		}
	}
}

#undef LOCTEXT_NAMESPACE
