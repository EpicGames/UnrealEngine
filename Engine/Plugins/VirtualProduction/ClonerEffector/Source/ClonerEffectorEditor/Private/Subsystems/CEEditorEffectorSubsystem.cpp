// Copyright Epic Games, Inc. All Rights Reserved.

#include "Subsystems/CEEditorEffectorSubsystem.h"

#include "Editor.h"
#include "Effector/Menus/CEEditorEffectorMenu.h"

#define LOCTEXT_NAMESPACE "CEEditorEffectorSubsystem"

UCEEditorEffectorSubsystem::UCEEditorEffectorSubsystem()
	: UEditorSubsystem()
{
}

UCEEditorEffectorSubsystem* UCEEditorEffectorSubsystem::Get()
{
	if (GEditor)
	{
		return GEditor->GetEditorSubsystem<UCEEditorEffectorSubsystem>();
	}

	return nullptr;
}

void UCEEditorEffectorSubsystem::FillEffectorMenu(UToolMenu* InMenu, const FCEEditorEffectorMenuContext& InContext, const FCEEditorEffectorMenuOptions& InOptions)
{
	if (!IsValid(InMenu) || InContext.IsEmpty())
	{
		return;
	}

	using namespace UE::EffectorEditor::Menu;

	FToolMenuSection* EffectorSection = nullptr;
	if (InOptions.ShouldCreateSubMenu())
	{
		EffectorSection = FindOrAddEffectorSection(InMenu);
	}

	FCEEditorEffectorMenuData MenuData(InContext, InOptions);

	if (InOptions.IsMenuType(ECEEditorEffectorMenuType::Enable) && InContext.ContainsAnyDisabledEffectors())
	{
		if (InOptions.ShouldCreateSubMenu())
		{
			check(EffectorSection)

			EffectorSection->AddSubMenu(
				TEXT("EnableEffectorMenu"),
				LOCTEXT("EnableEffectorMenu.Label", "Enable effectors"),
				LOCTEXT("EnableEffectorMenu.Tooltip", "Enable selected effectors"),
				FNewToolMenuDelegate::CreateLambda(&FillEnableEffectorSection, MenuData));
		}
		else
		{
			FillEnableEffectorSection(InMenu, MenuData);
		}
	}

	if (InOptions.IsMenuType(ECEEditorEffectorMenuType::Disable) && InContext.ContainsAnyEnabledEffectors())
	{
		if (InOptions.ShouldCreateSubMenu())
		{
			check(EffectorSection)

			EffectorSection->AddSubMenu(
				TEXT("DisableEffectorMenu"),
				LOCTEXT("DisableEffectorMenu.Label", "Disable effectors"),
				LOCTEXT("DisableEffectorMenu.Tooltip", "Disable selected effectors"),
				FNewToolMenuDelegate::CreateLambda(&FillDisableEffectorSection, MenuData));
		}
		else
		{
			FillDisableEffectorSection(InMenu, MenuData);
		}
	}
}

#undef LOCTEXT_NAMESPACE
