// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ActorFactories/ActorFactory.h"
#include "CEClonerActorFactory.generated.h"

UCLASS(MinimalAPI)
class UCEClonerActorFactory : public UActorFactory
{
	GENERATED_BODY()

public:
	UCEClonerActorFactory();

	CLONEREFFECTOREDITOR_API void SetClonerLayout(FName InLayoutName);

protected:
	//~ Begin UActorFactory
	virtual void PostSpawnActor(UObject* InAsset, AActor* InNewActor) override;
	virtual void PostPlaceAsset(TArrayView<const FTypedElementHandle> InHandle, const FAssetPlacementInfo& InPlacementInfo, const FPlacementOptions& InPlacementOptions) override;
	//~ End UActorFactory

	FName ClonerLayoutName = NAME_None;
};
