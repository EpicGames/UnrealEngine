// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ActorPickerMode.h"
#include "IDetailCustomNodeBuilder.h"
#include "Templates/SharedPointer.h"

class IPropertyHandle;
class IPropertyUtilities;
class SComboButton;
class SWidget;

/** Used to customize an actor picker to filter items based on a delegate */
class CLONEREFFECTOREDITOR_API FCEEditorClonerCustomActorPickerNodeBuilder : public IDetailCustomNodeBuilder, public TSharedFromThis<FCEEditorClonerCustomActorPickerNodeBuilder> 
{
public:
	explicit FCEEditorClonerCustomActorPickerNodeBuilder(const TSharedRef<IPropertyHandle>& InPropertyHandle, FOnShouldFilterActor InActorFilterDelegate)
		: PropertyHandle(InPropertyHandle)
		, ActorFilterDelegate(InActorFilterDelegate)
	{}

	//~ Begin IDetailCustomNodeBuilder
	virtual FName GetName() const override;
	virtual void GenerateHeaderRowContent(FDetailWidgetRow& InNodeRow) override;
	virtual void GenerateChildContent(IDetailChildrenBuilder& InChildrenBuilder) override;
	virtual TSharedPtr<IPropertyHandle> GetPropertyHandle() const override;
	//~ End IDetailCustomNodeBuilder

private:
	static TSharedPtr<SWidget> FindWidgetType(TSharedPtr<SWidget> InSearchWidget, FName InTypeName, bool bInReverseSearch);

	FText GetPickerLabelText() const;
	FText GetSelectTooltipText() const;
	FText GetUseSelectTooltipText() const;
	TSharedRef<SWidget> GetActorPickerWidget() const;

	void OnActorSelected(AActor* InSelection) const;
	void OnSelectActor() const;
	void OnUseSelectedActor() const;

	AActor* GetPropertyActor() const;
	AActor* GetSelectedActor() const;

	TSharedRef<IPropertyHandle> PropertyHandle;
	TSharedPtr<SComboButton> ComboButton;
	FOnShouldFilterActor ActorFilterDelegate;
};
