// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Set.h"
#include "UObject/Object.h"
#include "UObject/ObjectKey.h"
#include "UObject/WeakObjectPtrTemplates.h"

class AActor;
class UCEClonerComponent;
class UObject;
class UWorld;

/** Menu objects to apply actions on */
struct CLONEREFFECTOREDITOR_API FCEEditorClonerMenuContext
{
	FCEEditorClonerMenuContext() {}
	explicit FCEEditorClonerMenuContext(const TSet<UObject*>& InObjects);

	TSet<AActor*> GetActors() const;
	TSet<UCEClonerComponent*> GetCloners() const;
	TSet<UCEClonerComponent*> GetDisabledCloners() const;
	TSet<UCEClonerComponent*> GetEnabledCloners() const;
	UWorld* GetWorld() const;

	bool IsEmpty() const;
	bool ContainsAnyActor() const;
	bool ContainsAnyCloner() const;
	bool ContainsAnyDisabledCloner() const;
	bool ContainsAnyEnabledCloner() const;

protected:
	bool ContainsClonerState(bool bInState) const;
	TSet<UCEClonerComponent*> GetStateCloners(bool bInState) const;

	TSet<TObjectKey<AActor>> ContextActorsKey;
	TSet<TObjectKey<UCEClonerComponent>> ContextComponentsKey;
};