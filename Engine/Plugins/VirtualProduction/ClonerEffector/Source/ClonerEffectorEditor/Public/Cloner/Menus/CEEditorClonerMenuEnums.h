// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "HAL/Platform.h"

/** Menu type available */
enum class ECEEditorClonerMenuType : uint8
{
	/** Enable the cloner */
	Enable = 1 << 0,
	/** Disable the cloner */
	Disable = 1 << 1,
	/** Creates a cloner linked effector */
	CreateEffector = 1 << 2,
	/** Converts cloner to specific mesh */
	Convert = 1 << 3,
	/** Creates a cloner and attaches actors below it */
	CreateCloner = 1 << 4
};