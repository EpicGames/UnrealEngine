// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CEEditorClonerMenuEnums.h"
#include "Containers/Set.h"

/** Menu options to customize */
struct CLONEREFFECTOREDITOR_API FCEEditorClonerMenuOptions
{
	FCEEditorClonerMenuOptions() {}
	explicit FCEEditorClonerMenuOptions(const TSet<ECEEditorClonerMenuType>& InMenus);
	explicit FCEEditorClonerMenuOptions(const uint8 InMenus)
		: MenuTypes(InMenus)
	{}

	FCEEditorClonerMenuOptions& CreateSubMenu(bool bInCreateSubMenu);
	FCEEditorClonerMenuOptions& UseTransact(bool bInUseTransact);

	bool IsMenuType(ECEEditorClonerMenuType InMenuType) const;

	bool ShouldTransact() const
	{
		return bUseTransact;
	}

	bool ShouldCreateSubMenu() const
	{
		return bCreateSubMenu;
	}

protected:
	/** What type of menu should be generated */
	uint8 MenuTypes = 0;

	/** Create a transaction for actions performed using the menu */
	bool bUseTransact = true;

	/** Creates the section inside a submenu */
	bool bCreateSubMenu = false;
};