// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ActorFactories/ActorFactory.h"
#include "CEEffectorActorFactory.generated.h"

UCLASS(MinimalAPI)
class UCEEffectorActorFactory : public UActorFactory
{
	GENERATED_BODY()

public:
	UCEEffectorActorFactory();

	CLONEREFFECTOREDITOR_API void SetEffectorTypeName(FName InEffectorTypeName);

protected:
	//~ Begin UActorFactory
	virtual void PostSpawnActor(UObject* InAsset, AActor* InNewActor) override;
	virtual void PostPlaceAsset(TArrayView<const FTypedElementHandle> InHandle, const FAssetPlacementInfo& InPlacementInfo, const FPlacementOptions& InPlacementOptions) override;
	//~ End UActorFactory

	FName EffectorTypeName;
};
