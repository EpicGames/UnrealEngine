// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Set.h"
#include "UObject/Object.h"
#include "UObject/ObjectKey.h"
#include "UObject/WeakObjectPtrTemplates.h"

class UCEEffectorComponent;
class UObject;
class UWorld;

/** Menu objects to apply actions on */
struct CLONEREFFECTOREDITOR_API FCEEditorEffectorMenuContext
{
	FCEEditorEffectorMenuContext() {}
	explicit FCEEditorEffectorMenuContext(const TSet<UObject*>& InObjects);

	TSet<UCEEffectorComponent*> GetComponents() const;
	TSet<UCEEffectorComponent*> GetDisabledEffectors() const;
	TSet<UCEEffectorComponent*> GetEnabledEffectors() const;
	UWorld* GetWorld() const;

	bool IsEmpty() const;
	bool ContainsAnyComponent() const;
	bool ContainsAnyDisabledEffectors() const;
	bool ContainsAnyEnabledEffectors() const;

protected:
	bool ContainsEffectorState(bool bInState) const;
	TSet<UCEEffectorComponent*> GetStateEffectors(bool bInState) const;

	TSet<TObjectKey<UCEEffectorComponent>> ContextComponentsKey;
};