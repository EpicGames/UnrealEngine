// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "HAL/Platform.h"

/** Menu type available */
enum class ECEEditorEffectorMenuType : uint8
{
	Enable = 1 << 0,
	Disable = 1 << 1
};