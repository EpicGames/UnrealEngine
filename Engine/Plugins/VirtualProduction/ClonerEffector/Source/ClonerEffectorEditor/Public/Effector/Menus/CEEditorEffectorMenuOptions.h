// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CEEditorEffectorMenuEnums.h"
#include "Containers/Set.h"

/** Menu options to customize */
struct CLONEREFFECTOREDITOR_API FCEEditorEffectorMenuOptions
{
	FCEEditorEffectorMenuOptions() {}
	explicit FCEEditorEffectorMenuOptions(const TSet<ECEEditorEffectorMenuType>& InMenus);
	explicit FCEEditorEffectorMenuOptions(const uint8 InMenus)
		: MenuTypes(InMenus)
	{}

	FCEEditorEffectorMenuOptions& CreateSubMenu(bool bInCreateSubMenu);
	FCEEditorEffectorMenuOptions& UseTransact(bool bInUseTransact);

	bool IsMenuType(ECEEditorEffectorMenuType InMenuType) const;

	bool ShouldTransact() const
	{
		return bUseTransact;
	}

	bool ShouldCreateSubMenu() const
	{
		return bCreateSubMenu;
	}

protected:
	/** What type of menu should be generated */
	uint8 MenuTypes = 0;

	/** Create a transaction for actions performed using the menu */
	bool bUseTransact = true;

	/** Creates the section inside a submenu */
	bool bCreateSubMenu = false;
};