// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "EditorSubsystem.h"
#include "Templates/SharedPointer.h"
#include "UObject/WeakObjectPtr.h"
#include "CEEditorClonerSubsystem.generated.h"

struct FCEEditorClonerMenuContext;
struct FCEEditorClonerMenuOptions;
class UToolMenu;

/** Singleton class that handles editor operations for cloners */
UCLASS(MinimalAPI)
class UCEEditorClonerSubsystem : public UEditorSubsystem
{
	GENERATED_BODY()

public:
	UCEEditorClonerSubsystem();

	/** Get this subsystem instance */
	CLONEREFFECTOREDITOR_API static UCEEditorClonerSubsystem* Get();

	/** Fills a menu based on context objects and menu options */
	CLONEREFFECTOREDITOR_API void FillClonerMenu(UToolMenu* InMenu, const FCEEditorClonerMenuContext& InContext, const FCEEditorClonerMenuOptions& InOptions);
};