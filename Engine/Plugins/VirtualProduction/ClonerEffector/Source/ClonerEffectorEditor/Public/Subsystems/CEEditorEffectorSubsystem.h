// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "EditorSubsystem.h"
#include "Templates/SharedPointer.h"
#include "UObject/WeakObjectPtr.h"
#include "CEEditorEffectorSubsystem.generated.h"

struct FCEEditorEffectorMenuContext;
struct FCEEditorEffectorMenuOptions;
class UToolMenu;

/** Singleton class that handles editor operations for effectors */
UCLASS(MinimalAPI)
class UCEEditorEffectorSubsystem : public UEditorSubsystem
{
	GENERATED_BODY()

public:
	UCEEditorEffectorSubsystem();

	/** Get this subsystem instance */
	CLONEREFFECTOREDITOR_API static UCEEditorEffectorSubsystem* Get();

	/** Fills a menu based on context objects and menu options */
	CLONEREFFECTOREDITOR_API void FillEffectorMenu(UToolMenu* InMenu, const FCEEditorEffectorMenuContext& InContext, const FCEEditorEffectorMenuOptions& InOptions);
};