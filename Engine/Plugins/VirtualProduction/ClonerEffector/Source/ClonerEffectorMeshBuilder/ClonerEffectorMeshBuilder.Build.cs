// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class ClonerEffectorMeshBuilder : ModuleRules
{
    public ClonerEffectorMeshBuilder(ReadOnlyTargetRules Target) : base(Target)
    {
        PCHUsage = ModuleRules.PCHUsageMode.UseExplicitOrSharedPCHs;

        PublicDependencyModuleNames.AddRange(
            new string[]
            {
                "Core"
            }
        );

        PrivateDependencyModuleNames.AddRange(
            new string[]
            {
                "CoreUObject",
                "DynamicMesh",
                "Engine",
                "GeometryCore",
                "GeometryFramework",
                "GeometryScriptingCore",
                "MeshConversion",
				"MeshDescription",
				"ModelingComponents",
                "Niagara",
                "NiagaraCore",
                "ProceduralMeshComponent",
                "StaticMeshDescription"
            }
        );
        
        ShortName = "CEMB";
    }
}