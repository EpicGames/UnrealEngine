// Copyright Epic Games, Inc. All Rights Reserved.

#include "CEMeshBuilder.h"

#include "AssetUtils/StaticMeshMaterialUtil.h"
#include "Components/BrushComponent.h"
#include "Components/DynamicMeshComponent.h"
#include "Components/InstancedStaticMeshComponent.h"
#include "Components/SkeletalMeshComponent.h"
#include "Components/SplineMeshComponent.h"
#include "DynamicMesh/MeshNormals.h"
#include "DynamicMesh/MeshTransforms.h"
#include "DynamicMesh/Operations/MergeCoincidentMeshEdges.h"
#include "DynamicMeshEditor.h"
#include "DynamicMeshToMeshDescription.h"
#include "Engine/SkeletalMesh.h"
#include "Engine/StaticMesh.h"
#include "GeometryScript/MeshAssetFunctions.h"
#include "GeometryScript/MeshBasicEditFunctions.h"
#include "GeometryScript/SceneUtilityFunctions.h"
#include "Materials/MaterialInterface.h"
#include "Model.h"
#include "NiagaraComponent.h"
#include "NiagaraEmitter.h"
#include "NiagaraMeshRendererProperties.h"
#include "NiagaraSimCacheFunctionLibrary.h"
#include "NiagaraSystem.h"
#include "ProceduralMeshComponent.h"
#include "Rendering/SkeletalMeshRenderData.h"
#include "StaticMeshDescription.h"
#include "StaticMeshOperations.h"
#include "UDynamicMesh.h"

const FCEMeshBuilder::FCEMeshBuilderParams FCEMeshBuilder::DefaultBuildParams;
const FCEMeshBuilder::FCEMeshBuilderAppendParams FCEMeshBuilder::DefaultAppendParams;

bool FCEMeshBuilder::HasAnyGeometry(UActorComponent* InComponent)
{
	if (!IsComponentSupported(InComponent))
	{
		return false;
	}

	if (UDynamicMeshComponent* DynamicMeshComponent = Cast<UDynamicMeshComponent>(InComponent))
	{
		return DynamicMeshComponent->GetDynamicMesh()
			&& DynamicMeshComponent->GetDynamicMesh()->GetTriangleCount() > 0;
	}

	if (const USkeletalMeshComponent* SkeletalMeshComponent = Cast<USkeletalMeshComponent>(InComponent))
	{
		if (SkeletalMeshComponent->GetSkeletalMeshAsset())
		{
			if (FSkeletalMeshRenderData* RenderData = SkeletalMeshComponent->GetSkeletalMeshRenderData())
			{
				return !RenderData->LODRenderData.IsEmpty() && RenderData->LODRenderData[0].GetNumVertices() > 0;
			}
		}

		return false;
	}

	if (const UBrushComponent* BrushComponent = Cast<UBrushComponent>(InComponent))
	{
		return BrushComponent->Brush
			&& !BrushComponent->Brush->Verts.IsEmpty();
	}

	if (UProceduralMeshComponent* ProceduralMeshComponent = Cast<UProceduralMeshComponent>(InComponent))
	{
		for (int32 SectionIndex = 0; SectionIndex < ProceduralMeshComponent->GetNumSections(); SectionIndex++)
		{
			if (const FProcMeshSection* Section = ProceduralMeshComponent->GetProcMeshSection(SectionIndex))
			{
				if (Section->bSectionVisible && !Section->ProcVertexBuffer.IsEmpty() && !Section->ProcIndexBuffer.IsEmpty())
				{
					return true;
				}
			}
		}

		return false;
	}

	if (const UInstancedStaticMeshComponent* ISMComponent = Cast<UInstancedStaticMeshComponent>(InComponent))
	{
		if (ISMComponent->GetStaticMesh() && ISMComponent->GetStaticMesh()->GetNumTriangles(/**LOD*/0) > 0)
		{
			return ISMComponent->GetNumInstances() > 0;
		}

		return false;
	}

	if (const USplineMeshComponent* SplineMeshComponent = Cast<USplineMeshComponent>(InComponent))
	{
		return SplineMeshComponent->GetStaticMesh() && SplineMeshComponent->GetStaticMesh()->GetNumTriangles(/**LOD*/0) > 0;
	}

	if (const UStaticMeshComponent* StaticMeshComponent = Cast<UStaticMeshComponent>(InComponent))
	{
		return StaticMeshComponent->GetStaticMesh() && StaticMeshComponent->GetStaticMesh()->GetNumTriangles(/**LOD*/0) > 0;
	}

	if (const UNiagaraComponent* NiagaraComponent = Cast<UNiagaraComponent>(InComponent))
	{
		if (UNiagaraSystem* System = NiagaraComponent->GetAsset())
		{
			if (System->GetActiveInstancesCount() > 0)
			{
				for (const FNiagaraEmitterHandle& EmitterHandle : System->GetEmitterHandles())
				{
					if (const FVersionedNiagaraEmitterData* EmitterData = EmitterHandle.GetEmitterData())
					{
						for (UNiagaraRendererProperties* EmitterRenderer : EmitterData->GetRenderers())
						{
							if (UNiagaraMeshRendererProperties* MeshRenderer = Cast<UNiagaraMeshRendererProperties>(EmitterRenderer))
							{
								for (const FNiagaraMeshRendererMeshProperties& MeshProperty : MeshRenderer->Meshes)
								{
									if (MeshProperty.Mesh && MeshProperty.Mesh->GetNumTriangles(/**LOD*/0) > 0)
									{
										return true;
									}
								}
							}
						}
					}
				}
			}
		}

		return false;
	}

	return false;
}

FCEMeshBuilder::FCEMeshBuilder()
{
	OutputDynamicMesh = NewObject<UDynamicMesh>();
}

TArray<uint32> FCEMeshBuilder::GetMeshIndexes() const
{
	TArray<uint32> MeshIndexes;
	Meshes.GenerateKeyArray(MeshIndexes);
	return MeshIndexes;
}

bool FCEMeshBuilder::IsActorSupported(const AActor* InActor)
{
	return InActor && InActor->FindComponentByClass<UPrimitiveComponent>();
}

bool FCEMeshBuilder::IsComponentSupported(const UActorComponent* InComponent)
{
	return InComponent && InComponent->IsA<UPrimitiveComponent>();
}

void FCEMeshBuilder::Reset()
{
	ClearOutputMesh();
	Meshes.Empty();
	MeshInstances.Empty();
}

TArray<UPrimitiveComponent*> FCEMeshBuilder::AppendActor(const AActor* InActor, const FTransform& InSourceTransform, const FCEMeshBuilderAppendParams& InParams)
{
	TArray<UPrimitiveComponent*> AppendPrimitiveComponents;

	if (!IsValid(InActor) || InParams.ComponentTypes == ECEMeshBuilderComponentType::None)
	{
		return AppendPrimitiveComponents;
	}

	TArray<UPrimitiveComponent*> PrimitiveComponents;
	InActor->GetComponents(PrimitiveComponents, /** IncludeChildrenActors */ false);

	AppendPrimitiveComponents.Reserve(PrimitiveComponents.Num());

	for (UPrimitiveComponent* PrimitiveComponent : PrimitiveComponents)
	{
		if (!IsValid(PrimitiveComponent)
#if WITH_EDITOR
			|| PrimitiveComponent->IsVisualizationComponent()
#endif
			|| InParams.ExcludeComponents.Contains(PrimitiveComponent)
			)
		{
			continue;
		}

		if (UDynamicMeshComponent* DynamicMeshComponent = Cast<UDynamicMeshComponent>(PrimitiveComponent))
		{
			if (!EnumHasAnyFlags(InParams.ComponentTypes, ECEMeshBuilderComponentType::DynamicMeshComponent))
			{
				continue;
			}
			
			if (AppendComponent(DynamicMeshComponent, InSourceTransform))
			{
				AppendPrimitiveComponents.Add(PrimitiveComponent);
			}
		}
		else if (USkeletalMeshComponent* SkeletalMeshComponent = Cast<USkeletalMeshComponent>(PrimitiveComponent))
		{
			if (!EnumHasAnyFlags(InParams.ComponentTypes, ECEMeshBuilderComponentType::SkeletalMeshComponent))
			{
				continue;
			}
			
			if (AppendComponent(SkeletalMeshComponent, InSourceTransform))
			{
				AppendPrimitiveComponents.Add(PrimitiveComponent);
			}
		}
		else if (UBrushComponent* BrushComponent = Cast<UBrushComponent>(PrimitiveComponent))
		{
			if (!EnumHasAnyFlags(InParams.ComponentTypes, ECEMeshBuilderComponentType::BrushComponent))
			{
				continue;
			}
			
			if (AppendComponent(BrushComponent, InSourceTransform))
			{
				AppendPrimitiveComponents.Add(PrimitiveComponent);
			}
		}
		else if (UProceduralMeshComponent* ProceduralMeshComponent = Cast<UProceduralMeshComponent>(PrimitiveComponent))
		{
			if (!EnumHasAnyFlags(InParams.ComponentTypes, ECEMeshBuilderComponentType::ProceduralMeshComponent))
			{
				continue;
			}
			
			if (AppendComponent(ProceduralMeshComponent, InSourceTransform))
			{
				AppendPrimitiveComponents.Add(PrimitiveComponent);
			}
		}
		else if (UInstancedStaticMeshComponent* ISMComponent = Cast<UInstancedStaticMeshComponent>(PrimitiveComponent))
		{
			if (!EnumHasAnyFlags(InParams.ComponentTypes, ECEMeshBuilderComponentType::InstancedStaticMeshComponent))
			{
				continue;
			}
			
			if (AppendComponent(ISMComponent, InSourceTransform))
			{
				AppendPrimitiveComponents.Add(PrimitiveComponent);
			}
		}
		else if (USplineMeshComponent* SplineMeshComponent = Cast<USplineMeshComponent>(PrimitiveComponent))
		{
			if (!EnumHasAnyFlags(InParams.ComponentTypes, ECEMeshBuilderComponentType::SplineMeshComponent))
			{
				continue;
			}
			
			if (AppendComponent(SplineMeshComponent, InSourceTransform))
			{
				AppendPrimitiveComponents.Add(PrimitiveComponent);
			}
		}
		else if (UStaticMeshComponent* StaticMeshComponent = Cast<UStaticMeshComponent>(PrimitiveComponent))
		{
			if (!EnumHasAnyFlags(InParams.ComponentTypes, ECEMeshBuilderComponentType::StaticMeshComponent))
			{
				continue;
			}
			
			if (AppendComponent(StaticMeshComponent, InSourceTransform))
			{
				AppendPrimitiveComponents.Add(PrimitiveComponent);
			}
		}
		else if (UNiagaraComponent* NiagaraComponent = Cast<UNiagaraComponent>(PrimitiveComponent))
		{
			if (!EnumHasAnyFlags(InParams.ComponentTypes, ECEMeshBuilderComponentType::NiagaraComponent))
			{
				continue;
			}
			
			if (AppendComponent(NiagaraComponent, InSourceTransform))
			{
				AppendPrimitiveComponents.Add(PrimitiveComponent);
			}
		}
	}

	return AppendPrimitiveComponents;
}

bool FCEMeshBuilder::AppendMesh(const UDynamicMesh* InMesh, const TArray<TWeakObjectPtr<UMaterialInterface>>& InMaterials, const FTransform& InTransform)
{
	if (!IsValid(InMesh) || InMesh->GetTriangleCount() == 0)
	{
		return false;
	}

	return !!AddMeshInstance(InMesh->GetUniqueID(), InTransform, InMaterials, [&InMesh](FDynamicMesh3& InCreateMesh)->bool
	{
		// Create a copy of the mesh
		InMesh->ProcessMesh([&InCreateMesh](const FDynamicMesh3& EditMesh)
		{
			InCreateMesh = EditMesh;
		});

		return true;
	});
}

bool FCEMeshBuilder::AppendMesh(UStaticMesh* InMesh, const TArray<TWeakObjectPtr<UMaterialInterface>>& InMaterials, const FTransform& InSourceTransform)
{
	if (!IsValid(InMesh) || InMesh->GetNumTriangles(/** LOD*/ 0) == 0)
	{
		return false;
	}

	return !!AddMeshInstance(InMesh->GetUniqueID(), InSourceTransform, InMaterials, [this, &InMesh](FDynamicMesh3& InCreateMesh)->bool
	{
		// convert to dynamic mesh
		FGeometryScriptMeshReadLOD StaticMeshLOD;
		StaticMeshLOD.LODType = EGeometryScriptLODType::RenderData;

		FGeometryScriptCopyMeshFromAssetOptions OutputMeshOptions;
		OutputMeshOptions.bIgnoreRemoveDegenerates = false;
		OutputMeshOptions.bRequestTangents = false;
		OutputMeshOptions.bApplyBuildSettings = false;

		EGeometryScriptOutcomePins OutResult;
		UGeometryScriptLibrary_StaticMeshFunctions::CopyMeshFromStaticMesh(InMesh, OutputDynamicMesh, OutputMeshOptions, StaticMeshLOD, OutResult);

		if (OutResult != EGeometryScriptOutcomePins::Success)
		{
			return false;
		}

		OutputDynamicMesh->EditMesh([this, &InCreateMesh](FDynamicMesh3& EditMesh)
		{
			InCreateMesh = MoveTemp(EditMesh);

			// replace by empty mesh
			FDynamicMesh3 EmptyMesh;
			EditMesh = MoveTemp(EmptyMesh);
		}, EDynamicMeshChangeType::GeneralEdit, EDynamicMeshAttributeChangeFlags::Unknown, true);

		return true;
	});
}

bool FCEMeshBuilder::AppendComponent(const UStaticMeshComponent* InComponent, const FTransform& InSourceTransform)
{
	if (!IsValid(InComponent))
	{
		return false;
	}

	UStaticMesh* StaticMesh = InComponent->GetStaticMesh();

	if (!IsValid(StaticMesh))
	{
		return false;
	}

	const FTransform RelativeTransform = InComponent->GetComponentTransform().GetRelativeTransform(InSourceTransform);

	TArray<TWeakObjectPtr<UMaterialInterface>> Materials;
	Algo::Transform(InComponent->GetMaterials(), Materials, [](UMaterialInterface* InMaterial)
	{
		return InMaterial;
	});

	return AppendMesh(StaticMesh, Materials, RelativeTransform);
}

bool FCEMeshBuilder::AppendComponent(UProceduralMeshComponent* InComponent, const FTransform& InSourceTransform)
{
	if (!IsValid(InComponent))
	{
		return false;
	}

	const int32 SectionCount = InComponent->GetNumSections();

	if (SectionCount == 0)
	{
		return false;
	}

	// Transform the new mesh relative to the component
	const FTransform RelativeTransform = InComponent->GetComponentTransform().GetRelativeTransform(InSourceTransform);

	TArray<TWeakObjectPtr<UMaterialInterface>> Materials;
	Algo::Transform(InComponent->GetMaterials(), Materials, [](UMaterialInterface* InMaterial)
	{
		return InMaterial;
	});

	const uint32 MeshIndex = InComponent->GetUniqueID();

	return !!AddMeshInstance(MeshIndex, RelativeTransform, Materials, [this, &InComponent, &SectionCount](FDynamicMesh3& InCreateMesh)->bool
	{
		InCreateMesh.EnableAttributes();
		InCreateMesh.Attributes()->EnablePrimaryColors();
		InCreateMesh.Attributes()->EnableMaterialID();
		InCreateMesh.Attributes()->SetNumNormalLayers(1);
		InCreateMesh.Attributes()->SetNumUVLayers(1);
		InCreateMesh.Attributes()->SetNumPolygroupLayers(1);
		InCreateMesh.Attributes()->EnableTangents();

		using namespace UE::Geometry;

		FDynamicMeshColorOverlay* ColorOverlay = InCreateMesh.Attributes()->PrimaryColors();
		FDynamicMeshNormalOverlay* NormalOverlay = InCreateMesh.Attributes()->PrimaryNormals();
		FDynamicMeshUVOverlay* UVOverlay = InCreateMesh.Attributes()->PrimaryUV();
		FDynamicMeshMaterialAttribute* MaterialAttr = InCreateMesh.Attributes()->GetMaterialID();
		FDynamicMeshPolygroupAttribute* GroupAttr = InCreateMesh.Attributes()->GetPolygroupLayer(0);
		FDynamicMeshNormalOverlay* TangentOverlay = InCreateMesh.Attributes()->PrimaryTangents();

		for (int32 SectionIdx = 0; SectionIdx < SectionCount; SectionIdx++)
		{
			if (FProcMeshSection* Section = InComponent->GetProcMeshSection(SectionIdx))
			{
				if (Section->bSectionVisible)
				{
					TArray<int32> VtxIds;
					TArray<int32> NormalIds;
					TArray<int32> ColorIds;
					TArray<int32> UVIds;
					TArray<int32> TaIds;

					// copy vertices data (position, normal, color, UV, tangent)
					for (FProcMeshVertex& SectionVertex : Section->ProcVertexBuffer)
					{
						int32 VId = InCreateMesh.AppendVertex(SectionVertex.Position);
						VtxIds.Add(VId);

						int32 NId = NormalOverlay->AppendElement(static_cast<FVector3f>(SectionVertex.Normal));
						NormalIds.Add(NId);

						int32 CId = ColorOverlay->AppendElement(static_cast<FVector4f>(SectionVertex.Color));
						ColorIds.Add(CId);

						int32 UVId = UVOverlay->AppendElement(static_cast<FVector2f>(SectionVertex.UV0));
						UVIds.Add(UVId);

						int32 TaId = TangentOverlay->AppendElement(static_cast<FVector3f>(SectionVertex.Tangent.TangentX));
						TaIds.Add(TaId);
					}

					// copy tris data
					if (Section->ProcIndexBuffer.Num() % 3 != 0)
					{
						continue;
					}

					for (int32 Idx = 0; Idx < Section->ProcIndexBuffer.Num(); Idx+=3)
					{
						int32 VIdx1 = Section->ProcIndexBuffer[Idx];
						int32 VIdx2 = Section->ProcIndexBuffer[Idx + 1];
						int32 VIdx3 = Section->ProcIndexBuffer[Idx + 2];

						int32 VId1 = VtxIds[VIdx1];
						int32 VId2 = VtxIds[VIdx2];
						int32 VId3 = VtxIds[VIdx3];

						int32 TId = InCreateMesh.AppendTriangle(VId1, VId2, VId3, SectionIdx);

						if (TId < 0)
						{
							continue;
						}

						NormalOverlay->SetTriangle(TId, FIndex3i(NormalIds[VIdx1], NormalIds[VIdx2], NormalIds[VIdx3]), true);
						ColorOverlay->SetTriangle(TId, FIndex3i(ColorIds[VIdx1], ColorIds[VIdx2], ColorIds[VIdx3]), true);
						UVOverlay->SetTriangle(TId, FIndex3i(UVIds[VIdx1], UVIds[VIdx2], UVIds[VIdx3]), true);
						TangentOverlay->SetTriangle(TId, FIndex3i(TaIds[VIdx1], TaIds[VIdx2], TaIds[VIdx3]), true);

						MaterialAttr->SetValue(TId, SectionIdx);
						GroupAttr->SetValue(TId, SectionIdx);
					}
				}
			}
		}

		return true;
	});
}

bool FCEMeshBuilder::AppendComponent(UBrushComponent* InComponent, const FTransform& InSourceTransform)
{
	return AppendPrimitiveComponent(nullptr, InComponent, InSourceTransform);
}

bool FCEMeshBuilder::AppendComponent(const USkeletalMeshComponent* InComponent, const FTransform& InSourceTransform)
{
	if (!IsValid(InComponent))
	{
		return false;
	}

	USkeletalMesh* SkeletalMesh = InComponent->GetSkeletalMeshAsset();

	if (!IsValid(SkeletalMesh))
	{
		return false;
	}

	// Transform the new mesh relative to the component
	const FTransform RelativeTransform = InComponent->GetComponentTransform().GetRelativeTransform(InSourceTransform);

	TArray<TWeakObjectPtr<UMaterialInterface>> Materials;
	Algo::Transform(InComponent->GetMaterials(), Materials, [](UMaterialInterface* InMaterial)
	{
		return InMaterial;
	});

	return !!AddMeshInstance(SkeletalMesh->GetUniqueID(), RelativeTransform, Materials, [this, &SkeletalMesh](FDynamicMesh3& InCreateMesh)->bool
	{
		// convert to dynamic mesh
		FGeometryScriptMeshReadLOD SkeletalMeshLOD;
		SkeletalMeshLOD.LODType = EGeometryScriptLODType::SourceModel;

		FGeometryScriptCopyMeshFromAssetOptions OutputMeshOptions;
		OutputMeshOptions.bIgnoreRemoveDegenerates = false;
		OutputMeshOptions.bRequestTangents = false;
		OutputMeshOptions.bApplyBuildSettings = false;

		EGeometryScriptOutcomePins OutResult;
		UGeometryScriptLibrary_StaticMeshFunctions::CopyMeshFromSkeletalMesh(SkeletalMesh, OutputDynamicMesh, OutputMeshOptions, SkeletalMeshLOD, OutResult);

		if (OutResult != EGeometryScriptOutcomePins::Success)
		{
			return false;
		}

		OutputDynamicMesh->EditMesh([&InCreateMesh](FDynamicMesh3& EditMesh)
		{
			InCreateMesh = MoveTemp(EditMesh);

			// replace by empty mesh for next usage
			FDynamicMesh3 EmptyMesh;
			EditMesh = MoveTemp(EmptyMesh);
		}, EDynamicMeshChangeType::GeneralEdit, EDynamicMeshAttributeChangeFlags::Unknown, /** bDeferChanges */true);

		return true;
	});
}

bool FCEMeshBuilder::AppendComponent(UDynamicMeshComponent* InComponent, const FTransform& InSourceTransform)
{
	if (!IsValid(InComponent))
	{
		return false;
	}

	const UDynamicMesh* DynamicMesh = InComponent->GetDynamicMesh();

	if (!IsValid(DynamicMesh) || DynamicMesh->GetTriangleCount() == 0)
	{
		return false;
	}

	// Transform the new mesh relative to the component
	const FTransform RelativeTransform = InComponent->GetComponentTransform().GetRelativeTransform(InSourceTransform);

	// Copy all materials
	TArray<TWeakObjectPtr<UMaterialInterface>> Materials;
	Algo::Transform(InComponent->GetMaterials(), Materials, [](UMaterialInterface* InMaterial)
	{
		return InMaterial;
	});

	return AppendMesh(DynamicMesh, Materials, RelativeTransform);
}

bool FCEMeshBuilder::AppendComponent(UInstancedStaticMeshComponent* InComponent, const FTransform& InSourceTransform)
{
	if (!IsValid(InComponent) || !IsValid(InComponent->GetStaticMesh()))
	{
		return false;
	}

	return AppendPrimitiveComponent(nullptr, InComponent, InSourceTransform);
}

bool FCEMeshBuilder::AppendComponent(USplineMeshComponent* InComponent, const FTransform& InSourceTransform)
{
	if (!IsValid(InComponent) || !IsValid(InComponent->GetStaticMesh()))
	{
		return false;
	}

	return AppendPrimitiveComponent(nullptr, InComponent, InSourceTransform);
}

bool FCEMeshBuilder::AppendComponent(UNiagaraComponent* InComponent, const FTransform& InSourceTransform)
{
	if (!IsValid(InComponent))
	{
		return false;
	}

	UNiagaraSystem* System = InComponent->GetAsset();

	if (!IsValid(System))
	{
		return false;
	}

	struct FNiagaraSimCacheEmitterData
	{
		TArray<UNiagaraMeshRendererProperties*> MeshRenderers;
		TArray<FVector> ParticlePositions;
		TArray<FQuat> ParticleRotations;
		TArray<FVector> ParticleScales;
		TArray<int32> ParticleMeshIndexes;
	};

	TMap<FName, FNiagaraSimCacheEmitterData> EmittersData;

	// Set attributes to capture
	FNiagaraSimCacheCreateParameters Params;
	Params.AttributeCaptureMode = ENiagaraSimCacheAttributeCaptureMode::ExplicitAttributes;
	Params.bAllowDataInterfaceCaching = false;

	for (const FNiagaraEmitterHandle& EmitterHandle : System->GetEmitterHandles())
	{
		const FString EmitterName = EmitterHandle.GetUniqueInstanceName();

		Params.ExplicitCaptureAttributes.Add(FName(EmitterName + TEXT(".Particles.Position")));
		Params.ExplicitCaptureAttributes.Add(FName(EmitterName + TEXT(".Particles.MeshOrientation")));
		Params.ExplicitCaptureAttributes.Add(FName(EmitterName + TEXT(".Particles.Scale")));
		Params.ExplicitCaptureAttributes.Add(FName(EmitterName + TEXT(".Particles.MeshIndex")));

		if (const FVersionedNiagaraEmitterData* EmitterData = EmitterHandle.GetEmitterData())
		{
			for (UNiagaraRendererProperties* EmitterRenderer : EmitterData->GetRenderers())
			{
				if (UNiagaraMeshRendererProperties* MeshRenderer = Cast<UNiagaraMeshRendererProperties>(EmitterRenderer))
				{
					FNiagaraSimCacheEmitterData& CacheEmitterData = EmittersData.FindOrAdd(FName(EmitterName));
					CacheEmitterData.MeshRenderers.Add(MeshRenderer);
				}
			}
		}
	}

	if (EmittersData.IsEmpty())
	{
		return false;
	}

	UNiagaraSimCache* SimCache = UNiagaraSimCacheFunctionLibrary::CreateNiagaraSimCache(InComponent);

	if (!IsValid(SimCache))
	{
		return false;
	}

	const bool bSuccess = UNiagaraSimCacheFunctionLibrary::CaptureNiagaraSimCacheImmediate(SimCache, Params, InComponent, SimCache, /** AdvanceSim */ false);

	if (!bSuccess)
	{
		return false;
	}

	for (const FNiagaraEmitterHandle& Handle : System->GetEmitterHandles())
	{
		constexpr int32 FrameIndex = 0;
		constexpr bool bLocalToWorld = false;
		const FName EmitterName(Handle.GetUniqueInstanceName());

		FNiagaraSimCacheEmitterData& CacheEmitterData = EmittersData.FindChecked(EmitterName);

		SimCache->ReadPositionAttribute(CacheEmitterData.ParticlePositions, TEXT("Position"), EmitterName, bLocalToWorld, FrameIndex);
		SimCache->ReadQuatAttribute(CacheEmitterData.ParticleRotations, TEXT("MeshOrientation"), EmitterName, bLocalToWorld, FrameIndex);
		SimCache->ReadVectorAttribute(CacheEmitterData.ParticleScales, TEXT("Scale"), EmitterName, FrameIndex);
		SimCache->ReadIntAttribute(CacheEmitterData.ParticleMeshIndexes, TEXT("MeshIndex"), EmitterName, FrameIndex);

		if (!(CacheEmitterData.ParticlePositions.Num() == CacheEmitterData.ParticleRotations.Num()
			&& CacheEmitterData.ParticleRotations.Num() == CacheEmitterData.ParticleScales.Num()
			&& CacheEmitterData.ParticleScales.Num() == CacheEmitterData.ParticleMeshIndexes.Num()))
		{
			return false;
		}
	}

	const FTransform RelativeTransform = InComponent->GetComponentTransform().GetRelativeTransform(InSourceTransform);

	bool bResult = false;

	for (const TPair<FName, FNiagaraSimCacheEmitterData>& CacheEmitterDataPair : EmittersData)
	{
		const FNiagaraSimCacheEmitterData& CacheEmitterData = CacheEmitterDataPair.Value;

		for (int32 Index = 0; Index < CacheEmitterData.ParticlePositions.Num(); Index++)
		{
			const FVector& ParticlePosition = CacheEmitterData.ParticlePositions[Index];
			const FVector& ParticleScale = CacheEmitterData.ParticleScales[Index];
			const FQuat& ParticleRotation = CacheEmitterData.ParticleRotations[Index];
			const int32& ParticleMeshIndex = CacheEmitterData.ParticleMeshIndexes[Index];

			FTransform ParticleTransform(ParticleRotation, ParticlePosition, ParticleScale);

			if (ParticleMeshIndex >= 0)
			{
				for (UNiagaraMeshRendererProperties* MeshRenderer : CacheEmitterData.MeshRenderers)
				{
					if (!IsValid(MeshRenderer))
					{
						continue;
					}

					if (!MeshRenderer->Meshes.IsValidIndex(ParticleMeshIndex))
					{
						continue;
					}

					FNiagaraMeshRendererMeshProperties& Mesh = MeshRenderer->Meshes[ParticleMeshIndex];
					if (!IsValid(Mesh.Mesh)) //-V1051
					{
						continue;
					}

					FTransform MeshTransform(Mesh.Rotation.Quaternion(), Mesh.PivotOffset, Mesh.Scale);
					MeshTransform.Accumulate(ParticleTransform);
					MeshTransform.Accumulate(RelativeTransform);

					TArray<TWeakObjectPtr<UMaterialInterface>> Materials;
					for (int32 SectionIndex = 0; SectionIndex < Mesh.Mesh->GetNumSections(0); SectionIndex++)
					{
						Materials.Add(Mesh.Mesh->GetMaterial(SectionIndex));
					}

					bResult |= AppendMesh(Mesh.Mesh, Materials, MeshTransform);
				}
			}
		}
	}

	SimCache->MarkAsGarbage();

	return bResult;
}

bool FCEMeshBuilder::BuildDynamicMesh(UDynamicMesh* OutMesh, TArray<TWeakObjectPtr<UMaterialInterface>>& OutMaterials, const FCEMeshBuilderParams& InParams)
{
	if (!IsValid(OutMesh))
	{
		return false;
	}

	OutMaterials.Empty();

	const bool bDeferMeshChangeEvents = !InParams.bNotifyMeshChangeEvents;

	// Lets combine all meshes components from this actor together
	OutMesh->EditMesh([this, &OutMaterials, &InParams](FDynamicMesh3& InMergedMesh)
	{
		using namespace UE::Geometry;

		InMergedMesh.Clear();
		InMergedMesh.EnableAttributes();
		InMergedMesh.Attributes()->SetNumNormalLayers(1);

		FDynamicMeshEditor Editor(&InMergedMesh);
		FGeometryScriptAppendMeshOptions AppendOptions;
		AppendOptions.CombineMode = EGeometryScriptCombineAttributesMode::EnableAllMatching;
		int32 MaterialCount = 0;

		// Mesh index to material forward map to merge same meshes materials into same slot
		TMap<uint32, TMap<int32, int32>> MeshToMaterialMap;

		// Convert meshes
		for (const FCEMeshInstance& MeshInstance : MeshInstances)
		{
			if (!Meshes.Contains(MeshInstance.MeshIndex))
			{
				continue;
			}

			FDynamicMesh3& Mesh = Meshes[MeshInstance.MeshIndex];

			if (Mesh.TriangleCount() == 0)
			{
				continue;
			}

			// Apply transform
			FDynamicMesh3 ConvertedMesh = Mesh;
			MeshTransforms::ApplyTransform(ConvertedMesh, MeshInstance.MeshData.Transform);

			// Get materials
			if (!InParams.bMergeMaterials || !MeshToMaterialMap.Contains(MeshInstance.MeshIndex))
			{
				OutMaterials.Append(MeshInstance.MeshData.MeshMaterials);
			}

			// Enable matching attributes & append mesh
			FMeshIndexMappings TmpMappings;
			AppendOptions.UpdateAttributesForCombineMode(InMergedMesh, ConvertedMesh);
			Editor.AppendMesh(&ConvertedMesh, TmpMappings);

			// Fix triangles materials linking
			if (ConvertedMesh.HasAttributes() && ConvertedMesh.Attributes()->HasMaterialID())
			{
				const FDynamicMeshMaterialAttribute* FromMaterialIDAttrib = ConvertedMesh.Attributes()->GetMaterialID();
				FDynamicMeshMaterialAttribute* ToMaterialIDAttrib = InMergedMesh.Attributes()->GetMaterialID();

				TMap<int32, int32>& MaterialMap = MeshToMaterialMap.FindOrAdd(MeshInstance.MeshIndex);
				for (const TPair<int32, int32>& FromToTId : TmpMappings.GetTriangleMap().GetForwardMap())
				{
					const int32 FromMatId = FromMaterialIDAttrib->GetValue(FromToTId.Key);
					int32 ToMatId = FromMatId + MaterialCount;

					// used to merge materials for same mesh index
					if (const int32* MatId = MaterialMap.Find(FromMatId))
					{
						ToMatId = *MatId;
					}

					MaterialMap.Add(FromMatId, ToMatId);
					ToMaterialIDAttrib->SetNewValue(FromToTId.Value, ToMatId);
				}

				MaterialCount += MaterialMap.Num();

				if (!InParams.bMergeMaterials)
				{
					MaterialMap.Empty();
				}
			}
		}

		if (InMergedMesh.TriangleCount() > 0)
		{
			// Merge shared edges
			FMergeCoincidentMeshEdges WeldOp(&InMergedMesh);
			WeldOp.Apply();
		}
	}, EDynamicMeshChangeType::GeneralEdit, EDynamicMeshAttributeChangeFlags::Unknown, bDeferMeshChangeEvents);

	return true;
}

bool FCEMeshBuilder::BuildStaticMesh(UStaticMesh* OutMesh, TArray<TWeakObjectPtr<UMaterialInterface>>& OutMaterials, const FCEMeshBuilderParams& InParams)
{
	if (!IsValid(OutMesh))
	{
		return false;
	}

	ClearOutputMesh();

	if (!BuildDynamicMesh(OutputDynamicMesh, OutMaterials, InParams))
	{
		return false;
	}

	return DynamicMeshToStaticMesh(OutputDynamicMesh, OutMesh, OutMaterials);
}

bool FCEMeshBuilder::BuildStaticMesh(int32 InInstanceIndex, UStaticMesh* OutMesh, FCEMeshInstanceData& OutMeshInstance)
{
	if (!IsValid(OutMesh) || !MeshInstances.IsValidIndex(InInstanceIndex))
	{
		return false;
	}

	FCEMeshInstance& MeshInstance = MeshInstances[InInstanceIndex];

	if (!Meshes.Contains(MeshInstance.MeshIndex))
	{
		return false;
	}

	FDynamicMesh3& Mesh = Meshes[MeshInstance.MeshIndex];

	if (Mesh.TriangleCount() == 0)
	{
		return false;
	}

	OutMeshInstance.MeshMaterials.Empty(MeshInstance.MeshData.MeshMaterials.Num());
	OutMeshInstance.MeshMaterials.Append(MeshInstance.MeshData.MeshMaterials);

	OutMeshInstance.Transform = MeshInstance.MeshData.Transform;

	OutputDynamicMesh->SetMesh(Mesh);

	return DynamicMeshToStaticMesh(OutputDynamicMesh, OutMesh, OutMeshInstance.MeshMaterials);
}

bool FCEMeshBuilder::BuildDynamicMesh(int32 InInstanceIndex, UDynamicMesh* OutMesh, FCEMeshInstanceData& OutMeshInstance)
{
	if (!IsValid(OutMesh) || !MeshInstances.IsValidIndex(InInstanceIndex))
	{
		return false;
	}

	FCEMeshInstance& MeshInstance = MeshInstances[InInstanceIndex];

	if (!Meshes.Contains(MeshInstance.MeshIndex))
	{
		return false;
	}

	FDynamicMesh3& Mesh = Meshes[MeshInstance.MeshIndex];

	if (Mesh.TriangleCount() == 0)
	{
		return false;
	}

	OutMeshInstance.MeshMaterials.Empty(MeshInstance.MeshData.MeshMaterials.Num());
	OutMeshInstance.MeshMaterials.Append(MeshInstance.MeshData.MeshMaterials);

	OutMeshInstance.Transform = MeshInstance.MeshData.Transform;

	OutMesh->SetMesh(Mesh);

	return true;
}

bool FCEMeshBuilder::BuildStaticMesh(uint32 InMeshIndex, UStaticMesh* OutMesh, TArray<FCEMeshInstanceData>& OutMeshInstances)
{
	const FDynamicMesh3* Mesh = Meshes.Find(InMeshIndex);

	if (!IsValid(OutMesh) || !Mesh)
	{
		return false;
	}

	OutMeshInstances.Empty(MeshInstances.Num());
	for (const FCEMeshInstance& MeshInstance : MeshInstances)
	{
		if (MeshInstance.MeshIndex == InMeshIndex)
		{
			OutMeshInstances.Add(MeshInstance.MeshData);
		}
	}

	if (OutMeshInstances.IsEmpty())
	{
		return false;
	}

	OutputDynamicMesh->SetMesh(*Mesh);

	return DynamicMeshToStaticMesh(OutputDynamicMesh, OutMesh, OutMeshInstances[0].MeshMaterials);
}

bool FCEMeshBuilder::BuildDynamicMesh(uint32 InMeshIndex, UDynamicMesh* OutMesh, TArray<FCEMeshInstanceData>& OutMeshInstances)
{
	const FDynamicMesh3* Mesh = Meshes.Find(InMeshIndex);

	if (!IsValid(OutMesh) || !Mesh)
	{
		return false;
	}

	OutMeshInstances.Empty(MeshInstances.Num());
	for (const FCEMeshInstance& MeshInstance : MeshInstances)
	{
		if (MeshInstance.MeshIndex == InMeshIndex)
		{
			OutMeshInstances.Add(MeshInstance.MeshData);
		}
	}

	if (OutMeshInstances.IsEmpty())
	{
		return false;
	}

	OutMesh->SetMesh(*Mesh);

	return true;
}

bool FCEMeshBuilder::AppendPrimitiveComponent(const UObject* InMeshObject, UPrimitiveComponent* InComponent, const FTransform& InSourceTransform)
{
	if (!IsValid(InComponent))
	{
		return false;
	}

	// Transform the new mesh relative to the component
	const FTransform RelativeTransform = InComponent->GetComponentTransform().GetRelativeTransform(InSourceTransform);

	// Copy all materials
	TArray<TWeakObjectPtr<UMaterialInterface>> Materials;
	Materials.Reserve(InComponent->GetNumMaterials());
	for (int32 Index = 0; Index < InComponent->GetNumMaterials(); Index++)
	{
		Materials.Add(InComponent->GetMaterial(Index));
	}

	// Take mesh id or component id to find already converted mesh
	const uint32 MeshIndex = InMeshObject ? InMeshObject->GetUniqueID() : InComponent->GetUniqueID();

	return !!AddMeshInstance(MeshIndex, RelativeTransform, Materials, [this, &InComponent](FDynamicMesh3& InCreateMesh)->bool
	{
		constexpr FGeometryScriptCopyMeshFromComponentOptions Options;
		FTransform LocalToWorld = FTransform::Identity;

		// convert to dynamic mesh
		EGeometryScriptOutcomePins OutResult;
		UGeometryScriptLibrary_SceneUtilityFunctions::CopyMeshFromComponent(InComponent, OutputDynamicMesh, Options, false, LocalToWorld, OutResult);

		if (OutResult != EGeometryScriptOutcomePins::Success)
		{
			return false;
		}

		OutputDynamicMesh->EditMesh([&InCreateMesh](FDynamicMesh3& EditMesh)
		{
			InCreateMesh = MoveTemp(EditMesh);

			// replace by empty mesh
			FDynamicMesh3 EmptyMesh;
			EditMesh = MoveTemp(EmptyMesh);
		}, EDynamicMeshChangeType::GeneralEdit, EDynamicMeshAttributeChangeFlags::Unknown, /** bDeferChange */true);

		return true;
	});
}

bool FCEMeshBuilder::DynamicMeshToStaticMesh(UDynamicMesh* InMesh, UStaticMesh* OutMesh, const TArray<TWeakObjectPtr<UMaterialInterface>>& InMaterials)
{
	using namespace UE::Geometry;

	bool bResult = false;
	
	if (!InMesh || !OutMesh)
    {
        return bResult;
    }

	UStaticMeshDescription* StaticMeshDescription = OutMesh->GetStaticMeshDescription(0);

	if (!StaticMeshDescription)
	{
		StaticMeshDescription = UStaticMesh::CreateStaticMeshDescription(OutMesh);
	}

	if (!StaticMeshDescription)
	{
		return bResult;
	}

	// Mesh
	FMeshDescription& MeshDescription = StaticMeshDescription->GetMeshDescription();
	MeshDescription.Empty();

	InMesh->ProcessMesh([&MeshDescription](const FDynamicMesh3& InSourceMesh)
	{
		FConversionToMeshDescriptionOptions ConversionOptions;
		ConversionOptions.bUpdateTangents = true;
		ConversionOptions.bUpdateUVs = true;

		FDynamicMeshToMeshDescription Converter(ConversionOptions);
		Converter.Convert(&InSourceMesh, MeshDescription, /** CopyTangents */true);
	});

	// Materials
	TArray<FStaticMaterial> NewStaticMaterials;
	NewStaticMaterials.Reserve(InMaterials.Num());
	for (int32 MaterialIndex = 0; MaterialIndex < InMaterials.Num(); ++MaterialIndex)
	{
		FStaticMaterial& NewMaterial = NewStaticMaterials.AddDefaulted_GetRef();
		NewMaterial.MaterialInterface = InMaterials[MaterialIndex].Get();
		NewMaterial.MaterialSlotName = UE::AssetUtils::GenerateNewMaterialSlotName(NewStaticMaterials, NewMaterial.MaterialInterface, MaterialIndex);
		NewMaterial.ImportedMaterialSlotName = NewMaterial.MaterialSlotName;
		NewMaterial.UVChannelData = FMeshUVChannelInfo(1.f);
	}

	OutMesh->SetStaticMaterials(NewStaticMaterials);

	TPolygonGroupAttributesRef<FName> PolygonGroupImportedMaterialSlotNames = StaticMeshDescription->GetRequiredAttributes().GetPolygonGroupMaterialSlotNames();

	for (int32 SlotIdx = 0; SlotIdx < NewStaticMaterials.Num(); ++SlotIdx)
	{
		if (SlotIdx < PolygonGroupImportedMaterialSlotNames.GetNumElements())
		{
			PolygonGroupImportedMaterialSlotNames.Set(SlotIdx, NewStaticMaterials[SlotIdx].MaterialSlotName);
		}
	}

	// Build
	UStaticMesh::FBuildMeshDescriptionsParams Params;
	Params.bFastBuild = true;
	Params.bAllowCpuAccess = true;
	Params.bCommitMeshDescription = true;

	const TArray<const FMeshDescription*> MeshDescriptions {&MeshDescription};
	bResult = OutMesh->BuildFromMeshDescriptions(MeshDescriptions, Params);

	FStaticMeshOperations::ComputeTriangleTangentsAndNormals(MeshDescription);
	FStaticMeshOperations::ComputeTangentsAndNormals(MeshDescription, EComputeNTBsFlags::Normals);
	
#if WITH_EDITOR
	UStaticMesh::FBuildParameters BuildParameters;
	BuildParameters.bInSilent = true;
	BuildParameters.bInRebuildUVChannelData = true;
	BuildParameters.bInEnforceLightmapRestrictions = true;
	OutMesh->Build(BuildParameters);
#endif

	return bResult;
}

void FCEMeshBuilder::ClearOutputMesh() const
{
	if (OutputDynamicMesh)
	{
		OutputDynamicMesh->EditMesh([](FDynamicMesh3& EditMesh){ EditMesh.Clear(); });
	}
}

FCEMeshBuilder::FCEMeshInstance* FCEMeshBuilder::AddMeshInstance(uint32 InMeshIndex, const FTransform& InTransform, const TArray<TWeakObjectPtr<UMaterialInterface>>& InMaterials, TFunctionRef<bool(UE::Geometry::FDynamicMesh3&)> InCreateMeshFunction)
{
	FCEMeshInstance MeshInstance;
	MeshInstance.MeshIndex = InMeshIndex;
	MeshInstance.MeshData.Transform = InTransform;
	MeshInstance.MeshData.MeshMaterials = InMaterials;

	FDynamicMesh3* CachedMesh = Meshes.Find(InMeshIndex);

	if (!CachedMesh)
	{
		FDynamicMesh3 Mesh;

		ClearOutputMesh();

		if (InCreateMeshFunction(Mesh) && Mesh.TriangleCount() > 0)
		{
			CachedMesh = &Meshes.Add(InMeshIndex, MoveTemp(Mesh));
		}

		ClearOutputMesh();
	}

	if (CachedMesh)
	{
		return &MeshInstances.Add_GetRef(MeshInstance);
	}

	return nullptr;
}
