// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "DynamicMesh/DynamicMesh3.h"
#include "UObject/WeakObjectPtrTemplates.h"
#include "CEMeshBuilder.generated.h"

class AActor;
class UActorComponent;
class UBrushComponent;
class UDynamicMesh;
class UDynamicMeshComponent;
class UInstancedStaticMeshComponent;
class UMaterialInterface;
class UNiagaraComponent;
class UProceduralMeshComponent;
class USkeletalMeshComponent;
class USplineMeshComponent;
class UStaticMesh;
class UStaticMeshComponent;

/** Enumerates all component type that can be converted */
enum class ECEMeshBuilderComponentType : uint16
{
	None = 0,
	DynamicMeshComponent = 1 << 1,
	SkeletalMeshComponent = 1 << 2,
	BrushComponent = 1 << 3,
	ProceduralMeshComponent = 1 << 4,
	InstancedStaticMeshComponent = 1 << 5,
	SplineMeshComponent = 1 << 6,
	StaticMeshComponent = 1 << 7,
	NiagaraComponent = 1 << 8,
	All = DynamicMeshComponent | SkeletalMeshComponent | BrushComponent | ProceduralMeshComponent | InstancedStaticMeshComponent | SplineMeshComponent | StaticMeshComponent | NiagaraComponent
};

/** Struct used to build a mesh based out of other meshes class */
USTRUCT()
struct CLONEREFFECTORMESHBUILDER_API FCEMeshBuilder
{
	GENERATED_BODY()

	struct FCEMeshInstanceData
	{
		/** Transform to apply on the mesh instance */
		FTransform Transform;

		/** Materials applied on this instance */
		TArray<TWeakObjectPtr<UMaterialInterface>> MeshMaterials;
	};

	struct FCEMeshBuilderParams
	{
		/** Merge same meshes material slot in the final result */
		bool bMergeMaterials = false;
		/** Whether to call the PreMeshChangedEvent and MeshChangedEvents during the build */
		bool bNotifyMeshChangeEvents = false;
	};

	struct FCEMeshBuilderAppendParams
	{
		/** Component types to append */
		ECEMeshBuilderComponentType ComponentTypes = ECEMeshBuilderComponentType::All;

		/** Specific excluded components */
		TSet<UPrimitiveComponent*> ExcludeComponents;
	};

	static const FCEMeshBuilderParams DefaultBuildParams;
	static const FCEMeshBuilderAppendParams DefaultAppendParams;

	/** Checks if the component contains any geometry data */
	static bool HasAnyGeometry(UActorComponent* InComponent);

	/** Does the mesh builder supports this actor */
	static bool IsActorSupported(const AActor* InActor);

	/** Does the mesh builder supports this component */
	static bool IsComponentSupported(const UActorComponent* InComponent);

	FCEMeshBuilder();

	int32 GetMeshInstanceCount() const
	{
		return MeshInstances.Num();
	}

	int32 GetMeshCount() const
	{
		return Meshes.Num();
	}

	TArray<uint32> GetMeshIndexes() const;

	/** Resets the builder and clears data */
	void Reset();

	/** Appends supported components within actor */
	TArray<UPrimitiveComponent*> AppendActor(const AActor* InActor, const FTransform& InTransform, const FCEMeshBuilderAppendParams& InParams = DefaultAppendParams);

	/** Appends Static Mesh component */
	bool AppendComponent(const UStaticMeshComponent* InComponent, const FTransform& InTransform = FTransform::Identity);

	/** Appends Procedural Mesh component */
	bool AppendComponent(UProceduralMeshComponent* InComponent, const FTransform& InTransform = FTransform::Identity);

	/** Appends Brush component */
	bool AppendComponent(UBrushComponent* InComponent, const FTransform& InTransform = FTransform::Identity);

	/** Appends Skeletal Mesh component */
	bool AppendComponent(const USkeletalMeshComponent* InComponent, const FTransform& InTransform = FTransform::Identity);

	/** Appends Dynamic Mesh component */
	bool AppendComponent(UDynamicMeshComponent* InComponent, const FTransform& InTransform = FTransform::Identity);

	/** Appends Instanced Static Mesh component */
	bool AppendComponent(UInstancedStaticMeshComponent* InComponent, const FTransform& InTransform = FTransform::Identity);

	/** Appends Spline Mesh component */
	bool AppendComponent(USplineMeshComponent* InComponent, const FTransform& InTransform = FTransform::Identity);

	/** Appends Niagara component */
	bool AppendComponent(UNiagaraComponent* InComponent, const FTransform& InTransform = FTransform::Identity);

	/** Appends a dynamic mesh */
	bool AppendMesh(const UDynamicMesh* InMesh, const TArray<TWeakObjectPtr<UMaterialInterface>>& InMaterials, const FTransform& InTransform = FTransform::Identity);

	/** Appends a static mesh */
	bool AppendMesh(UStaticMesh* InMesh, const TArray<TWeakObjectPtr<UMaterialInterface>>& InMaterials, const FTransform& InTransform = FTransform::Identity);

	/** Builds a dynamic mesh by merging all the mesh data imported */
	bool BuildDynamicMesh(UDynamicMesh* OutMesh, TArray<TWeakObjectPtr<UMaterialInterface>>& OutMaterials, const FCEMeshBuilderParams& InParams = DefaultBuildParams);

	/** Builds a static mesh by merging all the mesh data imported */
	bool BuildStaticMesh(UStaticMesh* OutMesh, TArray<TWeakObjectPtr<UMaterialInterface>>& OutMaterials, const FCEMeshBuilderParams& InParams = DefaultBuildParams);

	/** Builds a static mesh for the specific instance index */
	bool BuildStaticMesh(int32 InInstanceIndex, UStaticMesh* OutMesh, FCEMeshInstanceData& OutMeshInstance);

	/** Builds a dynamic mesh for the specific instance index */
	bool BuildDynamicMesh(int32 InInstanceIndex, UDynamicMesh* OutMesh, FCEMeshInstanceData& OutMeshInstance);

	/** Builds a static mesh for the specific mesh index */
	bool BuildStaticMesh(uint32 InMeshIndex, UStaticMesh* OutMesh, TArray<FCEMeshInstanceData>& OutMeshInstances);

	/** Builds a dynamic mesh for the specific mesh index */
	bool BuildDynamicMesh(uint32 InMeshIndex, UDynamicMesh* OutMesh, TArray<FCEMeshInstanceData>& OutMeshInstances);

private:
	static bool DynamicMeshToStaticMesh(UDynamicMesh* InMesh, UStaticMesh* OutMesh, const TArray<TWeakObjectPtr<UMaterialInterface>>& InMaterials);

	struct FCEMeshInstance
	{
		/** Index of the mesh to use for this instance */
		uint32 MeshIndex;

		/** Data linked to this mesh instance */
		FCEMeshInstanceData MeshData;
	};

	FCEMeshInstance* AddMeshInstance(uint32 InMeshIndex, const FTransform& InTransform, const TArray<TWeakObjectPtr<UMaterialInterface>>& InMaterials, TFunctionRef<bool(UE::Geometry::FDynamicMesh3&)> InCreateMeshFunction);

	bool AppendPrimitiveComponent(const UObject* InMeshObject, UPrimitiveComponent* InComponent, const FTransform& InTransform);

	void ClearOutputMesh() const;

	TMap<uint32, UE::Geometry::FDynamicMesh3> Meshes;

	TArray<FCEMeshInstance> MeshInstances;

	UPROPERTY()
	TObjectPtr<UDynamicMesh> OutputDynamicMesh;
};
